{
    // Use IntelliSense to learn about possible attributes.
    // Hover to view descriptions of existing attributes.
    // For more information, visit: https://go.microsoft.com/fwlink/?linkid=830387
    "version": "0.2.0",
    "configurations": [
        {
            "name": "Python: Current File",
            "type": "python",
            "request": "launch",
            "program": "${file}",
            "args": ["test_small_blob_uploads"],
            "console": "integratedTerminal"
        },
        {
            "name": "Python: Benchmarker",
            "type": "python",
            "request": "launch",
            "cwd": "${workspaceFolder}/Benchmarks/benchmark-suite/local-files",
            "program": "${workspaceFolder}/Benchmarks/benchmark-suite/main.py",
            "console": "integratedTerminal",
            "args": [
                "--seed",
                "--host=http://localhost:57001",
                "--header=Authorization: Bearer eyJraWQiOiJhR0xWUGVCaExfV1lDaXNWQXNrTGZBMlRfdDRhS2p0eUs5aU5CcEpLVF9FIiwiYWxnIjoiUlMyNTYifQ.eyJ2ZXIiOjEsImp0aSI6IkFULlVVaFctaTRjODAycGhocDE5SlhpRGlJV1ZpMm5ndjFRVi1yTDJ2WW51X3ciLCJpc3MiOiJodHRwczovL2VwaWNnYW1lcy5va3RhLmNvbS9vYXV0aDIvYXVzbzY0NW9qaldWZFJJM2QweDciLCJhdWQiOiJhcGk6Ly9qdXBpdGVyIiwiaWF0IjoxNjM0NzQ1MDQ5LCJleHAiOjE2MzQ3NDg2NDksImNpZCI6IjBvYW9pMnlodHUzWEZOTzhTMHg3Iiwic2NwIjpbImNhY2hlX2FjY2VzcyJdLCJzdWIiOiIwb2FvaTJ5aHR1M1hGTk84UzB4NyIsIkZvcnRuaXRlIjoiMG9hb2kyeWh0dTNYRk5POFMweDciLCJTdG9yYWdlIjoiZnVsbCIsIlRyYW5zYWN0aW9uTG9nIjoiZnVsbCIsIlVFNCI6IjBvYW9pMnlodHUzWEZOTzhTMHg3IiwiVHJlZSI6ImZ1bGwiLCJDYWNoZSI6ImZ1bGwifQ.EYmu-zsVGHPhiexPmccFC1XAD9JeSwy-lIFH-hN6DWEQ1cwYmD7B6TGENAFHsAM3hVDtnIUFZ73LDC30B-634qK7_-ypup57aa-h-7baCOxlYeE0N7RPgo78EDk08Q9dRpVe3gDAAlhccExN4JxTX2r1q7kGw64Ijq02Bj4_RC59zI5Wm6-GhsTZlZa3_NF5Dp8ODpBkhtx2RHlE7eY13oTG1BFL_NNra1xmyhrcV_J5fsxvlPrwSyat5KXrI5TrRwUHLOq3aJ-sestqYq75E0jz4ky9oZriGYncfM-G_T4Vl9_EItOPnEBMmtLlBWqxlMQobY6P8wpuBbBJz4n7OA",
                "${input:test-to-run}"
            ]
        }
    ],
    "inputs": [
        {
          "id": "test-to-run",
          "type":"promptString",
          "description": "Arguments for benchmarker",
          "default": ""
        },
    ]
}