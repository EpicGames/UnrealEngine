function Upload-BenchmarkECR {
    Write-Host "Logging in to AWS ECR"
    aws ecr get-login-password --region us-east-1 | docker login --username AWS --password-stdin 728559092788.dkr.ecr.us-east-1.amazonaws.com

    Write-Host "Starting docker build"
    docker build -t benchmarker .

    Write-Host "Uploading to ECR"
    docker tag benchmarker 728559092788.dkr.ecr.us-east-1.amazonaws.com/jupiter_benchmark:latest
    docker push 728559092788.dkr.ecr.us-east-1.amazonaws.com/jupiter_benchmark:latest
}

function Upload-BenchmarkECR-Dev {
    Write-Host "Logging in to AWS ECR (Dev)"
    aws ecr get-login-password --region us-east-1 | docker login --username AWS --password-stdin 730468387612.dkr.ecr.us-east-1.amazonaws.com

    Write-Host "Starting docker build"
    docker build -t benchmarker .

    Write-Host "Uploading to ECR (Dev)"
    docker tag benchmarker 730468387612.dkr.ecr.us-east-1.amazonaws.com/jupiter_benchmark:latest
    docker push 730468387612.dkr.ecr.us-east-1.amazonaws.com/jupiter_benchmark:latest
}

function Upload-BenchmarkS3 {
    $scriptDir = $PSScriptRoot
    $archivePath = (Join-Path $scriptDir "benchmark.zip")
    Write-Host "Creating archive $archivePath"
    Compress-Archive -Path (Join-Path $scriptDir "*.py"),(Join-Path $scriptDir "*.txt"),(Join-Path $scriptDir "*.tar") -DestinationPath $archivePath -Force

    Write-Host "Uploading to S3"
    aws s3 cp $archivePath "s3://devtools-misc/jupiter-benchmark.zip"
}

