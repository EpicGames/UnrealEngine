FROM ubuntu AS build

WORKDIR /app
ENV DEBIAN_FRONTEND=noninteractive
RUN apt-get update
RUN apt-get install -y python3 pip

COPY vegeta_12.8.4_linux_386.tar .
RUN tar -xf vegeta_12.8.4_linux_386.tar

COPY requirements.txt .
RUN python3 -m pip install -r requirements.txt --break-system-packages
COPY *.py ./

# Labels used by datadog to understand the log output we produce
LABEL "com.datadoghq.ad.logs"='[{"source": "python", "service": "jupiter_benchmark"}]'
ENTRYPOINT ["python3", "main.py"]