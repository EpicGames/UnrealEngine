import logging
import json
import sys
import subprocess
import os.path
import random

import requests
from blake3 import blake3
import varint
import scipy

logger = logging.getLogger("helpers")


def blake3_to_iohash(blake3_hash):
    return blake3_hash[:20]  # return the first 20 bytes


def write_payload(args, content):
    blob_identifier = hash_content(content)
    blob_identifier_str = blob_identifier.hex()

    filename = os.path.join(args['payloads_dir'], blob_identifier_str)

    try:
        f = open(filename, 'wb')
        f.write(content)
    finally:
        if f is not None:
            f.close()

    return (blob_identifier_str, filename)


def generate_blob(args, seed, file_length):
    # generate a byte array of file_length,
    # hash it to determine its identifier and then write it to disk
    random.seed(seed)

    # randbytes is python 3.9 only, so we steal the implementation of randbytes and use that below to be compatible with older versions of Python
    #content = random.randbytes(file_length)
    content = random.getrandbits(file_length * 8).to_bytes(file_length, 'little')

    return write_payload(args, content)

# generates a file with random content were the size follows distributions from how large the cache of a game can be
def generate_game_asset(args, seed):
    def random_game_asset_size():
        value = random.random()
        if (value < 0.25): # 25% chance
            return 130
        if (value < 0.31): # 6% chance
            return 386
        if (value < 0.34): # 3% chance
            return 5136  
        if (value < 0.37): # 3% chance
            return 65608
        if (value < 0.41): # 4% chance
            return 350 # approximation for medium set
            
        if (value > 0.999): # 0.1 chance
            # our precision on really large assets is missing so making sure we have some of those
            # approximating a avg of the large assets
            return 84_186_659

        if (value > 0.99): # 1% chance
            # really large values have a special log distribution
            s_large = 1.1272032578046247,
            loc_large = 233474.20499173328,
            scale_large = 292597.08600436774
            return int(scipy.stats.lognorm(s=s_large, loc=loc_large, scale=scale_large).rvs(size=1, random_state=seed)[0])
        
        # follow lognormal distribution for the rest    
        s = 1.7898978392258396,
        loc = 489.70804173566125,
        scale = 13112.784274036905

        return int(scipy.stats.lognorm(s=s, loc=loc, scale=scale).rvs(size=1, random_state=seed)[0])

    random.seed(seed)
    
    file_length = random_game_asset_size()
    # generate a byte array of file_length,
    # hash it to determine its identifier and then write it to disk

    # randbytes is python 3.9 only, so we steal the implementation of randbytes and use that below to be compatible with older versions of Python
    #content = random.randbytes(file_length)
    content = random.getrandbits(file_length * 8).to_bytes(file_length, 'little')

    return write_payload(args, content)

def generate_uecb_singlefield(args, fieldName, attachments):
    # object with a single field
    compact_binary_data = b'\x02'  # object type

    field_payload = b'\x85'  # field type is uniform array with a field name
    # add the field name
    field_payload += len(fieldName,).to_bytes(1, byteorder=sys.byteorder)
    field_payload += bytearray(fieldName, 'ascii')

    array_payload = varint.encode(len(attachments))
    array_payload += b'\x0f'  # uniform array of binary attachments
    # write a uniform array field
    for attachment in attachments:
        assert(len(attachment) == 20)  # a io hash is 20 bytes
        array_payload += attachment

    field_payload += varint.encode(len(array_payload))
    field_payload += array_payload
    compact_binary_data += varint.encode(len(field_payload))
    compact_binary_data += field_payload
    return compact_binary_data


def get_content(path):
    try:
        f = open(path, 'rb')
        content = f.read()
        id = hash_content(content)
        return id.hex(), content
    finally:
        if f is not None:
            f.close()


def hash_content(content):
    hash = blake3(content)
    digest = hash.digest()
    return blake3_to_iohash(digest)


def hash_file(file):
    f = None
    try:
        f = open(file, 'rb')
        return hash_content(f.read())
    finally:
        if f is not None:
            f.close()


def run_vegeta(vegeta_args):
    args = [a for a in vegeta_args if a]
    logger.debug(f"Running vegeta with args: {args}")
    process = subprocess.Popen(["./vegeta", *args])
    process.communicate()


def write_test(test_name, test_contents, args):
    test_file = os.path.join(args['tests_dir'], test_name)
    f = open(test_file, 'w')
    f.writelines(test_contents)
    f.close()


def run_test(args, test_name, duration_seconds=5, rate_per_second=None):
    test_file = os.path.join(args['tests_dir'], test_name)
    results_file = os.path.join(args['reports_dir'], test_name)

    run_vegeta(["attack",
                f"-name={test_name}",
                f"-duration={duration_seconds}s" if duration_seconds is not None else '',
                f"-rate={rate_per_second}/s" if rate_per_second is not None else '',
                "-max-body=0", # do not capture the response body as it becomes very large
                "-workers=64", # bump the number of workers and thus the number of connections we start with
                f"-targets={test_file}",
                f"-output={results_file}"])

    json_result_path = os.path.join(args['reports_dir'], test_name + ".json")
    run_vegeta(["report", '-type', "json", "-output", json_result_path, results_file])
    json_file = None
    try:
        logger.debug(f"Parsing json result file: {json_result_path}")
        json_file = open(json_result_path)

        for json_line in json_file.readlines():
            json_blob = json.loads(json_line)

            logging_args = json_blob
            # add custom tags into the log event which we can use to filter out the data
            logging_args['test_name'] = test_name
            logging_args['url'] = args['host'] # we can not call this host as that is a reserved name in datadog
            logging_args['bytes_out_avg'] = int(json_blob['bytes_out']['total']) / duration_seconds
            logging_args['bytes_in_avg'] = int(json_blob['bytes_in']['total']) / duration_seconds

            logger.info("Test execution of test %s", test_name, extra=logging_args)

    except:
        if json_file:
            json_file.close()

def build_http_call(args, http_method, path, body=None, headers=[]):
    base = args['host']
    headers_all = []
    headers_all.extend(headers)
    headers_all.extend(args['headers'])
    if headers_all:
        headers_str = '\n' + '\n'.join(headers_all)
    else:
        headers_str = ''

    if body:
        body_str = f"\n@{body}"
    else:
        body_str = ''

    extra_newline = ''
    # A extra new line is expected if we have any header or body passed to the command
    if headers_str or body_str:
        extra_newline = '\n'
        
    return f"{http_method} {base}{path}{headers_str}{body_str}{extra_newline}\n"


def execute_http_call(session: requests.session, args : dict, http_method: str, path: str, body=None, headers=[], expected_status_code=200):
    base = args['host']

    headers_all = []
    headers_all.extend(headers)
    headers_all.extend(args['headers'])

    headers_dict = {
        header.split(':')[0].strip():
        header.split(':')[1].strip() 
        for header in headers_all
    }

    url = f"{base}{path}"
    result = session.request(
        http_method, url, headers=headers_dict, data=body)
    if result.status_code != expected_status_code:
        raise Exception(
            f"HTTP status code {result.status_code} returned with body: {result.text}")
    return result
