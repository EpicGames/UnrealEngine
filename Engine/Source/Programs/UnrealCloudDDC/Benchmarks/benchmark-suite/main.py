import argparse
import logging
import os
import os.path
import sys

import json_log_formatter

import tests


def seed(args):
    logging.info("Seeding data")
    for method in dir(tests):
        if (method.startswith("seed_")):
            exec(f"tests.{method}(args)")


def run_tests(tests_to_run, args):
    logging.info("Running tests: %s", tests_to_run)
    for test in tests_to_run:
        if test == 'all':
            for test in get_all_tests():
                logging.info(f"Running test: {test}")
                exec(f"tests.test_{test}(args)")
        else:
            logging.info(f"Running test: {test}")
            exec(f"tests.test_{test}(args)")


def get_all_tests():
    found_tests = []
    for method in dir(tests):
        if (method.startswith("test_")):
            found_tests.append(method[5:])

    return found_tests


def print_tests():
    logging.info("Tests that can be run:")

    for test in get_all_tests():
        logging.info(test)

class CustomJSONFormatter(json_log_formatter.JSONFormatter):
    def json_record(self, message, extra, record):
        extra['level'] = record.levelname
        return super(CustomJSONFormatter, self).json_record(message, extra, record)

def main():
    parser = argparse.ArgumentParser(
        description="UnrealCloudDDC Benchmarker - Tool for benchmarking UnrealCloudDDC")
    parser.add_argument('--seed', action="store_true",
                        help='Set to generate test data')
    parser.add_argument('--seed-remote', action="store_true",
                        help='Set to generate test data')
    parser.add_argument('--list', action="store_true", help='List known tests')
    # host.docker.internal is the ip of the hostmachine for docker
    parser.add_argument(
        '--host', type=str, default="http://host.docker.internal", help='The url to test against')
    parser.add_argument('--namespace', type=str, default="test.benchmark",
                        help='The jupiter namespace to upload contents to')
    parser.add_argument('--header', type=str, nargs='*',
                        help='Header to add to each request')
    parser.add_argument('--verbose', action="store_true",
                        help='Enable verbose logging')

    parser.add_argument('test', nargs='*', help='Which tests to execute')
    args = parser.parse_args()

    logLevel = 'DEBUG' if args.verbose else 'INFO'
    formatter = CustomJSONFormatter()
    handler = logging.StreamHandler(sys.stdout)
    handler.setFormatter(formatter)
    root = logging.getLogger()
    root.setLevel(logLevel)
    root.addHandler(handler)

    logging.info("UnrealCloudDDC Benchmarker")

    global_args = {
        'host': args.host,
        'tests_dir': os.path.join(".", "tests"),
        'payloads_dir': os.path.join(".", "payloads"),
        'reports_dir': os.path.join(".", "reports"),
        'headers': [] if args.header is None else args.header,
        'namespace': args.namespace,
        'seed-remote': args.seed_remote
    }

    for name in ['tests_dir', 'payloads_dir', 'reports_dir']:
        d = global_args[name]
        os.makedirs(d, exist_ok=True)

    hasDoneSomething = False
    if (args.seed):
        seed(global_args)
        hasDoneSomething = True

    if (args.list):
        print_tests()
        hasDoneSomething = True

    if (args.test):
        run_tests(args.test, global_args)
        hasDoneSomething = True

    if (not hasDoneSomething):
        parser.print_help()
    else:
        logging.info("UnrealCloudDDC Benchmarker finished its run")


if __name__ == "__main__":
    main()
