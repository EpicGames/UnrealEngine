import logging

import requests
import helpers

logger = logging.getLogger("tests")

start_seeds = {
    'small_blob_seed': 1001,
    'large_blob_seed': 1501,
    'refs_raw_seed': 2001,
    'compact_binary_seed': 3001,
    'game_seed': 5001
}


def seed_healthchecks(args):
    helpers.write_test('healthcheck_ready', helpers.build_http_call(
        args, 'GET', '/health/ready'), args)

    helpers.write_test('healthcheck_live', helpers.build_http_call(
        args, 'GET', '/health/live'), args)


def test_healthcheck_ready(args):
    helpers.run_test(args, 'healthcheck_ready', duration_seconds=15)


def test_healthcheck_live(args):
    helpers.run_test(args, 'healthcheck_live', duration_seconds=15)


def generate_and_upload_blobs(args, start_seed, count, file_length):
    ns = args['namespace']
    seed_remote = args['seed-remote']
    blobs = [helpers.generate_blob(args, start_seed + i, file_length)
             for i in range(0, count)]

    if seed_remote:
        session = requests.session()
        # upload the refs so that they are present for downloads
        logger.info(f"Seeding {count} blobs to server with size {file_length / 1024} kb")
        for id, payload_path in blobs:
            id, content = helpers.get_content(payload_path)
            helpers.execute_http_call(session, args, 'PUT', f"/api/v1/blobs/{ns}/{id}", body=content, headers=[
                                    "Content-Type: application/octet-stream"])
    return blobs

def seed_small_blobs(args):
    ns = args['namespace']

    seed = start_seeds['small_blob_seed']
    file_length = 64 * 1024  # 64kb blobs
    blob_count = 100
    blobs = generate_and_upload_blobs(args, seed, blob_count, file_length)

    test_contents_upload = [helpers.build_http_call(args, 'PUT', f"/api/v1/blobs/{ns}/{id}", body=body, headers=[
                                                    "Content-Type: application/octet-stream"]) for id, body in blobs]
    helpers.write_test('small_blob_uploads', test_contents_upload, args)

    test_contents_download = [helpers.build_http_call(
        args, 'GET', f"/api/v1/blobs/{ns}/{id}") for id, body in blobs]
    helpers.write_test('small_blob_downloads', test_contents_download, args)


def test_small_blob_uploads(args):
    helpers.run_test(args, 'small_blob_uploads', duration_seconds=60, rate_per_second=100)


def test_small_blob_downloads(args):
    helpers.run_test(args, 'small_blob_downloads', duration_seconds=60, rate_per_second=100)

def seed_large_blobs(args):
    ns = args['namespace']

    seed = start_seeds['large_blob_seed']
    file_length = 20 * 1024 * 1024  # 20MB blobs
    blob_count = 100
    blobs = generate_and_upload_blobs(args, seed, blob_count, file_length)

    test_contents_upload = [helpers.build_http_call(args, 'PUT', f"/api/v1/blobs/{ns}/{id}", body=body, headers=[
                                                    "Content-Type: application/octet-stream"]) for id, body in blobs]
    helpers.write_test('large_blob_uploads', test_contents_upload, args)

    test_contents_download = [helpers.build_http_call(
        args, 'GET', f"/api/v1/blobs/{ns}/{id}") for id, body in blobs]
    helpers.write_test('large_blob_downloads', test_contents_download, args)

def test_large_blob_uploads(args):
    helpers.run_test(args, 'large_blob_uploads', duration_seconds=60, rate_per_second=10)

def test_large_blob_downloads(args):
    helpers.run_test(args, 'large_blob_downloads', duration_seconds=60, rate_per_second=10)

def seed_refs(args):
    ns = args['namespace']
    bucket = 'default'
    seed_remote = args['seed-remote']

    seed = start_seeds['refs_raw_seed']
    file_length = 2 * 1024 * 1024  # 2 MB refs
    blobs = [helpers.generate_blob(args, seed + i, file_length)
             for i in range(0, 50)]

    refs = [(
        helpers.hash_content(b'ref_' + id.encode()).hex(),
        body) for id, body in blobs]

    if seed_remote:
        session = requests.session()
        # upload the refs so that they are present for downloads
        logger.info("Seeding refs to server")
        for ref_id, payload_path in refs:
            id, content = helpers.get_content(payload_path)
            helpers.execute_http_call(session, args, 'PUT', f"/api/v1/refs/{ns}/{bucket}/{ref_id}", body=content, headers=[
                                    "Content-Type: application/octet-stream", f"X-Jupiter-IoHash: {id}"])

    # we generate a ref hash based on the content id and a random string, the ref identifier is not intresting but needs to be stable
    test_contents_download = [helpers.build_http_call(
        args, 'GET', f"/api/v1/refs/{ns}/{bucket}/{helpers.hash_content(b'ref_' + id.encode()).hex()}.raw") for id, body in blobs]
    helpers.write_test('refs_raw_download', test_contents_download, args)

    test_contents_upload = [helpers.build_http_call(args, 'PUT', f"/api/v1/refs/{ns}/{bucket}/{helpers.hash_content(b'ref_' + id.encode()).hex()}", body=body, headers=[
                                                    "Content-Type: application/octet-stream", f"X-Jupiter-IoHash: {id}"]) for id, body in blobs]
    helpers.write_test('refs_raw_upload', test_contents_upload, args)


def test_refs_raw_download(args):
    helpers.run_test(args, 'refs_raw_download', duration_seconds=15)


def test_refs_raw_upload(args):
    helpers.run_test(args, 'refs_raw_upload', duration_seconds=15)


def seed_compact_binary(args):
    seed_remote = args['seed-remote']
    ns = args['namespace']
    bucket = 'default'

    seed = start_seeds['compact_binary_seed']
    file_length = 2 * 1024 * 1024  # 2 MB refs
    blobs = [helpers.generate_blob(args, seed + i, file_length)
             for i in range(0, 10)]
    compact_binaries = [(
        helpers.hash_content(b'ref_' + id.encode()).hex(),
        helpers.generate_uecb_singlefield(
            args, 'payload', attachments=[bytes.fromhex(id)]),
        body) for id, body in blobs]

    cb_payloads = {}
    for ref_id, cb, attachment_path in compact_binaries:
        cb_payloads[ref_id] = helpers.write_payload(args, cb)

    if seed_remote:
        session = requests.session()
        logger.info("Seeding compact binary attachments to server")
        # upload the blobs we will use as attachments
        for ref_id, cb, attachment_path in compact_binaries:
            id, content = helpers.get_content(attachment_path)
            helpers.execute_http_call(session, args, 'PUT', f"/api/v1/blobs/{ns}/{id}", body=content, headers=[
                                    "Content-Type: application/octet-stream"])

        logger.info("Seeding compact binary refs to server")
        # make sure the refs we which to download are present on the server
        for ref_id, cb, attachment_path in compact_binaries:
            hash = helpers.hash_content(cb).hex()
            helpers.execute_http_call(session, args, 'PUT', f"/api/v1/refs/{ns}/{bucket}/{ref_id}", body=cb, headers=[
                                    "Content-Type: application/x-ue-cb", f"X-Jupiter-IoHash: {hash}"])

    # we generate a ref hash based on the content id and a random string, the ref identifier is not intresting but needs to be stable
    test_contents_download = [helpers.build_http_call(
        args, 'GET', f"/api/v1/refs/{ns}/{bucket}/{ref_id}.uecb") for ref_id, cb, attachment_path in compact_binaries]
    helpers.write_test('uecb_download', test_contents_download, args)

    test_contents_package_download = [helpers.build_http_call(
        args, 'GET', f"/api/v1/refs/{ns}/{bucket}/{ref_id}.uecbpkg") for ref_id, cb, attachment_path in compact_binaries]
    helpers.write_test('uecb_pkg_download', test_contents_package_download, args)

    test_contents_upload = [helpers.build_http_call(args, 'PUT', f"/api/v1/refs/{ns}/{bucket}/{ref_id}", body=cb_payloads[ref_id][1], headers=[
                                                    "Content-Type: application/x-ue-cb", f"X-Jupiter-IoHash: {cb_payloads[ref_id][0]}"]) for ref_id, cb, attachment_path in compact_binaries]
    helpers.write_test('uecb_upload', test_contents_upload, args)


def test_uecb_download(args):
    helpers.run_test(args, 'uecb_download', duration_seconds=15)

def test_uecb_pkg_download(args):
    helpers.run_test(args, 'uecb_pkg_download', duration_seconds=15)

def test_uecb_upload(args):
    helpers.run_test(args, 'uecb_upload', duration_seconds=15)


def test_game_upload(args):
    helpers.run_test(args, 'game_upload', duration_seconds=120)

def test_game_download(args):
    helpers.run_test(args, 'game_download', duration_seconds=120)

def seed_game(args):
    seed = start_seeds['game_seed']
    seed_remote = args['seed-remote']
    ns = args['namespace']
    bucket = 'default'

    logger.info("Generating game blobs")
    blobs = [helpers.generate_game_asset(args, seed + i) for i in range(0, 1000)]

    compact_binaries = [(
        helpers.hash_content(b'ref_' + id.encode()).hex(),
        helpers.generate_uecb_singlefield(
            args, 'payload', attachments=[bytes.fromhex(id)]),
        body) for id, body in blobs
    ]

    cb_payloads = {}
    for ref_id, cb, attachment_path in compact_binaries:
        cb_payloads[ref_id] = helpers.write_payload(args, cb)
    if seed_remote:
        session = requests.session()
    
        logger.info("Seeding game blobs to server")
        # upload the blobs we will use as attachments
        for ref_id, cb, attachment_path in compact_binaries:
            id, content = helpers.get_content(attachment_path)
            helpers.execute_http_call(session, args, 'PUT', f"/api/v1/blobs/{ns}/{id}", body=content, headers=[
                                    "Content-Type: application/octet-stream"])

        logger.info("Seeding game refs to server")
        # make sure the refs we which to download are present on the server
        for ref_id, cb, attachment_path in compact_binaries:
            hash = helpers.hash_content(cb).hex()
            helpers.execute_http_call(session, args, 'PUT', f"/api/v1/refs/{ns}/{bucket}/{ref_id}", body=cb, headers=[
                                    "Content-Type: application/x-ue-cb", f"X-Jupiter-IoHash: {hash}"])

    # we generate a ref hash based on the content id and a random string, the ref identifier is not intresting but needs to be stable
    test_contents_package_download = [helpers.build_http_call(
        args, 'GET', f"/api/v1/refs/{ns}/{bucket}/{ref_id}.uecbpkg") for ref_id, cb, attachment_path in compact_binaries]
    helpers.write_test('game_download', test_contents_package_download, args)

    test_contents_upload = [helpers.build_http_call(args, 'PUT', f"/api/v1/refs/{ns}/{bucket}/{ref_id}", body=cb_payloads[ref_id][1], headers=[
                                                    "Content-Type: application/x-ue-cb", f"X-Jupiter-IoHash: {cb_payloads[ref_id][0]}"]) for ref_id, cb, attachment_path in compact_binaries]
    helpers.write_test('game_upload', test_contents_upload, args)