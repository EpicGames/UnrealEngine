# Intro
Under `benchmark-suite` you will find a python program and dockerfile to build it into a container that can run benchmarks against UnrealCloudDDC.

# Usage
We use vegeta to run these benchmarks and the python app generates test data which vegeta can use to run concurrent http tests.

The docker image will automatically run all tests on startup, so we usually use this to just start the docker image somewhere and having that run its tests.
The image expects to get a valid access token passed to it as a argument.
Usage:
```
docker build -t jupiter_benchmark .
docker run --network host jupiter_benchmark \
--seed --seed-remote --host <url-to-service> \
--header="Authorization: Bearer <access-token>" \
all
```

While the python script can be executed on any platform vegeta only works for linux, if you have a linux machine you can run these locally otherwise run them via docker.

# Expected results

These tests assumes they are running on a machine with good network and a lot of cores to generate enough load on your UnrealCloudDDC deployment, we typically run it with 100 gigabit networking, in the same datacenter as the deployment (to avoid network conditions) and 64 cores which is what is assumed for these expected results.

If you do want to factor in the networking conditions you can run this elsewhere but those results can not compared to these.

This a rough guide for what kind of results you should be seeing if your setup is correct

| Test                 | p50      | p95     |
|----------------------|----------|---------|
| healthcheck_ready    | 1.3 ms   | 1.8 ms  |
| uecb_download        | 1.5 ms   | 2.0 ms  |
| small_blob_uploads   | 6.6 ms   | 13.0 ms |
| small_blob_downloads | 2.1 ms   | 3 ms    |
| large_blob_uploads   | 470 ms   | 880 ms  |
| large_blob_download  | 34.4 ms  | 38.8 ms |
| refs_raw_download    | 5.7 ms   | 7 ms    |
| uecb_pkg_download    | 4.9 ms   | 5.6 ms  |
| game_download        | 3.4 ms   | 129 ms  |
| game_upload          | 42.2 ms  | 88 ms   |