######
# AWS configuration
######

serviceAccount:
  name: svc-unreal-cloud-ddc
  create: false

global:
  cloudProvider: "AWS"
  awsRegion: "%REGION_REPLACE%"
  awsRole: AssumeRoleWebIdentity # service accounts are IAM mapped
  
  siteName: "%REGION_REPLACE%"

######
# Set a disabled bearer handler, update this to your SSO
######
  auth:
    defaultScheme: ServiceAccount
    #defaultScheme: Bearer
    schemes:
      ServiceAccount: 
        implementation: ServiceAccount
      #Bearer: # example integration of a standard SSO
      #  implementation: JWTBearer
      #  JwtAudience: api://audience
      #  JwtAuthority: https://url-to-your-server

      #Bearer: # example integration integrating with Okta
      #  implementation: Okta
      #  OktaDomain: https://url-to-your-server
      #  OktaAuthorizationServerId: default
      #  JwtAudience: api://audience
      #  JwtAuthority: http://url-to-your-server/oauth2/auth_server_Id
    acls:
    - claims: 
      - groups=cloud-ddc-admin
      actions:
      - ReadObject
      - WriteObject
      - DeleteObject
      - DeleteBucket
      - DeleteNamespace
      - AdminAction

######
# Map the claims coming from your SSO or the service account to actual actions in Unreal CLoud DDC - see `Namespace access` in the cloud ddc readme for information on what actions can be mapped
######

  namespaces:
    Policies:
      ddc:
        acls:
        - actions: 
          - ReadObject
          - WriteObject
          claims: 
          - groups=app-cloud-ddc-project
######
# Kubernetes configuration
######

nodeSelector:
  cloud-ddc.epicgames.com/node-type: nvme

tolerations:
  - key: role
    operator: Equal
    value: unreal-cloud-ddc
    effect: NoSchedule

config:

######
# Scylla configuration
######
  Scylla: &scylla-connection-info
    ConnectionString: Contact Points=scylla-client.scylla.svc.cluster.local;Default Keyspace=jupiter;

    KeyspaceReplicationStrategy:
      class : "NetworkTopologyStrategy"
      %REGION_REPLACE%: 2

    LocalDatacenterName: "%REGION_REPLACE%"
    LocalKeyspaceSuffix: "%REGION_REPLACE_CLEAN%"

######
# S3 configuration
######
  S3: &s3-connection-info
    BucketName: %S3BUCKET_NAME%

######
# Auth configuration for the service token - this can be removed once you have updated to your SSO
######
  ServiceAccounts:
    Accounts:
    - Token: %SERVICE_ACCOUNT_TOKEN%
      Claims:
      - groups:app-cloud-ddc-project
      - groups:cloud-ddc-admin

######
# Worker configuration - setup to run GC right now - update the replication configuration if you want to enable replication between regions
######

worker:
  enabled: true

  imagePullSecrets: 
    - name: regcred

  nodeSelector:
    cloud-ddc.epicgames.com/node-type: general

  tolerations:
    - key: role
      operator: Equal
      value: unreal-cloud-ddc
      effect: NoSchedule
  
  config:
    Scylla: *scylla-connection-info
    S3: *s3-connection-info
    
    GC:
      # should only be true for a single region
      CleanOldRefRecords: true
      CleanOldBlobs: true

    Replication:
      Enabled: false

######
# Mount nvme drive
######
persistence:
  enabled: false
  volume:     
    hostPath:
      path: /data-nvme # we set up this to run on our attached nvme disk
      type: Directory

######
# Enable nginx proxy
######
nginx:
  enabled: true
  useDomainSockets: true

######
# Create a NLB
######
service:
  annotations:
    service.beta.kubernetes.io/aws-load-balancer-name: "unreal-cloud-ddc"
    service.beta.kubernetes.io/aws-load-balancer-type: "external"
    service.beta.kubernetes.io/aws-load-balancer-nlb-target-type: "ip"
    service.beta.kubernetes.io/aws-load-balancer-scheme: "internet-facing"
    service.beta.kubernetes.io/aws-load-balancer-healthcheck-protocol: "HTTP"
    service.beta.kubernetes.io/aws-load-balancer-healthcheck-port: "80"
    service.beta.kubernetes.io/aws-load-balancer-healthcheck-path: "/health/live"
    service.beta.kubernetes.io/aws-load-balancer-healthcheck-healthy-threshold: "2"
    service.beta.kubernetes.io/aws-load-balancer-healthcheck-unhealthy-threshold: "2"
    service.beta.kubernetes.io/aws-load-balancer-additional-resource-tags: application=unreal-cloud-ddc

  type: LoadBalancer
  portName: http
  port: 80
  targetPort: nginx-http
  extraPort:
  - name: http2
    port: 8080
    targetPort: nginx-h2