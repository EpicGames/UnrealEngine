
function Upload-AWSBootstrapperToS3Dev {
    $PSNativeCommandUseErrorActionPreference = $true; 
    $ErrorActionPreference = 'Stop';

    Write-Host "Compressing configuration files"
    Get-ChildItem -Path $PSScriptRoot -Exclude *.ps1,*.zip |  Compress-Archive -CompressionLevel Fastest -DestinationPath setup.zip -Force

    Write-Host "Uploading to S3"
    aws s3 cp setup.zip s3://devtools-misc/dev-cloud-ddc-bootstrapper/aws/setup.zip --acl public-read --region eu-north-1
}

function Upload-AWSBootstrapperToS3Prod {
    $PSNativeCommandUseErrorActionPreference = $true; 
    $ErrorActionPreference = 'Stop';

    Write-Host "Compressing configuration files"
    Get-ChildItem -Path $PSScriptRoot -Exclude *.ps1,*.zip |  Compress-Archive -CompressionLevel Fastest -DestinationPath setup.zip -Force

    Write-Host "Uploading to S3"
    aws s3 cp setup.zip s3://devtools-misc/cloud-ddc-bootstrapper/aws/setup.zip --acl public-read --region eu-north-1
}
