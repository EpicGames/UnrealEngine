#! /bin/bash
set -euo pipefail

REGION_REPLACE=$AWS_REGION
AZ_REPLACE_0=$(aws ec2 describe-availability-zones | jq -r .AvailabilityZones[0].ZoneName)
AZ_REPLACE_1=$(aws ec2 describe-availability-zones | jq -r .AvailabilityZones[1].ZoneName)

echo "Removing cloud ddc from region: \"$REGION_REPLACE\" using Availability zones: \"$AZ_REPLACE_0\" and \"$AZ_REPLACE_1\""

sed -i "s/\%REGION_REPLACE\%/$REGION_REPLACE/g" eksctl.yaml
sed -i "s/\%REGION_REPLACE\%/$REGION_REPLACE/g" cloud-ddc.yaml

sed -i "s/\%REGION_REPLACE\%/$REGION_REPLACE/g" scylla.cluster.yaml
sed -i "s/\%AZ_REPLACE_0\%/$AZ_REPLACE_0/g" scylla.cluster.yaml
sed -i "s/\%AZ_REPLACE_1\%/$AZ_REPLACE_1/g" scylla.cluster.yaml

echo "Removing Scylla monitoring..."
kubectl -n scylla delete ScyllaDBMonitoring/scylla-monitoring

echo "Removing Helm charts..."
helm uninstall unreal-cloud-ddc -n unreal-cloud-ddc

helm uninstall scylla -n scylla
helm uninstall scylla-manager -n scylla-manager
helm uninstall scylla-operator -n scylla-operator

helm uninstall prometheus-operator --namespace prometheus-operator

echo "Removing Scylla csi driver..."
kubectl -n local-csi-driver delete -f ./scylla-provisioner/local-csi-driver

kubectl delete pods -n scylla --all --force
echo "Warning! Not deleting S3 bucket, you will need to manually delete this ..."

echo "Deleting cluster..."
eksctl delete addon --name aws-ebs-csi-driver --cluster unreal-cloud-ddc
eksctl delete addon --name coredns --cluster unreal-cloud-ddc

eksctl delete cluster -f ./eksctl.yaml --parallel 4 --disable-nodegroup-eviction