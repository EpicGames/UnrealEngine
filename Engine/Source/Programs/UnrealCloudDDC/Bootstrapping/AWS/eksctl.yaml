apiVersion: eksctl.io/v1alpha5
kind: ClusterConfig

metadata:
  name: unreal-cloud-ddc
  region: %REGION_REPLACE%
  tags:
    created-by: CloudDDCBootstrapper

vpc:
  cidr: 192.168.0.0/19

availabilityZones:
- %AZ_REPLACE_0%
- %AZ_REPLACE_1%

addons:
- name: vpc-cni # no version is specified so it deploys the default version
  attachPolicyARNs:
    - arn:aws:iam::aws:policy/AmazonEKS_CNI_Policy
- name: coredns
  version: latest # auto discovers the latest available
- name: kube-proxy
  version: latest
- name: aws-ebs-csi-driver
  version: latest

managedNodeGroups:
  - name: nvme
    instanceType: i3en.xlarge
    desiredCapacity: 2
    labels:
      cloud-ddc.epicgames.com/node-type: nvme
    taints:
    - key: role
      value: unreal-cloud-ddc
      effect: NoSchedule
    preBootstrapCommands:
    - sudo mkfs.ext4 -E nodiscard /dev/nvme1n1
    - sudo mkdir /data-nvme
    - sudo mount /dev/nvme1n1 /data-nvme
  - name: general 
    instanceType: c5.xlarge
    desiredCapacity: 1
    labels:
      cloud-ddc.epicgames.com/node-type: general

nodeGroups:
  - name: scylla-pool
    instanceType: i4i.xlarge
    desiredCapacity: 2
    labels:
      scylla.scylladb.com/node-type: scylla
    taints:
    - key: role
      value: scylla-clusters
      effect: NoSchedule
    kubeletExtraConfig:
      cpuManagerPolicy: static

iam:
  withOIDC: true
  serviceAccounts:
  - metadata:
      name: svc-unreal-cloud-ddc
      namespace: unreal-cloud-ddc
      labels: {aws-usage: "application"}
    attachPolicyARNs:
    - "arn:aws:iam::aws:policy/AmazonS3FullAccess"
  - metadata:
      name: external-dns
      namespace: kube-system
    wellKnownPolicies:
      externalDNS: true
  - metadata:
      name: aws-load-balancer-controller
      namespace: kube-system
    wellKnownPolicies:
      awsLoadBalancerController: true
