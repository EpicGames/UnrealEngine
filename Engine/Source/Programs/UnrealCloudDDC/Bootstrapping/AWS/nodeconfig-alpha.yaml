apiVersion: scylla.scylladb.com/v1alpha1
kind: NodeConfig
metadata:
  name: cluster
spec:
  localDiskSetup:
    filesystems:
    - device: /dev/md/nvmes
      type: xfs
    mounts:
    - device: /dev/md/nvmes
      mountPoint: /mnt/persistent-volumes
      unsupportedOptions:
      - prjquota
    raids:
    - name: nvmes
      type: RAID0
      RAID0:
        devices:
          modelRegex: Amazon EC2 NVMe Instance Storage
          nameRegex: ^/dev/nvme\d+n\d+$
  placement:
    nodeSelector:
      scylla.scylladb.com/node-type: scylla
    tolerations:
    - effect: NoSchedule
      key: role
      operator: Equal
      value: scylla-clusters