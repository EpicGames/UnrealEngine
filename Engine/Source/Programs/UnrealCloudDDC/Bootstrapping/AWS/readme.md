This helps you setup a single region deployment of Unreal Cloud DDC on AWS if you have no or very limited previous experience setting up services on AWS.

If you are more familar with how AWS works we recommend using a IaC (Infrastructure as code) system to define these resources instead.
You can use this as a example of how to setup it up but its also possible to set this up without EKS and just using EC2 if you prefer.

## Prereqs
* An AWS account

## Setup

To install go to the AWS Management console and start AWS Cloudshell by hitting the >_ button on the top right side of the page (alternativley use the button on the bottom left labelled `CloudShell`).
Make sure you have selected the correct region you wish to create this in before starting AWS CloudShell.

Once this shell has started up run this command (note that this will take an hour or so to run, your cloud shell can timeout during this time but the command will continue to run):
```
nohup bash -c 'curl --silent -LO https://devtools-misc.s3.amazonaws.com/cloud-ddc-bootstrapper/aws/setup.zip && unzip -oq setup.zip && chmod +x setup.sh && ./setup.sh'
```

This command will do the following:
* Create a AWS EKS Cluster
* Setup the EKS cluster to run a instance of Scylla Open Source using Scylla Operator
* Install Cloud DDC onto the same cluster
* Create a S3 bucket for storage of blobs
* Setup simple service account auth (one shared key that can be used by everyone).

### Cost examples
This generates a bit of hardware that has flat costs so you should expect a run of this to end up with a cost about $1600 / month broken down like this:
* i3en instances - $700 (these are used to host the cloud ddc api)
* i3 instances - $120 (this hosts the scylla monitoring and mangement services)
* i4i instances - $500 (this hosts the actual scylla database)
* c5 instances - $130 (used to host the cloud ddc worker)

On top of this you will see cost for the load balancer and s3 bucket based on usage and some other minor AWS fees.
These cost are list price and without prepurchashing hardware.

### Making the cluster production ready
After completing the setup here follows some suggestions for preparing this cluster for production use which we strongly recommend you follow.

#### Setup nicer domain name
This requires you to setup a Route53 hosted zone which your domain can be in, AWS provides a guide for that [here|https://us-east-1.console.aws.amazon.com/route53/domains/home?region=us-east-1#/DomainSearch] .
Once this hosted zone is setup you need to add a annotation to your svc to indicate the domain name you want it to use, this can be achived by running this command:
`kubectl annotate svc unreal-cloud-ddc -n unreal-cloud-ddc "external-dns.alpha.kubernetes.io/hostname=<fully-qualifed-url-you-want-to-use>"`
After a few minutes a CName record should have been created in Route53 that points to your load balancer.

Alternativley update the `cloud-ddc.yaml` file with this annotation and run
`helm upgrade unreal-cloud-ddc oci://ghcr.io/epicgames/unreal-cloud-ddc --namespace unreal-cloud-ddc --values cloud-ddc.yaml`

#### Setup https
To use https you will need to allocate a certificate in AWS Cert Manager, you will usually need a domain name associated with it so its recommended to follow the `Setup nicer domain name` steps first.
Once you have a certificate allocated in Cert Manager you should take the ARN of this certificate and run this this command:
`kubectl annotate svc unreal-cloud-ddc -n unreal-cloud-ddc "service.beta.kubernetes.io/aws-load-balancer-ssl-ports=443,8080" "service.beta.kubernetes.io/aws-load-balancer-ssl-cert=<add-arn-here>"`
Alternativley update the `cloud-ddc.yaml` file with this annotation and run
`helm upgrade unreal-cloud-ddc oci://ghcr.io/epicgames/unreal-cloud-ddc --namespace unreal-cloud-ddc --values cloud-ddc.yaml`


After you have done this you should update load balancer to correctly report which http versions are supported which you can do using these commands:
```
aws modify-listener --listener-arn <arn-to-load-balancer-listener-port-80> --alpn-policy HTTP1Only
aws modify-listener --listener-arn <arn-to-load-balancer-listener-port-8080> --alpn-policy HTTP2Only
```

#### Enable SSO Auth
We strongly recommended to change auth mechanism to use a SSO method that supports `auth code` logins, which most SSOs will. To configure this on the editor side see `IdentityProvider setup` in the UnrealCloudDDC readme .
To change your deployment you will need to know the jwtAudience and jwtAuthority your SSO will usually tell you what these are. If you are using Okta we provide a special setup that integrates easier.
To update these settings edit the `cloud-ddc.yaml` file and change the `auth` section. For the standard SSO setup you should change it to this:
```
  auth:
    defaultScheme: Bearer
    schemes:
      ServiceAccount: 
        implementation: ServiceAccount
      Bearer: # example integration of a standard SSO
        implementation: JWTBearer
        JwtAudience: api://audience
        JwtAuthority: https://url-to-your-server
```
Note that we kept the ServiceAccount setup we use by default around here, this means that we accept both SSO logins and these service accounts. This can be useful while transitioning but its recommended to remove the ServiceAccount scheme once you have finished transitioning.

The okta setup is similar
```
  auth:
    defaultScheme: Bearer
    schemes:
      ServiceAccount: 
        implementation: ServiceAccount
      Bearer: # example integration integrating with Okta
        implementation: Okta
        OktaDomain: https://url-to-your-server
        OktaAuthorizationServerId: default
        JwtAudience: api://audience
        JwtAuthority: http://url-to-your-server/oauth2/auth_server_Id
```

Once you have updated the `cloud-ddc.yaml` you can run this command to update the kubernetes setup with the new configuration
`helm upgrade unreal-cloud-ddc oci://ghcr.io/epicgames/unreal-cloud-ddc --namespace unreal-cloud-ddc --values cloud-ddc.yaml`

## Other useful topics
Next follows some other useful topics

### Remoting into the machines
We do not setup ssh into any machines, but you can use AWS SSM to access the machines, for that you will need to know the EC2 instance id. To remote into that machine simply use this command:
`aws ssm start-session --target <ec2-instance-id like i-0000000000abc>`
You can also remote into the kubernetes pods themselves, this can be useful to inspect the local state of the filesystem cache.
First you need to find the pod name you want to remote into `kubectl get pods -n unreal-cloud-ddc` will show you all the pods and you can then use
`kubectl exec -it <pod-name> -c unreal-cloud-ddc -n unreal-cloud-ddc -- /bin/bash` to remote into the pod.

### Going multiregion
Scaling out to multiple regions is quite complicated, we recommend you make sure to consult with someone that understands the best way to achive that within your AWS setup and in a way that is compatible with your security considerations.

Note that you will need to aquire a Scylla Enterprise license before being able to scale to a multiregion deployment using Scylla Operator. Alternativley you can build your own manual cluster on EC2 using just Scylla Open Source.

Steps to take
1. The configurations generated by this script can be used as a foundation but you will need to run it manually. Start by updating these files:
* Update `eksctl.yaml` to use another of the `192.168` CIDRs.
* Update `scylla-cluster.yaml` - This may need to update the region and AZ specifications. You will also need to expose the cluster between your old and new VPC - described in this documentation: https://operator.docs.scylladb.com/stable/exposing.html#multi-vpc
* Update `cloud-ddc.yaml` - Adjust any region specific settings in here.
2. Update this new VPC to be peered with your previous VPC and setup routing between the regions.
3. Create a new S3 bucket for the new region.
3. Remote into one of your scylla nodes, and run `cqlsh` to update the Scylla replication factor.
4. Manually trigger a repair of scylla using the Scylla Manager.
5. Update the configuration of Unreal Cloud DDC to setup the replication between each region (see `Blob replication setup` in readme for Unreal Cloud DDC).
6. Setup a Route53 latency record that includes the public host names of both of your regions, update your UE ini files to use this latency record.


## Deleting the cluster
If you wish to remove the cluster you can run this script:
./delete.sh

If you ran this due to setup.sh failing it may not be able to delete everything in which case you can manually delete the EKS cluster by running this command from AWS CloudShell:
`eksctl delete cluster -f ./eksctl.yaml --parallel 4 --disable-nodegroup-eviction`