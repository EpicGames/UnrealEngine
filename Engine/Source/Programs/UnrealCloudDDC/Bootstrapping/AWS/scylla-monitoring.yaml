apiVersion: scylla.scylladb.com/v1alpha1
kind: ScyllaDBMonitoring
metadata:
  name: scylla-monitoring
  namespace: scylla
spec:
  type: Platform
  endpointsSelector:
    matchLabels:
      app.kubernetes.io/name: scylla
      scylla-operator.scylladb.com/scylla-service-type: identity
      scylla/cluster: scylla
  components:
    prometheus:
      storage:
        volumeClaimTemplate:
          spec:
            resources:
              requests:
                storage: 1Gi
    grafana:
      exposeOptions:
        webInterface:
          ingress:
            ingressClassName: alb
            dnsDomains:
            - scylla-grafana.svc.cluster.local