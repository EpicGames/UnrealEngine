apiVersion: storage.k8s.io/v1
kind: CSIDriver
metadata:
  name: local.csi.scylladb.com
spec:
  attachRequired: false
  storageCapacity: true
