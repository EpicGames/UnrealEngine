apiVersion: v1
kind: ServiceAccount
metadata:
  name: local-csi-driver
  namespace: local-csi-driver
