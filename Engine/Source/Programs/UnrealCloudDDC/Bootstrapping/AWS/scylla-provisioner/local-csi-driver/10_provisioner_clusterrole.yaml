kind: ClusterRole
apiVersion: rbac.authorization.k8s.io/v1
metadata:
  name: scylladb:csi-external-provisioner
rules:
- apiGroups:
  - ""
  resources:
  - "persistentvolumes"
  verbs:
  - "get"
  - "list"
  - "watch"
  - "create"
  - "delete"
- apiGroups:
  - ""
  resources:
  - "persistentvolumeclaims"
  verbs:
  - "get"
  - "list"
  - "watch"
  - "update"
- apiGroups:
  - "storage.k8s.io"
  resources:
  - "storageclasses"
  verbs:
  - "get"
  - "list"
  - "watch"
- apiGroups:
  - ""
  resources:
  - "events"
  verbs:
  - "list"
  - "watch"
  - "create"
  - "update"
  - "patch"
- apiGroups:
  - "snapshot.storage.k8s.io"
  resources:
  - "volumesnapshots"
  verbs:
  - "get"
  - "list"
- apiGroups:
  - "snapshot.storage.k8s.io"
  resources:
  - "volumesnapshotcontents"
  verbs:
  - "get"
  - "list"
- apiGroups:
  - "storage.k8s.io"
  resources:
  - "csinodes"
  verbs:
  - "get"
  - "list"
  - "watch"
- apiGroups:
  - ""
  resources:
  - "nodes"
  verbs:
  - "get"
  - "list"
  - "watch"
- apiGroups:
  - "storage.k8s.io"
  resources:
  - "csistoragecapacities"
  verbs:
  - "get"
  - "list"
  - "watch"
  - "create"
  - "update"
  - "patch"
  - "delete"
# The GET permissions below are needed for walking up the ownership chain
# for CSIStorageCapacity. They are sufficient for deployment via
# StatefulSet (only needs to get Pod) and Deployment (needs to get
# Pod and then ReplicaSet to find the Deployment).
- apiGroups:
  - ""
  resources:
  - "pods"
  verbs:
  - "get"
- apiGroups:
  - "apps"
  resources:
  - "replicasets"
  verbs:
  - "get"
