kind: ClusterRoleBinding
apiVersion: rbac.authorization.k8s.io/v1
metadata:
  name: scylladb:csi-external-provisioner
subjects:
- kind: ServiceAccount
  name: local-csi-driver
  namespace: local-csi-driver
roleRef:
  kind: ClusterRole
  name: scylladb:csi-external-provisioner
  apiGroup: rbac.authorization.k8s.io
