kind: DaemonSet
apiVersion: apps/v1
metadata:
  name: local-csi-driver
  namespace: local-csi-driver
  labels:
    app.kubernetes.io/name: local-csi-driver
spec:
  selector:
    matchLabels:
      app.kubernetes.io/name: local-csi-driver
  template:
    metadata:
      labels:
        app.kubernetes.io/name: local-csi-driver
    spec:
      nodeSelector:
        scylla.scylladb.com/node-type: scylla
        kubernetes.io/os: linux
      serviceAccountName: local-csi-driver
      tolerations:
      - effect: NoSchedule
        key: role
        operator: Equal
        value: scylla-clusters
      containers:
      - name: local-csi-driver
        securityContext:
          privileged: true
        image: docker.io/scylladb/k8s-local-volume-provisioner:0.1.0@sha256:d8c12e96d352077fd4b639c8e4fb050e49e0be11161d78d9339b69f73466f59a
        imagePullPolicy: Always
        args:
        - --listen=/csi/csi.sock
        - --node-name=$(NODE_NAME)
        - --volumes-dir=/mnt/persistent-volumes
        - --v=2
        env:
        - name: NODE_NAME
          valueFrom:
            fieldRef:
              fieldPath: spec.nodeName
        volumeMounts:
        - name: kubelet-dir
          mountPath: /var/lib/kubelet
          mountPropagation: "Bidirectional"
        - name: plugin-dir
          mountPath: /csi
        - name: volumes-dir
          mountPath: /mnt/persistent-volumes
        ports:
        - name: healthz
          containerPort: 9809
          protocol: TCP
        livenessProbe:
          httpGet:
            path: /healthz
            port: healthz
          initialDelaySeconds: 10
          timeoutSeconds: 3
          periodSeconds: 2
          failureThreshold: 5
      - name: csi-driver-registrar
        image: registry.k8s.io/sig-storage/csi-node-driver-registrar@sha256:fdff3ee285341bc58033b6b2458a5d45fd90ec6922a8ba6ebdd49b0c41e2cd34
        imagePullPolicy: Always
        args:
        - --csi-address=/csi/csi.sock
        - --kubelet-registration-path=/var/lib/kubelet/plugins/local.csi.scylladb.com/csi.sock
        volumeMounts:
        - name: plugin-dir
          mountPath: /csi
        - name: registration-dir
          mountPath: /registration
      - name: liveness-probe
        image: registry.k8s.io/sig-storage/livenessprobe@sha256:cacee2b5c36dd59d4c7e8469c05c9e4ef53ecb2df9025fa8c10cdaf61bce62f0
        imagePullPolicy: Always
        args:
        - --csi-address=/csi/csi.sock
        - --health-port=9809
        - --v=2
        volumeMounts:
        - name: plugin-dir
          mountPath: /csi
      - name: csi-provisioner
        image: registry.k8s.io/sig-storage/csi-provisioner@sha256:ee3b525d5b89db99da3b8eb521d9cd90cb6e9ef0fbb651e98bb37be78d36b5b8
        imagePullPolicy: Always
        args:
        - --csi-address=/csi/csi.sock
        - --v=2
        - --node-deployment
        - --feature-gates=Topology=true
        - --immediate-topology=false
        - --enable-capacity
        - --capacity-ownerref-level=0
        - --capacity-poll-interval=30s
        - --default-fstype=xfs
        env:
        - name: NAMESPACE
          valueFrom:
            fieldRef:
              fieldPath: metadata.namespace
        - name: POD_NAME
          valueFrom:
            fieldRef:
              fieldPath: metadata.name
        - name: NODE_NAME
          valueFrom:
            fieldRef:
              fieldPath: spec.nodeName
        volumeMounts:
        - name: plugin-dir
          mountPath: /csi
      volumes:
      - name: kubelet-dir
        hostPath:
          path: /var/lib/kubelet
          type: Directory
      - name: plugin-dir
        hostPath:
          path: /var/lib/kubelet/plugins/local.csi.scylladb.com/
          type: DirectoryOrCreate
      - name: registration-dir
        hostPath:
          path: /var/lib/kubelet/plugins_registry/
          type: Directory
      - name: volumes-dir
        hostPath:
          path: /mnt/persistent-volumes
          type: Directory
