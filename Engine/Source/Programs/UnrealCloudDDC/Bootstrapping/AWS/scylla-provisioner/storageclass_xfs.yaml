kind: StorageClass
apiVersion: storage.k8s.io/v1
metadata:
  name: scylladb-local-xfs
provisioner: local.csi.scylladb.com
volumeBindingMode: WaitForFirstConsumer
parameters:
  csi.storage.k8s.io/fstype: xfs
