serviceMonitor:
  create: true

network:
  hostNetworking: true
sysctls:
- "fs.aio-max-nr=2097152"
developerMode: false
datacenter: "%REGION_REPLACE%"
racks:
- name: a
  members: 1
  storage:
    storageClassName: scylladb-local-xfs
    capacity: 800G
  resources:
    limits:
      cpu: 3
      memory: 28G
  placement:
    nodeAffinity:
      requiredDuringSchedulingIgnoredDuringExecution:
        nodeSelectorTerms:
          - matchExpressions:
            - key: topology.kubernetes.io/zone
              operator: In
              values:
              - "%AZ_REPLACE_0%"
            - key: scylla.scylladb.com/node-type
              operator: In
              values:
              - scylla
    tolerations:
      - key: role
        operator: Equal
        value: scylla-clusters
        effect: NoSchedule
- name: b
  members: 1
  storage:
    storageClassName: scylladb-local-xfs
    capacity: 800G
  resources:
    limits:
      cpu: 3
      memory: 28G
  placement:
    nodeAffinity:
      requiredDuringSchedulingIgnoredDuringExecution:
        nodeSelectorTerms:
        - matchExpressions:
          - key: topology.kubernetes.io/zone
            operator: In
            values:
            - "%AZ_REPLACE_1%"
          - key: scylla.scylladb.com/node-type
            operator: In
            values:
            - scylla
    tolerations:
    - key: role
      operator: Equal
      value: scylla-clusters
      effect: NoSchedule