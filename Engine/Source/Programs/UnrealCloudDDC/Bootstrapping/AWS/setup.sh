#! /bin/bash
set -euo pipefail

REGION_REPLACE=$AWS_REGION
REGION_REPLACE_CLEAN=$(echo $AWS_REGION | sed -r 's/-/_/g')
AZ_REPLACE_0=$(aws ec2 describe-availability-zones | jq -r .AvailabilityZones[0].ZoneName)
AZ_REPLACE_1=$(aws ec2 describe-availability-zones | jq -r .AvailabilityZones[1].ZoneName)

if [ -f "/home/cloudshell-user/cloud-ddc.conf" ] 
then
  source /home/cloudshell-user/cloud-ddc.conf
else
  S3BUCKET_NAME_SUFFIX=$(date | md5sum | cut -d ' ' -f 1 | cut -b 1-8 | tr '[:upper:]' '[:lower:]')
  S3BUCKET_NAME=cloud-ddc-${S3BUCKET_NAME_SUFFIX}-${AWS_REGION}
  SERVICE_ACCOUNT_TOKEN=$(date | md5sum | cut -d ' ' -f 1)

  echo "S3BUCKET_NAME_SUFFIX=$S3BUCKET_NAME_SUFFIX
S3BUCKET_NAME=$S3BUCKET_NAME
SERVICE_ACCOUNT_TOKEN=$SERVICE_ACCOUNT_TOKEN
" > /home/cloudshell-user/cloud-ddc.conf  
fi

echo "Deploying to region: \"$REGION_REPLACE\" using Availability zones: \"$AZ_REPLACE_0\" and \"$AZ_REPLACE_1\""

echo "Updating configuration files..."
sed -i "s/\%REGION_REPLACE\%/$REGION_REPLACE/g" eksctl.yaml
sed -i "s/\%AZ_REPLACE_0\%/$AZ_REPLACE_0/g" eksctl.yaml
sed -i "s/\%AZ_REPLACE_1\%/$AZ_REPLACE_1/g" eksctl.yaml

sed -i "s/\%REGION_REPLACE\%/$REGION_REPLACE/g" cloud-ddc.yaml
sed -i "s/\%REGION_REPLACE_CLEAN\%/$REGION_REPLACE_CLEAN/g" cloud-ddc.yaml
sed -i "s/\%AZ_REPLACE_0\%/$AZ_REPLACE_0/g" cloud-ddc.yaml
sed -i "s/\%AZ_REPLACE_1\%/$AZ_REPLACE_1/g" cloud-ddc.yaml
sed -i "s/\%SERVICE_ACCOUNT_TOKEN\%/$SERVICE_ACCOUNT_TOKEN/g" cloud-ddc.yaml
sed -i "s/\%S3BUCKET_NAME\%/$S3BUCKET_NAME/g" cloud-ddc.yaml

sed -i "s/\%REGION_REPLACE\%/$REGION_REPLACE/g" scylla.cluster.yaml
sed -i "s/\%AZ_REPLACE_0\%/$AZ_REPLACE_0/g" scylla.cluster.yaml
sed -i "s/\%AZ_REPLACE_1\%/$AZ_REPLACE_1/g" scylla.cluster.yaml

chmod +x ./delete.sh

mkdir -p /home/cloudshell-user/bin

echo "Installing eksctl..."
curl --silent --location "https://github.com/weaveworks/eksctl/releases/latest/download/eksctl_$(uname -s)_amd64.tar.gz" | tar xz -C /tmp && sudo mv /tmp/eksctl /usr/local/bin && sudo cp /usr/local/bin/eksctl /home/cloudshell-user/bin/eksctl

echo "Installing helm..."
curl -LO https://get.helm.sh/helm-v3.11.0-linux-amd64.tar.gz && tar xfz helm-v3.11.0-linux-amd64.tar.gz && sudo mv linux-amd64/helm /usr/local/bin/ && sudo cp /usr/local/bin/helm /home/cloudshell-user/bin/helm

echo "Preparing AWS Resource Group for created resources"
aws resource-groups create-group --name unreal-cloud-ddc --description "Resource group of all resources created by the UnrealCloudDDC bootstrapper" --resource-query file://resource-query.json

if aws s3api head-bucket --bucket "$S3BUCKET_NAME" 2>/dev/null; then
  echo "S3 Bucket $S3BUCKET_NAME already present ..."
else
  echo "Creating S3 bucket ..."
  aws s3api create-bucket --bucket "$S3BUCKET_NAME" --region "$AWS_REGION" --acl private --create-bucket-configuration LocationConstraint=$AWS_REGION
  aws s3api put-bucket-tagging --bucket "$S3BUCKET_NAME" --region "$AWS_REGION" --tagging "TagSet=[{Key=created-by,Value=CloudDDCBootstrapper}]"
  
  echo "Created S3 bucket $S3BUCKET_NAME..."
fi

echo "Creating EKS cluster..."
eksctl create cluster -f ./eksctl.yaml

#This works around a strange issue in eksctl were it can't have this managed node group included as part of the rest of the cluster definitions
eksctl create nodegroup --cluster unreal-cloud-ddc --name monitoring-pool --node-type i3.large --nodes 1 --node-labels "pool=monitoring-pool"

echo "Adding helm repos..."
helm repo add eks https://aws.github.io/eks-charts
helm repo add external-dns https://kubernetes-sigs.github.io/external-dns/
helm repo add jetstack https://charts.jetstack.io
helm repo add scylla https://scylla-operator-charts.storage.googleapis.com/stable
helm repo update

echo "Installing AWS Load balancer..."
helm upgrade aws-load-balancer eks/aws-load-balancer-controller --install -n kube-system --wait --set serviceAccount.create=false,serviceAccount.name=aws-load-balancer-controller,clusterName=unreal-cloud-ddc
echo "Installing External-DNS..."
helm upgrade external-dns external-dns/external-dns --install -n kube-system --wait --set serviceAccount.create=false,serviceAccount.name=external-dns

echo "Starting the cert manger..."
helm install \
  cert-manager jetstack/cert-manager \
  --namespace cert-manager \
  --create-namespace \
  --version v1.13.1 \
  --set installCRDs=true
kubectl wait --for condition=established --timeout=60s crd/certificates.cert-manager.io crd/issuers.cert-manager.io
kubectl -n cert-manager rollout status --timeout=5m deployment.apps/cert-manager-webhook

echo "Installing prometheus..."
helm install prometheus-operator oci://registry-1.docker.io/bitnamicharts/kube-prometheus --namespace prometheus-operator --create-namespace 

echo "Starting the scylla operator..."
helm install scylla-operator scylla/scylla-operator --create-namespace --namespace scylla-operator
kubectl wait --for condition=established crd/nodeconfigs.scylla.scylladb.com
kubectl wait --for condition=established crd/scyllaclusters.scylla.scylladb.com
kubectl -n scylla-operator rollout status --timeout=5m deployment.apps/scylla-operator
kubectl -n scylla-operator rollout status --timeout=5m deployment.apps/webhook-server

# Configure nodes
echo "Configuring scylla nodes..."
kubectl apply --server-side -f nodeconfig-alpha.yaml
kubectl -n local-csi-driver apply --server-side -f ./scylla-provisioner/local-csi-driver/
kubectl apply --server-side -f ./scylla-provisioner/storageclass_xfs.yaml
echo "Your disks are ready to use."

echo "Installing scylla manager..."
helm install scylla-manager scylla/scylla-manager --create-namespace --namespace scylla-manager -f scylla.manager.yaml

echo "Installing ScyllaDB..."
helm install scylla scylla/scylla --create-namespace --namespace scylla -f scylla.cluster.yaml --wait
echo "Waiting for scylla nodes..."
kubectl wait pods -l "app.kubernetes.io/name=scylla" -n scylla --for condition=Ready --timeout=10m

echo "Configuring Scylla Monitoring..."
kubectl -n scylla apply --server-side -f ./scylla-monitoring.yaml

echo "Installing UnrealCloudDDC..."
helm upgrade unreal-cloud-ddc oci://ghcr.io/epicgames/unreal-cloud-ddc --install --namespace unreal-cloud-ddc --values cloud-ddc.yaml --wait --version 0.5.2+helm

# wait for the load balancer to get a host name
DDC_HOST=""
while [ -z $DDC_HOST ]; do
  echo "Waiting for external IP"
  DDC_HOST=$(kubectl get svc unreal-cloud-ddc --namespace unreal-cloud-ddc -o jsonpath="{.status.loadBalancer.ingress[*].hostname}")
  [ -z "$DDC_HOST" ] && sleep 10
done

echo "CloudDDC Installed and ready to use"  
echo " * Use \"$SERVICE_ACCOUNT_TOKEN\" as service account"
echo " * Use \"$DDC_HOST\" as host url"

echo "Please note that we strongly urge you to update this setup to use your SSO if you have one and to setup https, please see the documentation for more information."

echo "To verify that you can upload data you can use this command: \"curl http://$DDC_HOST/api/v1/refs/ddc/default/00000000000000000000000000000000000000aa -X PUT --data 'test' -H 'content-type: application/octet-stream' -H 'X-Jupiter-IoHash: 4878CA0425C739FA427F7EDA20FE845F6B2E46BA' -i -H 'Authorization: ServiceAccount $SERVICE_ACCOUNT_TOKEN'\""
echo "After uploading data you can fetch it using this command: \"curl http://$DDC_HOST/api/v1/refs/ddc/default/00000000000000000000000000000000000000aa.json -i -H 'Authorization: ServiceAccount $SERVICE_ACCOUNT_TOKEN'\""
echo "Please note that it may take a few minutes for this host to be resolvable by DNS, if you encounter issues about resolving the host then please wait a while and try again. "
echo "To run against this instance in your Unreal Editor you can add this to your DefaultEngine.ini - please note that this requires UE 5.4 to work"
echo ""
echo "[StorageServers]"
echo "Default=(Host=\"$DDC_HOST\", Namespace=\"ddc\", AuthScheme=\"ServiceAccount\", OAuthAccess=\"$SERVICE_ACCOUNT_TOKEN\")"
echo ""