<?xml version='1.0' ?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../../Engine/Build/Graph/Schema.xsd" >
	<Option Name="PreflightChange" DefaultValue="" Description="Preflight changelist number"/>
	
	<Property Name="EngineDir" Value="$(RootDir)/Engine"/>

	<Property Name="Version" Value="$(EngineMajorVersion).$(EngineMinorVersion).$(EnginePatchVersion)"/>
	<Property Name="InformationalVersion" Value="$(Version)-$(Change)"/>
	<Property Name="InformationalVersion" Value="$(InformationalVersion)-PF-$(PreflightChange)" If="'$(PreflightChange)' != ''"/>

	<Property Name="DotNetPath" Value=""/>
	<Property Name="DotNetPath" Value="$(RootDir)/Engine/Binaries/ThirdParty/DotNet/10.0/win-x64/dotnet.exe" If="$(HostPlatform) == 'Win64'"/>
	<Property Name="DotNetPath" Value="$(RootDir)/Engine/Binaries/ThirdParty/DotNet/10.0/linux-x64/dotnet" If="$(HostPlatform) == 'Linux'"/>

	<Agent Name="Jupiter" Type="Linux">
		<Property Name="ComposeFile" Value="$(RootDir)/Engine/Source/Programs/UnrealCloudDDC/Composes/docker-compose-tests.yml"/>
		<Node Name="Test Jupiter">
			<Docker-Compose-Up File="$(ComposeFile)"/>

			<!-- Run the unit tests, and override the dotnet sdk used to the global one for the machine rather then what is in the sync, to allow us to use newer versions then the rest of the engine has migrated to -->
			<DotNet Arguments="test Engine/Source/Programs/UnrealCloudDDC/UnrealCloudDDC.sln -l console;verbosity=normal --filter TestCategory!=SlowTests" DotNetPath="$(DotNetPath)" Environment="UE_DOTNET_VERSION=net8.0" />
		</Node>

		<Node Name="Build Jupiter" Requires="Test Jupiter">
			<!-- Tag all the files that need to be staged to build -->
			<Property Name="StagePaths">
				Engine/Source/Programs/Shared/...
				Engine/Source/Programs/UnrealCloudDDC/...
			</Property>
			<Tag Files="$(StagePaths)" Except=".../.vs/...;.../.git/...;.../bin/...;.../obj/..." With="#InputFiles"/>

			<!-- Build the Docker image -->
			<Docker-Build BaseDir="Engine/Source" Files="#InputFiles" Tag="unreal-cloud-ddc:$(InformationalVersion)" DockerFile="Engine/Source/Programs/UnrealCloudDDC/dockerfile" Environment="UE_P4_CHANGELIST=(Change)" UseBuildKit="true" Arguments="--target unreal-cloud-ddc-datadog"/>
		</Node>
	</Agent>
</BuildGraph>
