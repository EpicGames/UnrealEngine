Serilog:
  MinimumLevel:
    Default:
      Warning # Toogle this to Debug to get debug logging, can be quite a lot

Auth:
  Enabled: false
Jupiter:
  CurrentSite: docker-compose
  UseDomainSockets: true
  DomainSocketsRoot: /nginx
  ChmodDomainSockets: true
Replication:
  CurrentSite: docker-compose
Nginx:
  UseNginxRedirect: true
UnrealCloudDDC:
  storageImplementations:
  - Filesystem
Filesystem:
  RootDir: /data/blob-store
Scylla:
  ContactPoints:
  - scylla
  LocalDatacenterName: datacenter1
  LocalKeyspaceSuffix: dc1

Namespaces:
  Policies:
    # Define a single namespace that is open to anyone with a valid credential to use
    # As auth is disabled in this config this doesn't really matter and is provided mostly for reference
    test-namespace:
    - acls:
      actions:
      - ReadObject
      - WriteObject
      claims:
      - "*"