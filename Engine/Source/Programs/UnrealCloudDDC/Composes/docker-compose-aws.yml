version: '3'
services:
  unreal_cloud_ddc:
    environment:
    # NOTE: These are not real AWS credentials, but credentials for the local MinIO instance set up below.
    
    - AWSCredentials__AwsAccessKey=FSYPTVCPKC2IV5D1U8RE
    - AWSCredentials__AwsSecretKey=WaUf4Zq+J6YMlvbCVvTRkQJIPc7DvZwTm5UP0tPJ

    - AWS_ACCESS_KEY_ID=FSYPTVCPKC2IV5D1U8RE
    - AWS_SECRET_ACCESS_KEY=WaUf4Zq+J6YMlvbCVvTRkQJIPc7DvZwTm5UP0tPJ

    - UnrealCloudDDC__storageImplementations__1=s3
    - S3__ConnectionString=http://minio:9000
    - S3__ForceAWSPathStyle=true
    - S3__BucketName=docker-bucket
    - S3__SetBucketPolicies=false
    networks:
    - network_minio
  minio:
    image: "minio/minio"
    #logging:
    #  driver: none
    command: "minio server /data"
    environment:
    - "MINIO_ACCESS_KEY=FSYPTVCPKC2IV5D1U8RE"
    - "MINIO_SECRET_KEY=WaUf4Zq+J6YMlvbCVvTRkQJIPc7DvZwTm5UP0tPJ"
    volumes:
    - s3-volume:/data
    networks:
    - network_minio
volumes:
  s3-volume:

networks:
  network_minio:
    driver: bridge