version: '3'
services:
  unreal_cloud_ddc:
    environment:
    - UnrealCloudDDC__storageImplementations__1=azure
    - Azure__ConnectionString=DefaultEndpointsProtocol=https;BlobEndpoint=http://azure_storage:11002/blobaccount;AccountName=blobaccount;AccountKey=iU6uTvlF1ysppmft+NO5lAD0E3hwrAORr5Rb5xcBWUgEz/OicrSkFxwZYMNK5XL29/wXZKGOoOVSW040nAOfPg==
    
    networks:
    - network_azure_storage
  azure_storage:
    image: "mcr.microsoft.com/azure-blob-storage"
    logging:
      driver: none
    environment:
    - "LOCAL_STORAGE_ACCOUNT_NAME=blobaccount"
    - "LOCAL_STORAGE_ACCOUNT_KEY=iU6uTvlF1ysppmft+NO5lAD0E3hwrAORr5Rb5xcBWUgEz/OicrSkFxwZYMNK5XL29/wXZKGOoOVSW040nAOfPg=="
    volumes:
    - blob-volume:/data
    networks:
    - network_azure_storage
volumes:
  blob-volume:

networks:
  network_azure_storage:
    driver: bridge