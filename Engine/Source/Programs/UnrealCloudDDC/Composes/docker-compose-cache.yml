version: '3'
services:
  unreal_cloud_ddc:
    build:
      context: "../../../../"
      dockerfile: Programs/UnrealCloudDDC/dockerfile
    ports:
    - "58001:80" 
    - "58002:8080" 
    networks:
    - network_cache_bridge
    environment:
    - JUPITER_MODE=Cache
    - Jupiter__CurrentSite=docker-compose-cache
    - Serilog__MinimumLevel__Default=Debug
    - CORECLR_ENABLE_PROFILING=0
    - Mongo__ConnectionString=mongodb://root:mongo_root_passwd@mongo
    - Filesystem__RootDir=/data/blob-store
    - Upstream__ConnectionString=https://jupiter.devtools-dev.epicgames.com
    - Auth__Method=Disabled
    - Namespaces__Policies__*__Claims=*
    links:
    - mongo
    volumes:
    - jupiter-storage-volume:/data/blob-store
    labels:
      com.datadoghq.ad.logs: '[{"source":"csharp", "service": "jupiter"}]'
  mongo:
    image: mongo
    restart: always
    environment:
      MONGO_INITDB_ROOT_USERNAME: root
      MONGO_INITDB_ROOT_PASSWORD: mongo_root_passwd
    ports:
    - "58000:27017" 
    networks:
    - network_cache_bridge
    volumes:
    - mongo-volume:/data/db
    labels:
      com.datadoghq.ad.logs: '[{"source":"mongo", "service": "mongo"}]'
volumes:
  mongo-volume:
  jupiter-storage-volume:
networks:
  network_cache_bridge:
    driver: bridge
