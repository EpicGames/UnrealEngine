version: '3'
services:
  minio:
    image: "minio/minio"
    # Use to disable logging which can be spammy but also useful
    # logging:
    #  driver: none
    command: "minio server /data --console-address :9001"
    environment:
    - "MINIO_ACCESS_KEY=FSYPTVCPKC2IV5D1U8RE"
    - "MINIO_SECRET_KEY=WaUf4Zq+J6YMlvbCVvTRkQJIPc7DvZwTm5UP0tPJ"
    networks:
    - network_bridge
    ports:
    - "9000:9000"
    - "9001:9001" 
  mongo:
    image: mongo
    # Use to disable logging which can be spammy but also useful
    #logging:
    #  driver: none
    restart: always
    environment:
      MONGO_INITDB_ROOT_USERNAME: root
      MONGO_INITDB_ROOT_PASSWORD: abc123
    networks:
    - network_bridge
    ports:
    - "27018:27017"
  azure_storage:
    image: "mcr.microsoft.com/azure-blob-storage"
    # Use to disable logging which can be spammy but also useful
    #logging:
    #  driver: none
    environment:
    - "LOCAL_STORAGE_ACCOUNT_NAME=blobaccount"
    - "LOCAL_STORAGE_ACCOUNT_KEY=iU6uTvlF1ysppmft+NO5lAD0E3hwrAORr5Rb5xcBWUgEz/OicrSkFxwZYMNK5XL29/wXZKGOoOVSW040nAOfPg=="
    networks:
    - network_bridge
    ports:
    - "11002:11002"   
  scylla:
    image: scylladb/scylla:6.1
    command: --smp=1 --developer-mode 1
    # Use to disable logging which can be spammy but also useful
    #logging:
    # driver: none
    networks:
    - network_bridge
    ports:
    - "9042:9042" 

networks:
  network_bridge:
    driver: bridge
