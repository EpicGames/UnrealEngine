version: '3'
services:
  loadbalancer:
    build:
      context: .
      dockerfile: nginx_dockerfile
    ports:
    - "80:80" 
    - "8008:8008" 
    networks:
    - network_jupiter

    depends_on: 
      - unreal_cloud_ddc
    volumes:
     - jupiter-storage-volume:/data/blob-store
     - nginx-shared-volume:/nginx
  scylla:
    logging:
      driver: none
    image: scylladb/scylla:6.1
    command: --smp=1 --developer-mode 1
    volumes:
    - scylla-volume-1:/var/lib/scylla
    networks:
    - network_jupiter
    ports:
    - "9042:9042"
  unreal_cloud_ddc:
    build:
      context: "../../../"
      dockerfile: Programs/UnrealCloudDDC/dockerfile
    ports:
    - "8080:8080" 
    networks:
    - network_jupiter
    configs:
    - source: cloud_ddc_config
      target: /config/appsettings.Local.yaml
    environment:
    - CORECLR_ENABLE_PROFILING=0
    links:
    - scylla
    labels:
      com.datadoghq.ad.logs: '[{"source":"csharp", "service": "jupiter"}]'
    volumes:
     - jupiter-storage-volume:/data/blob-store
     - nginx-shared-volume:/nginx
volumes:
  scylla-volume-1:
  jupiter-storage-volume:
  nginx-shared-volume:
networks:
  network_jupiter:
    driver: bridge
configs:
  cloud_ddc_config:
    file: appsettings.Local.yaml
