# nginx config files for docker-compose load balancer

events {
    worker_connections   2048;
}

http {
    client_max_body_size 0;

    server_tokens off;
    sendfile on;
    tcp_nopush on;

    # increases the number of requests allowed per keepalive connection, reduces connection churn at a cost of some memory usage
    keepalive_requests 10000;

    upstream jupiter_backend {
      zone upstreams 64K;
      server unix:/nginx/jupiter-http.sock max_fails=1 fail_timeout=2s;
      keepalive 512;
    }

    server {

        listen 80 backlog=4096;

        proxy_buffering off;
        proxy_http_version 1.1;
    
        location /api {
          proxy_set_header X-Jupiter-XAccel-Supported true;
          proxy_set_header X-Jupiter-Port Public;
          proxy_set_header Connection "";
          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;
          proxy_set_header X-Real-IP       $remote_addr;
        }
        location /health {
          proxy_set_header X-Jupiter-XAccel-Supported true;
          proxy_set_header X-Jupiter-Port Public;
          proxy_set_header Connection "";
          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;
          proxy_set_header X-Real-IP       $remote_addr;
        }
        location /nginx-blobs {
          internal;
          alias /data/blob-store;

          # Forward Jupiter specific headers so they are not lost in the internal redirect
          add_header X-Jupiter-IoHash $upstream_http_x_jupiter_iohash;
          add_header X-Jupiter-LastAccess $upstream_http_x_jupiter_lastaccess;
          add_header X-Jupiter-InlinePayloadHash $upstream_http_x_jupiter_inlinepayloadhash;
          add_header Server-Timing $upstream_http_server_timing;
        }

        location / {
            return 404;
        }

        error_page 404 502 504 @error_page;
        location @error_page {
          default_type "application/json";
          return 200 '{"error_code": $status}'; # error_code is ignored
        }
    }

      # public http2 port
      server {
        listen 8008 backlog=4096;
        http2 on;
        server_name http2;

        proxy_buffering on;
        proxy_request_buffering off;
        proxy_http_version 1.1;
        
        location /api {
          proxy_set_header X-Jupiter-XAccel-Supported true;
          proxy_set_header X-Jupiter-Port Public;
          proxy_set_header Connection "";
          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;
        }
        location /health {
          proxy_set_header X-Jupiter-XAccel-Supported true;
          proxy_set_header X-Jupiter-Port Public;
          proxy_set_header Connection "";
          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;
        }
        location /nginx-blobs {
          internal;
          alias /data/blob-store;
          add_header X-Jupiter-XAccel-Used true;

          add_header X-Jupiter-IoHash $upstream_http_x_jupiter_iohash;
          add_header X-Jupiter-LastAccess $upstream_http_x_jupiter_lastaccess;
          add_header X-Jupiter-InlinePayloadHash $upstream_http_x_jupiter_inlinepayloadhash;
          add_header Server-Timing $upstream_http_server_timing;
        }

        location / {
            return 404;
        }

        error_page 404 502 504 @error_page;
        location @error_page {
          default_type "application/json";
          return 200 '{"error_code": "$status"}'; # error_code is ignored
        }
      }

}