FROM nginx
COPY nginx.conf /etc/nginx/nginx.conf