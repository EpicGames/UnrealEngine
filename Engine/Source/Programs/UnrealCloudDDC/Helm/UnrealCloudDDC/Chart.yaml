apiVersion: v2
name: unreal-cloud-ddc
description: A Helm chart for Kubernetes deploying UnrealCloudDDC

type: application

version: 0.4.0

icon: https://devtools-misc.s3.amazonaws.com/ue4logo.png
home: https://unrealengine.com

maintainers:
  - name: Joakim Lindqvist
    email: joakim.lindqvist@epicgames.com
  - name: UE-DevTools
    email: ue-devtools@epicgames.com
