# UnrealCloudDDC

UnrealCloudDDC - A Distributed storage service for use with Unreal Engine

# Components 

* UnrealCloudDDC - The service implementing most of the logic

# Chart Values

We recommend using a base value file for your general settings and then having a secondary values file for per-region settings. See example-values-base.yaml and example-values-base-A.yaml for a example of such a setup.

## Globals

| Name | Description | Possible Values |
| ------------- |:-------------:| -----:|
| cloudProvider | Which cloud provider this will be deployed on, impacts which services we interact with. | AWS or Azure |
| awsRegion | Only used for AWS cloud, which region you expect the services to be in | AWS Region format |
| awsRole | How to assume a IAM role in AWS. AssumeRoleWebIdentity assumes a kubernetes service account with a mapped IAM role. AssumeRole will explicitly run a assume role call on the arn. | AssumeRoleWebIdentity / AssumeRole / Basic |
| awsRoleArn | The arn of a AWS role to assume if using AssumeRole for awsRole | A valid ARN | 
| siteName | The name of this particular site within your cluster of Jupiter nodes. Should be unique amongst all your jupiter sites and is used to understand were a record has been replicated | Any globally unique value |


## Resolving secrets

UnrealCloudDDC supports resolving secrets for some configuration fields. The fields are:
* ServiceCredentials.OAuthClientId
* ServiceCredentials.OAuthClientSecret
* azureBlobStore.ConnectionString
* Scylla.ConnectionString

A secret can be just the raw text string if you want to (not recommended as this puts the secret in clear text in your kubernetes objects and in logs as well as the values files) instead we support using cloud specific formats for resolving the secrets.

### Resolving secret in AWS

The expected format for the secret string is:
`aws!<arn>|<secret-value>`

A example could look a bit like this:
`aws!arn:aws:secretsmanager:region-A:<aws-account-number>:secret:ddc-storage-oktaaccount|client_secret`

### Resolving secret in Azure

The expected format for the secret string is:
`akv!<vault-name>|<secret-name>`

A example could look a bit like this:
`akv!ddc-storage-keyvault|client_secret`


## Authentication setup
See the Authentication section in readme.md under the UnrealCloudDDC root for a example of how to setup authentication with namespaces


## Scylla configuration

Setting up scylla is documented in the readme.md in the UnrealCloudDDC root.

Scylla will require a connection string of where to connect to. Notice that scylla will find the layout of the cluster itself so you only need to point it to one node and it will find the best nodes to query. Example: `Contact Points=your-scylla-dns.your-domain.com;;Default Keyspace=jupiter;`

Each region also needs to setup some configuration of the local table (we keep a table that is not replicated within the cluster for data that is unique to each region):

```
    LocalKeyspaceReplicationStrategy:
      class : "NetworkTopologyStrategy"
      region-A : 2
      region-B: 0
    LocalDatacenterName: "region-ABC"
    LocalKeyspaceSuffix: "abc"
```
The local keyspace suffix is appened to the keyspace (database) name to make it globally unique.
The local datacenter name is used to make sure we attempt to query the local data center before hitting remote nodes and should be the name of the DC in scylla (usually the same name as the region name from your cloud provider) 
LocalKeyspaceReplicationStrategy should always be set to `NetworkTopologyStrategy` class and then you need to enumerate all the regions that exists - setting them to 0 except for the region which you are creating the configuration for (that should be set to 2).

## Enabling the worker deployment
You can enable the worker deployment by setting
```
worker:
  enabled: true
```
We generally recommend using this as it moves background task of a seperate deployment that doesnt take capacity away from your api serving.

Do note that any configuration override that is specified for your api deployment will
need to be duplicated if it also applies to the worker, if it is not specified under the global section. Most commonly this would be configuration with connection strings like `Scylla` and `S3`.