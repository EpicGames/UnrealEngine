# Patterns to ignore when building packages.
# This supports shell glob matching, relative path matching, and
# negation (prefixed with !). Only one pattern per line.
.DS_Store
# Common VCS dirs
.git/
.gitignore
.bzr/
.bzrignore
.hg/
.hgignore
.svn/
# Common backup files
*.swp
*.bak
*.tmp
*~
# Various IDEs
.project
.idea/
*.tmproj
.vscode/
# img folder
img/
# Changelog
CHANGELOG.md