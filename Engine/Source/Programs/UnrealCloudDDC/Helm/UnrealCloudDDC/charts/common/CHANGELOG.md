# Changelog

## 2.31.0 (2025-05-05)

* [bitnami/common] chore: :recycle: Remove deprecated APIs (<1.23.0) ([#33320](https://github.com/bitnami/charts/pull/33320))

## <small>2.30.2 (2025-04-30)</small>

* [bitnami/common] add namespaces to extraPodAffinityTerms for affinities (#33173) ([4e35d60](https://github.com/bitnami/charts/commit/4e35d6016945db7b9fd4eef72b177d4826d69ece)), closes [#33173](https://github.com/bitnami/charts/issues/33173)

## <small>2.30.1 (2025-04-30)</small>

* [bitnami/common] Allows tpl in provided passwords "common.secrets.passwords.manage" (#33196) ([1f53dd8](https://github.com/bitnami/charts/commit/1f53dd862f2aca1071f5734c3ba825e3ff4fa383)), closes [#33196](https://github.com/bitnami/charts/issues/33196)
* [bitnami/common] Restore 'Paremeters' section of the README (#32861) ([72f3f35](https://github.com/bitnami/charts/commit/72f3f353e35da99060a1662770655a12a2253887)), closes [#32861](https://github.com/bitnami/charts/issues/32861)

## 2.30.0 (2025-02-19)

* [bitnami/*] Use CDN url for the Bitnami Application Icons (#31881) ([d9bb11a](https://github.com/bitnami/charts/commit/d9bb11a9076b9bfdcc70ea022c25ef50e9713657)), closes [#31881](https://github.com/bitnami/charts/issues/31881)
* [bitnami/common] Add helper to check API versions (#31969) ([5ba89c5](https://github.com/bitnami/charts/commit/5ba89c5afc3d57e36f90364638d9beabb32499f4)), closes [#31969](https://github.com/bitnami/charts/issues/31969)
* Update copyright year (#31682) ([e9f02f5](https://github.com/bitnami/charts/commit/e9f02f5007068751f7eb2270fece811e685c99b6)), closes [#31682](https://github.com/bitnami/charts/issues/31682)

## <small>2.29.1 (2025-01-23)</small>

* [bitnami/common] Removing seLinuxOptions from omission (#31279) ([e7cb168](https://github.com/bitnami/charts/commit/e7cb168ca2bccd57e28bb985e099953a4f7e3b38)), closes [#31279](https://github.com/bitnami/charts/issues/31279) [#31278](https://github.com/bitnami/charts/issues/31278)

## 2.29.0 (2025-01-03)

* [bitnami/common] Add "common.capabilities.job.apiVersion" template (#31164) ([2ca979a](https://github.com/bitnami/charts/commit/2ca979a6add279384d60e6b35199eaf13553cefa)), closes [#31164](https://github.com/bitnami/charts/issues/31164)

## 2.28.0 (2024-12-10)

* [bitnami/common] New helper to detect non-standard images (#30851) ([ae33d01](https://github.com/bitnami/charts/commit/ae33d01968e8a353a569785f9867827153c797dc)), closes [#30851](https://github.com/bitnami/charts/issues/30851)

## <small>2.27.2 (2024-11-27)</small>

* [bitnami/common] Fix appVersion (#30628) ([b87d39e](https://github.com/bitnami/charts/commit/b87d39e27a0889c74b20c3d2fe5ae0c4a2417bfd)), closes [#30628](https://github.com/bitnami/charts/issues/30628)
* [bitnami/common]: bump patch version (#30639) ([623e584](https://github.com/bitnami/charts/commit/623e5846ff827b7ecbcffa2dc51e2e94b14ef8fd)), closes [#30639](https://github.com/bitnami/charts/issues/30639)

## <small>2.27.1 (2024-11-26)</small>

* [bitnami/common] Fix VPA apiVersion (#30625) ([8c24438](https://github.com/bitnami/charts/commit/8c24438a2f6e8ec646cad9901ed82d71d4196e3e)), closes [#30625](https://github.com/bitnami/charts/issues/30625)

## 2.27.0 (2024-11-07)

* [bitnami/*] Remove wrong comment about imagePullPolicy (#30107) ([a51f9e4](https://github.com/bitnami/charts/commit/a51f9e4bb0fbf77199512d35de7ac8abe055d026)), closes [#30107](https://github.com/bitnami/charts/issues/30107)
* [bitnami/common] feat: :sparkles: Add honorProvidedValues in common.secrets.manage (#30243) ([3d76a49](https://github.com/bitnami/charts/commit/3d76a4955c11fa4d2464da2c4d2096e1e3c6fa37)), closes [#30243](https://github.com/bitnami/charts/issues/30243)
* Update documentation links to techdocs.broadcom.com (#29931) ([f0d9ad7](https://github.com/bitnami/charts/commit/f0d9ad78f39f633d275fc576d32eae78ded4d0b8)), closes [#29931](https://github.com/bitnami/charts/issues/29931)

## 2.26.0 (2024-10-14)

* [bitnami/common] Drop unused custom empty password validators (#29432) ([5fb0e97](https://github.com/bitnami/charts/commit/5fb0e97d9336d40d86c3295637d4233218b8afea)), closes [#29432](https://github.com/bitnami/charts/issues/29432)

## 2.25.0 (2024-10-11)

* [bitnami/common] Add the ability to specify namespaces for affinity (#29479) ([005e0d6](https://github.com/bitnami/charts/commit/005e0d696004dd972915f290b7caffb2bc332400)), closes [#29479](https://github.com/bitnami/charts/issues/29479)

## 2.24.0 (2024-10-03)

* [bitnami/common] Add common.tplvalues.merge-overwrite helper (#29668) ([611b2a5](https://github.com/bitnami/charts/commit/611b2a59e06feaac878b3b218fd848a687216158)), closes [#29668](https://github.com/bitnami/charts/issues/29668)

## 2.23.0 (2024-09-13)

* [bitnami/common] Add option to remove empty seLinuxOptions from securityContext in non OpenShift env ([7e44e64](https://github.com/bitnami/charts/commit/7e44e64626f5b1fc6d56889cdfdeadc1f62c7cf1)), closes [#28945](https://github.com/bitnami/charts/issues/28945)

## 2.22.0 (2024-08-08)

* [bitnami/common] Fallback to chart appVersion in common.images.image (#28764) ([b4aa0a6](https://github.com/bitnami/charts/commit/b4aa0a685a21c50ca10e41e3eb2023bbd4282cf7)), closes [#28764](https://github.com/bitnami/charts/issues/28764)

## 2.21.0 (2024-08-05)

* [bitnami/common] Allow handling of new secrets after initial installation (#28581) ([07062ee](https://github.com/bitnami/charts/commit/07062ee01382e24b8204b27083ff3e8102110c2f)), closes [#28581](https://github.com/bitnami/charts/issues/28581)

## <small>2.20.5 (2024-07-16)</small>

* [bitnami/common] [bitnami/wordpress] Use global.storageClass for fallback, not override (#24863) ([2b78e13](https://github.com/bitnami/charts/commit/2b78e137ac278cdf9d54523e8d37833a4ff0cd5b)), closes [#24863](https://github.com/bitnami/charts/issues/24863)

## <small>2.20.4 (2024-07-11)</small>

* [bitnami/*] Update README changing TAC wording (#27530) ([52dfed6](https://github.com/bitnami/charts/commit/52dfed6bac44d791efabfaf06f15daddc4fefb0c)), closes [#27530](https://github.com/bitnami/charts/issues/27530)
* [bitnami/common] Increase ephemeral-storage default limits (#27902) ([dc0000d](https://github.com/bitnami/charts/commit/dc0000d7b56f68991bb8d8fff473103ed9026f5f)), closes [#27902](https://github.com/bitnami/charts/issues/27902)

## <small>2.20.3 (2024-06-17)</small>

* [bitnami/common] chore: :wrench: Relax large and xlarge presets resource requests (#27312) ([6ca69f6](https://github.com/bitnami/charts/commit/6ca69f6769d0f65acc850fa0bcc08506de50cc41)), closes [#27312](https://github.com/bitnami/charts/issues/27312)

## <small>2.20.2 (2024-06-10)</small>

* [bitnami/common] remove trailing spaces from imagePullSecrets rendering (#26882) ([362d4ac](https://github.com/bitnami/charts/commit/362d4ac94dd69be1b607fc531ceac4d67d8d57ef)), closes [#26882](https://github.com/bitnami/charts/issues/26882)

## <small>2.20.1 (2024-06-10)</small>

* [bitnami/common] improve renderSecurityContext (#27053) ([5f0bdde](https://github.com/bitnami/charts/commit/5f0bdde77cf05afa20cb4a800090748a8d102d02)), closes [#27053](https://github.com/bitnami/charts/issues/27053)

## 2.20.0 (2024-06-05)

* [bitnami/*] ci: :construction_worker: Add tag and changelog support (#25359) ([91c707c](https://github.com/bitnami/charts/commit/91c707c9e4e574725a09505d2d313fb93f1b4c0a)), closes [#25359](https://github.com/bitnami/charts/issues/25359)
* [bitnami/common] Capabilities to return latest apiVersion if kubeVersion is undefined (#26758) ([6582c32](https://github.com/bitnami/charts/commit/6582c3237b772af9cb379f7eaceddb2d64b507f0)), closes [#26758](https://github.com/bitnami/charts/issues/26758)
* [bitnami/common] docs: :memo: Add changelog ([23349c9](https://github.com/bitnami/charts/commit/23349c99b70313f3e19ebcf9d3e0c154836b2cc0))

## <small>2.19.3 (2024-05-20)</small>

* [bitnami/*] Change non-root and rolling-tags doc URLs (#25628) ([b067c94](https://github.com/bitnami/charts/commit/b067c94f6bcde427863c197fd355f0b5ba12ff5b)), closes [#25628](https://github.com/bitnami/charts/issues/25628)
* [bitnami/*] Set new header/owner (#25558) ([8d1dc11](https://github.com/bitnami/charts/commit/8d1dc11f5fb30db6fba50c43d7af59d2f79deed3)), closes [#25558](https://github.com/bitnami/charts/issues/25558)
* [bitnami/common] feat: :sparkles: Show warning when original images are replaced (#25952) ([855045a](https://github.com/bitnami/charts/commit/855045a1a62618154c1216e8da31a4d2c14c7586)), closes [#25952](https://github.com/bitnami/charts/issues/25952)

## <small>2.19.2 (2024-04-29)</small>

* [bitnami/common] Simplify syntax to deal with nullable objects (#25446) ([7dcea6a](https://github.com/bitnami/charts/commit/7dcea6aeb7c45d56bd6175b457bb8a2cddf8defc)), closes [#25446](https://github.com/bitnami/charts/issues/25446)
* Replace VMware by Broadcom copyright text (#25306) ([a5e4bd0](https://github.com/bitnami/charts/commit/a5e4bd0e35e419203793976a78d9d0a13de92c76)), closes [#25306](https://github.com/bitnami/charts/issues/25306)

## <small>2.19.1 (2024-03-27)</small>

* [bitnami/common] chore: :wrench: Relax preset resource requests xlarge and 2xlarge instances (#24713 ([fdd93bb](https://github.com/bitnami/charts/commit/fdd93bb2a2f73a7df3e498b5072736a54610a908)), closes [#24713](https://github.com/bitnami/charts/issues/24713)

## 2.19.0 (2024-03-08)

* [bitnami/common] feat: :sparkles: Remove empty seLinuxOptions in adapted Openshift rendered security ([1f2f5ef](https://github.com/bitnami/charts/commit/1f2f5ef476efba7f284df0c36c265216325ffda9)), closes [#24268](https://github.com/bitnami/charts/issues/24268)

## 2.18.0 (2024-03-04)

* [bitnami/common] feat: :sparkles: :lock: Add compatibility support for securityContext in Openshift  ([8fb0dd4](https://github.com/bitnami/charts/commit/8fb0dd48b6d7ec69bb59db2376365f6d76b26d97)), closes [#24040](https://github.com/bitnami/charts/issues/24040)

## 2.17.0 (2024-02-20)

* [bitnami/*] Bump all versions (#23602) ([b70ee2a](https://github.com/bitnami/charts/commit/b70ee2a30e4dc256bf0ac52928fb2fa7a70f049b)), closes [#23602](https://github.com/bitnami/charts/issues/23602)

## <small>2.16.1 (2024-02-19)</small>

* [bitnami/common] chore: :wrench: Bump ephemeral storage limits (#23564) ([18c4d88](https://github.com/bitnami/charts/commit/18c4d88f7d4ae93f36d0896fa66dbe872bba1c48)), closes [#23564](https://github.com/bitnami/charts/issues/23564)

## 2.16.0 (2024-02-15)

* [bitnami/common] feat: :sparkles: Add ephemeral-storage to resources preset (#23544) ([23b6856](https://github.com/bitnami/charts/commit/23b68563a0e2e721aa07864cff1b877e1d074388)), closes [#23544](https://github.com/bitnami/charts/issues/23544)

## <small>2.15.3 (2024-02-14)</small>

* [bitnami/common] chore: :pencil2: Fix typo in comment ([d07fb32](https://github.com/bitnami/charts/commit/d07fb324bd6455bf8607f66c642ff346443199ba))

## <small>2.15.2 (2024-02-14)</small>

* [bitnami/common] fix: :children_crossing: Improve resource warning message (#23425) ([7593e4f](https://github.com/bitnami/charts/commit/7593e4fc69fb8c50f7d626cc305c5adc56d23f48)), closes [#23425](https://github.com/bitnami/charts/issues/23425)

## <small>2.15.1 (2024-02-13)</small>

* [bitnami/common] fix: :bug: Check if section is enabled before printing resource warning ([262b6ee](https://github.com/bitnami/charts/commit/262b6ee64c57a5293333879ec423ad41c44f162c))

## 2.15.0 (2024-02-13)

* [bitnami/*] Fix docs.bitnami.com broken links (#21901) ([f35506d](https://github.com/bitnami/charts/commit/f35506d2dadee4f097986e7792df1f53ab215b5d)), closes [#21901](https://github.com/bitnami/charts/issues/21901)
* [bitnami/*] Move documentation sections from docs.bitnami.com back to the README (#22203) ([7564f36](https://github.com/bitnami/charts/commit/7564f36ca1e95ff30ee686652b7ab8690561a707)), closes [#22203](https://github.com/bitnami/charts/issues/22203)
* [bitnami/*] Update copyright: Year and company (#21815) ([6c4bf75](https://github.com/bitnami/charts/commit/6c4bf75dec58fc7c9aee9f089777b1a858c17d5b)), closes [#21815](https://github.com/bitnami/charts/issues/21815)
* [bitnami/common] feat: :sparkles: Add support for resource presets (#23410) ([310d9f9](https://github.com/bitnami/charts/commit/310d9f9e44cb913a2e482f57107970ed5bde9a69)), closes [#23410](https://github.com/bitnami/charts/issues/23410)

## <small>2.14.1 (2023-12-19)</small>

* [bitnami/common] Fix typo with new line in common.secrets.passwords.manage (#21653) ([7e70463](https://github.com/bitnami/charts/commit/7e704634ef564adac330f1e0a67feb2a40a271dc)), closes [#21653](https://github.com/bitnami/charts/issues/21653)

## 2.14.0 (2023-12-19)

* [bitnami/common] add params skipB64enc and skipQuote to common.secrets.passwords.manage (#21595) ([2070eeb](https://github.com/bitnami/charts/commit/2070eeb30bbf48639e0177a42f65a1d13f42a180)), closes [#21595](https://github.com/bitnami/charts/issues/21595)

## <small>2.13.4 (2023-12-15)</small>

* [bitnami/*] Remove relative links to non-README sections, add verification for that and update TL;DR ([1103633](https://github.com/bitnami/charts/commit/11036334d82df0490aa4abdb591543cab6cf7d7f)), closes [#20967](https://github.com/bitnami/charts/issues/20967)
* [bitnami/*] Rename VMware Application Catalog (#20361) ([3acc734](https://github.com/bitnami/charts/commit/3acc73472beb6fb56c4d99f929061001205bc57e)), closes [#20361](https://github.com/bitnami/charts/issues/20361)
* [bitnami/common] fix failOnNew implementation in common.secrets.passwords.manage (#21342) ([76a5f24](https://github.com/bitnami/charts/commit/76a5f248fbceb3d1d948c7e60fbba74fd7eb3200)), closes [#21342](https://github.com/bitnami/charts/issues/21342)
* [bitnami/common] Standardize documentation (#20334) ([3af2426](https://github.com/bitnami/charts/commit/3af242606877aea25c623b4185e6fcd285b7308d)), closes [#20334](https://github.com/bitnami/charts/issues/20334)

## <small>2.13.3 (2023-10-17)</small>

* [bitnami/*] Update Helm charts prerequisites (#19745) ([eb755dd](https://github.com/bitnami/charts/commit/eb755dd36a4dd3cf6635be8e0598f9a7f4c4a554)), closes [#19745](https://github.com/bitnami/charts/issues/19745)
* [bitnami/common]: Address admission configuration typo  (#19840) ([9a936f1](https://github.com/bitnami/charts/commit/9a936f158646e101c2507421fdcb85b787bbaf64)), closes [#19840](https://github.com/bitnami/charts/issues/19840)

## <small>2.13.2 (2023-10-05)</small>

* [bitnami/common] update imagePullSecrets to handle map and list format (#19702) ([1d30563](https://github.com/bitnami/charts/commit/1d30563bf53d4c0ac898cf1070af57aa28a039f1)), closes [#19702](https://github.com/bitnami/charts/issues/19702)

## <small>2.13.1 (2023-10-04)</small>

* [bitnami/common] render labels correctly when they contains templates (#19680) ([3cb44e3](https://github.com/bitnami/charts/commit/3cb44e376a472ca6721866b09f6d0ab412338cbc)), closes [#19680](https://github.com/bitnami/charts/issues/19680)

## 2.13.0 (2023-09-29)

* [bitnami/common]: Add capabilities macros to manage Pod Security Standard objects (#19428) ([322b76d](https://github.com/bitnami/charts/commit/322b76d6450840f08d53ecfddb5e151cac5c9e88)), closes [#19428](https://github.com/bitnami/charts/issues/19428)

## <small>2.12.1 (2023-09-29)</small>

* [bitnami/common] allow for empty appVersion (#19467) ([8b46a33](https://github.com/bitnami/charts/commit/8b46a3366abc7d216d16ace89675f3fc42691e8f)), closes [#19467](https://github.com/bitnami/charts/issues/19467)

## 2.12.0 (2023-09-22)

* [bitnami/common] new macro to checksum config resources (#19261) ([73945fe](https://github.com/bitnami/charts/commit/73945fedfa2acff03fe172430fcc4b8bcf55282f)), closes [#19261](https://github.com/bitnami/charts/issues/19261)
* Revert "Autogenerate schema files (#19194)" (#19335) ([73d80be](https://github.com/bitnami/charts/commit/73d80be525c88fb4b8a54451a55acd506e337062)), closes [#19194](https://github.com/bitnami/charts/issues/19194) [#19335](https://github.com/bitnami/charts/issues/19335)

## <small>2.11.1 (2023-09-15)</small>

* Common - Adding app.kubernetes.io/version to common labels (#17201) ([9c497be](https://github.com/bitnami/charts/commit/9c497be9d99a98a20cd01e5858014e097ebe0eaa)), closes [#17201](https://github.com/bitnami/charts/issues/17201)

## 2.11.0 (2023-09-12)

* [bitnami/common] New helper to return image version (#19223) ([db46696](https://github.com/bitnami/charts/commit/db466964c6cfb3368ab87be6bb4d16f74d5c6fd0)), closes [#19223](https://github.com/bitnami/charts/issues/19223)
* Autogenerate schema files (#19194) ([a2c2090](https://github.com/bitnami/charts/commit/a2c2090b5ac97f47b745c8028c6452bf99739772)), closes [#19194](https://github.com/bitnami/charts/issues/19194)

## <small>2.10.1 (2023-09-08)</small>

* [bitnami/common]: Compatiblity with Helm 3.2.0+ (#19177) ([e4fc03d](https://github.com/bitnami/charts/commit/e4fc03d96bef6ab0318d642fb65ba508c49844f1)), closes [#19177](https://github.com/bitnami/charts/issues/19177)

## 2.10.0 (2023-09-04)

* [bitnami/common] new macro to merge a list of values with rendering  (#18889) ([0fb66f2](https://github.com/bitnami/charts/commit/0fb66f2c6f6828a240a0c1e6857c337bf9f4202a)), closes [#18889](https://github.com/bitnami/charts/issues/18889)

## <small>2.9.2 (2023-08-31)</small>

* Avoid using a tpl when there is no template (#18792) ([134924a](https://github.com/bitnami/charts/commit/134924a260fe2cd758a954f34e89ccb14012f348)), closes [#18792](https://github.com/bitnami/charts/issues/18792)

## <small>2.9.1 (2023-08-29)</small>

* [bitnami/common] Add extraLabelSelectors to affinities templates (#18127) ([b9ecfdb](https://github.com/bitnami/charts/commit/b9ecfdb3421a057b76e6f35f58c26e631c74e686)), closes [#18127](https://github.com/bitnami/charts/issues/18127)

## 2.9.0 (2023-08-22)

* [bitnami/common] Add support for customizing standard labels (#18154) ([9a20483](https://github.com/bitnami/charts/commit/9a20483cfd1daa6bfe08fd8116516a9bb5cd9754)), closes [#18154](https://github.com/bitnami/charts/issues/18154)

## 2.8.0 (2023-08-07)

* [bitnami/common] Delete app kubernetes version field (#18240) ([5fe3ee4](https://github.com/bitnami/charts/commit/5fe3ee44eed88e9b6843c70cbeb6378194b2276b)), closes [#18240](https://github.com/bitnami/charts/issues/18240)

## 2.7.0 (2023-08-07)

* Add app.kubernetes.io/version based on AppVersion (#18194) ([4f698f8](https://github.com/bitnami/charts/commit/4f698f8ac54fc68cd8dab433b7c2d8ffb77a4067)), closes [#18194](https://github.com/bitnami/charts/issues/18194)

## 2.6.0 (2023-07-04)

* [bitnami/common] Add scope for common.tplvalues.render (#17033) ([daf1b54](https://github.com/bitnami/charts/commit/daf1b5445a5e1c961ab78673899dd8007b4f1000)), closes [#17033](https://github.com/bitnami/charts/issues/17033)

## 2.5.0 (2023-06-30)

* [bitnami/*] Change copyright section in READMEs (#17006) ([ef986a1](https://github.com/bitnami/charts/commit/ef986a1605241102b3dcafe9fd8089e6fc1201ad)), closes [#17006](https://github.com/bitnami/charts/issues/17006)
* [bitnami/common] Update common.secrets.passwords.manage and common.secrets.lookup (#17397) ([5a73cf1](https://github.com/bitnami/charts/commit/5a73cf19f92b93d88ee766669a947375135db903)), closes [#17397](https://github.com/bitnami/charts/issues/17397)
* [bitnami/several] Change copyright section in READMEs (#16989) ([5b6a5cf](https://github.com/bitnami/charts/commit/5b6a5cfb7625a751848a2e5cd796bd7278f406ca)), closes [#16989](https://github.com/bitnami/charts/issues/16989)
* Add copyright header (#17300) ([da68be8](https://github.com/bitnami/charts/commit/da68be8e951225133c7dfb572d5101ca3d61c5ae)), closes [#17300](https://github.com/bitnami/charts/issues/17300)
* Update charts readme (#17217) ([31b3c0a](https://github.com/bitnami/charts/commit/31b3c0afd968ff4429107e34101f7509e6a0e913)), closes [#17217](https://github.com/bitnami/charts/issues/17217)

## 2.4.0 (2023-05-18)

* [bitnami/common] feat: :sparkles: Add apiVersions for DaemonSet and VPA ([a86cfaf](https://github.com/bitnami/charts/commit/a86cfaf0acb7cc26a7a91256f4b76db8f31797ef))

## 2.3.0 (2023-05-12)

* Add wording for enterprise page (#16560) ([8f22774](https://github.com/bitnami/charts/commit/8f2277440b976d52785ba9149762ad8620a73d1f)), closes [#16560](https://github.com/bitnami/charts/issues/16560)
* Remove duplicate in image pull secrets (#16529) ([ddfea70](https://github.com/bitnami/charts/commit/ddfea70831875639cb298a555ad6dd5e68f059e4)), closes [#16529](https://github.com/bitnami/charts/issues/16529)

## <small>2.2.6 (2023-05-09)</small>

* [bitnami/several] Adapt Chart.yaml to set desired OCI annotations (#16546) ([fc9b18f](https://github.com/bitnami/charts/commit/fc9b18f2e98805d4df629acbcde696f44f973344)), closes [#16546](https://github.com/bitnami/charts/issues/16546)

## <small>2.2.5 (2023-05-02)</small>

* [bitnami/*] Make Helm charts 100% OCI (#15998) ([8841510](https://github.com/bitnami/charts/commit/884151035efcbf2e1b3206e7def85511073fb57d)), closes [#15998](https://github.com/bitnami/charts/issues/15998)
* [bitnami/common] Fix typo in README.md to test chart publishing from GitHub (#16143) ([5b05ec3](https://github.com/bitnami/charts/commit/5b05ec32caa73240d38135e19501ab2658397d2e)), closes [#16143](https://github.com/bitnami/charts/issues/16143)

## <small>2.2.4 (2023-03-07)</small>

* [bitnami/*] Fix markdown linter issues (#14874) ([a51e0e8](https://github.com/bitnami/charts/commit/a51e0e8d35495b907f3e70dd2f8e7c3bcbf4166a)), closes [#14874](https://github.com/bitnami/charts/issues/14874)
* [bitnami/*] Fix markdown linter issues 2 (#14890) ([aa96572](https://github.com/bitnami/charts/commit/aa9657237ee8df4a46db0d7fdf8a23230dd6902a)), closes [#14890](https://github.com/bitnami/charts/issues/14890)
* [bitnami/common] Allow empty registry name (#15296) ([f13df7b](https://github.com/bitnami/charts/commit/f13df7b00f38e5fce67eab7a1b78afb0b064344e)), closes [#15296](https://github.com/bitnami/charts/issues/15296)

## <small>2.2.3 (2023-02-03)</small>

* [bitnami/*] Add license annotation and remove obsolete engine parameter (#14293) ([da2a794](https://github.com/bitnami/charts/commit/da2a7943bae95b6e9b5b4ed972c15e990b69fdb0)), closes [#14293](https://github.com/bitnami/charts/issues/14293)
* [bitnami/*] Change copyright date (#14682) ([add4ec7](https://github.com/bitnami/charts/commit/add4ec701108ac36ed4de2dffbdf407a0d091067)), closes [#14682](https://github.com/bitnami/charts/issues/14682)
* [bitnami/*] Change licenses annotation format (#14377) ([0ab7608](https://github.com/bitnami/charts/commit/0ab760862c660fcc78cffadf8e1d8cdd70881473)), closes [#14377](https://github.com/bitnami/charts/issues/14377)
* [bitnami/*] Unify READMEs (#14472) ([2064fb8](https://github.com/bitnami/charts/commit/2064fb8dcc78a845cdede8211af8c3cc52551161)), closes [#14472](https://github.com/bitnami/charts/issues/14472)
* [bitnami/common] chore: Correct common.images.image global in example (#14735) ([69ada7d](https://github.com/bitnami/charts/commit/69ada7da0c9c6b7ce718faef6920c61e3632fd02)), closes [#14735](https://github.com/bitnami/charts/issues/14735)

## <small>2.2.2 (2022-12-12)</small>

* [bitnami/common] resolve namespace using common.names.namespace macro (#13481) ([35b84e8](https://github.com/bitnami/charts/commit/35b84e8ba209681d4f160ca102188af61307fccf)), closes [#13481](https://github.com/bitnami/charts/issues/13481)

## <small>2.2.1 (2022-11-25)</small>

* [bitnami/common] fix common topology key affinity function (#13593) ([f95dec8](https://github.com/bitnami/charts/commit/f95dec803bd138b76d67a296545974c5a644d63e)), closes [#13593](https://github.com/bitnami/charts/issues/13593)

## 2.2.0 (2022-11-14)

* [bitnami/common] affinity topologyKey override (#13435) ([624c14e](https://github.com/bitnami/charts/commit/624c14e7121557e6a29ff0e814cb800c2f3cf619)), closes [#13435](https://github.com/bitnami/charts/issues/13435)
* [bitnami/common] Fixed naming of common.secrets.passwords.manage function in README (#13250) ([39a8bcb](https://github.com/bitnami/charts/commit/39a8bcbb1b606cc165643ae4ddcdc15f05e91583)), closes [#13250](https://github.com/bitnami/charts/issues/13250)

## <small>2.1.2 (2022-10-31)</small>

* [bitnami/common] Do not explicitly specify namespace in affinity term. (#12932) ([638a48e](https://github.com/bitnami/charts/commit/638a48e4d3ec7b5d160f4b525ec40218512c464b)), closes [#12932](https://github.com/bitnami/charts/issues/12932) [#12668](https://github.com/bitnami/charts/issues/12668)

## <small>2.1.1 (2022-10-27)</small>

* [bitnami/common] Fix appVersion mismatch (#13189) ([42b3b3e](https://github.com/bitnami/charts/commit/42b3b3e6c68e6af8ba19f7ec42be0d71b4c21852)), closes [#13189](https://github.com/bitnami/charts/issues/13189)

## 2.1.0 (2022-10-27)

* [bitnami/common] Add new function 'common.secrets.lookup' (#13150) ([e848934](https://github.com/bitnami/charts/commit/e84893410321b88adbd7d2e40b891685a15ce640)), closes [#13150](https://github.com/bitnami/charts/issues/13150)

## <small>2.0.4 (2022-10-24)</small>

* [bitnami/*] Use new default branch name in links (#12943) ([a529e02](https://github.com/bitnami/charts/commit/a529e02597d49d944eba1eb0f190713293247176)), closes [#12943](https://github.com/bitnami/charts/issues/12943)
* [bitnami/common] kubernetes.io/tls-acme Ingress annotation triggers IngressTLS array (#13054) ([2008857](https://github.com/bitnami/charts/commit/200885790b34afd6fd04ea45949c887a907b6b38)), closes [#13054](https://github.com/bitnami/charts/issues/13054)
* [bitnami/common] quote secret value when lookup (#11276) ([c8e3019](https://github.com/bitnami/charts/commit/c8e301965f05996a2ae18e0fc8dbfcbe64428356)), closes [#11276](https://github.com/bitnami/charts/issues/11276)

## <small>2.0.3 (2022-09-12)</small>

* [bitnami/common] Revert changes in HPA context from #12282 (#12372) ([55fdc3a](https://github.com/bitnami/charts/commit/55fdc3aff3e32502abfd8f0607ac2be54e585744)), closes [#12282](https://github.com/bitnami/charts/issues/12282) [#12372](https://github.com/bitnami/charts/issues/12372)

## <small>2.0.2 (2022-09-05)</small>

* fix context for HPA util (#12282) ([ccd54a0](https://github.com/bitnami/charts/commit/ccd54a0d47a96903f499fbcdb52a336863020efe)), closes [#12282](https://github.com/bitnami/charts/issues/12282)

## <small>2.0.1 (2022-08-23)</small>

* [bitnami/common] Digest/Tag new approach backward compatible (#12029) ([f1c27dc](https://github.com/bitnami/charts/commit/f1c27dc5d9540c2ea192abf1245da67f5b4f8916)), closes [#12029](https://github.com/bitnami/charts/issues/12029)

## 2.0.0 (2022-08-18)

* [bitnami/common] MAJOR: Add support for image digest apart from tag (#11830) ([e3fee4e](https://github.com/bitnami/charts/commit/e3fee4e41d34a6584660c3b77b8521922603ccab)), closes [#11830](https://github.com/bitnami/charts/issues/11830)

## <small>1.17.1 (2022-08-18)</small>

* Revert changes from #11797 (#11829) ([22bb033](https://github.com/bitnami/charts/commit/22bb033224176c498920596c8d8b25b5f60a277d)), closes [#11797](https://github.com/bitnami/charts/issues/11797) [#11829](https://github.com/bitnami/charts/issues/11829)

## 1.17.0 (2022-08-18)

* [bitnami/common] Add support for image digest apart from tag (#11797) ([b069345](https://github.com/bitnami/charts/commit/b0693450f653318ac7da64575dac389d7041b69f)), closes [#11797](https://github.com/bitnami/charts/issues/11797)

## <small>1.16.1 (2022-07-13)</small>

* [bitnami/*] Replace Kubeapps URL in READMEs (and kubeapps Chart.yaml) and remove BKPR references (#1 ([c6a7914](https://github.com/bitnami/charts/commit/c6a7914361e5aea6016fb45bf4d621edfd111d32)), closes [#10600](https://github.com/bitnami/charts/issues/10600)
* [bitnami/common] Affinities section does not use common.names.namespace (#11137) ([b70c24c](https://github.com/bitnami/charts/commit/b70c24c82c7a9112a4288441ad1fa8c035bb68b4)), closes [#11137](https://github.com/bitnami/charts/issues/11137)

## 1.16.0 (2022-06-03)

* [bitnami/common] Add mysql validation (#10565) ([75ae79a](https://github.com/bitnami/charts/commit/75ae79a434137694fd82198abe1f861d6e5a04ba)), closes [#10565](https://github.com/bitnami/charts/issues/10565)

## <small>1.15.2 (2022-06-02)</small>

* Update Redis trademark references ([2cada87](https://github.com/bitnami/charts/commit/2cada87ed4967d5cb578b0409a0bb1edee79029a))

## <small>1.15.1 (2022-06-01)</small>

* [bitnami/several] Replace maintainers email by url (#10523) ([ff3cf61](https://github.com/bitnami/charts/commit/ff3cf617a1680509b0f3855d17c4ccff7b29a0ff)), closes [#10523](https://github.com/bitnami/charts/issues/10523)

## 1.15.0 (2022-06-01)

* Add common function common.names.fullname.namespace (#10462) ([96f447c](https://github.com/bitnami/charts/commit/96f447cd8654b6db51d9301c841bacb3a13089b3)), closes [#10462](https://github.com/bitnami/charts/issues/10462)

## <small>1.14.2 (2022-05-30)</small>

* [bitnami/common] use -d flag for base64 (#10491) ([ca8d588](https://github.com/bitnami/charts/commit/ca8d5886a1bc0fb37d1bc770ad2333acdffd7996)), closes [#10491](https://github.com/bitnami/charts/issues/10491) [#10486](https://github.com/bitnami/charts/issues/10486)

## <small>1.14.1 (2022-05-20)</small>

* Differentiate between autoscaling v1beta1 and v1beta2 (#10331) ([16d8a4e](https://github.com/bitnami/charts/commit/16d8a4ee73705ee6db2191d84e03a2ba3ea95deb)), closes [#10331](https://github.com/bitnami/charts/issues/10331)

## 1.14.0 (2022-05-13)

* [bitnami/common] Add common function for HPA api version (#10174) ([4379ab5](https://github.com/bitnami/charts/commit/4379ab56bd8f4d7f7b7817bf302c683bf9087e81)), closes [#10174](https://github.com/bitnami/charts/issues/10174)

## <small>1.13.1 (2022-04-19)</small>

* Fix affinities identifier in README.md for common chart (#9821) ([fe95640](https://github.com/bitnami/charts/commit/fe95640ce3f5ddfb0458f440959ceda3a849a3a4)), closes [#9821](https://github.com/bitnami/charts/issues/9821)

## 1.13.0 (2022-03-24)

* [bitnami/common] Add apiService.apiVersion function to common.capabilities (#9562) ([bba2272](https://github.com/bitnami/charts/commit/bba227223e15937bb1f29f77425f6bd7d9238c02)), closes [#9562](https://github.com/bitnami/charts/issues/9562)

## 1.12.0 (2022-03-16)

* [bitnami/common] Helper to allow overriding namespace name (#9396) ([794fecb](https://github.com/bitnami/charts/commit/794fecb8cb112e8e5e9d55420451752e8bd21431)), closes [#9396](https://github.com/bitnami/charts/issues/9396)

## <small>1.11.3 (2022-03-03)</small>

* [bitnami/common] Improve docs for passwords.manage (#9269) ([0d06114](https://github.com/bitnami/charts/commit/0d061147a5b7c7cf2bf44d2b61603ffeb48a0b51)), closes [#9269](https://github.com/bitnami/charts/issues/9269)

## <small>1.11.2 (2022-02-28)</small>

* [bitnami/common] README: Fixed the desscription for `common.labels.matchLabels` (#9062) ([7f17db7](https://github.com/bitnami/charts/commit/7f17db7e9bcdd7918bde322b3b76a62c6a86e752)), closes [#9062](https://github.com/bitnami/charts/issues/9062) [bitnami/charts#9060](https://github.com/bitnami/charts/issues/9060) [bitnami/charts#9060](https://github.com/bitnami/charts/issues/9060)

## <small>1.11.1 (2022-02-02)</small>

* [bitnami/common] Improve "common.secrets.passwords.manage" helper (#8861) ([01477b4](https://github.com/bitnami/charts/commit/01477b42f2be362388d69da913879c52f2250ac1)), closes [#8861](https://github.com/bitnami/charts/issues/8861)

## 1.11.0 (2022-02-01)

* [bitnami/common] Add ingress helper to detect cert-manager annotations (#8857) ([c0c986f](https://github.com/bitnami/charts/commit/c0c986f8d5c911c09dc84d289d2993ae1779a6ca)), closes [#8857](https://github.com/bitnami/charts/issues/8857)

## <small>1.10.4 (2022-01-20)</small>

* [bitnami/several] Add license to the README ([05f7633](https://github.com/bitnami/charts/commit/05f763372501d596e57db713dd53ff4ff3027cc4))
* [bitnami/several] Add license to the README ([32fb238](https://github.com/bitnami/charts/commit/32fb238e60a0affc6debd3142eaa3c3d9089ec2a))
* [bitnami/several] Add license to the README ([b87c2f7](https://github.com/bitnami/charts/commit/b87c2f7899d48a8b02c506765e6ae82937e9ba3f))
* [bitnami/several] Change prerequisites (#8725) ([8d740c5](https://github.com/bitnami/charts/commit/8d740c566cfdb7e2d933c40128b4e919fce953a5)), closes [#8725](https://github.com/bitnami/charts/issues/8725)

## <small>1.10.3 (2021-11-29)</small>

* [bitnami/common] fix: :bug: Add extra check for "\"\"" values in existing secrets (#8266) ([de27be6](https://github.com/bitnami/charts/commit/de27be6e649472608f076a04a36be3674fe3b84e)), closes [#8266](https://github.com/bitnami/charts/issues/8266)

## <small>1.10.2 (2021-11-29)</small>

* [bitnami/several] Replace HTTP by HTTPS when possible (#8259) ([eafb5bd](https://github.com/bitnami/charts/commit/eafb5bd5a2cc3aaf04fc1e8ebedd73f420d76864)), closes [#8259](https://github.com/bitnami/charts/issues/8259)

## <small>1.10.1 (2021-10-27)</small>

* [bitnami/*] Mark PodSecurityPolicy resources as deprecated (#7948) ([5cac753](https://github.com/bitnami/charts/commit/5cac7539dcb6c3baef06ed6676bfa99c16fdb5fe)), closes [#7948](https://github.com/bitnami/charts/issues/7948)

## 1.10.0 (2021-09-30)

* [bitnami/common] Add new capability helper for Network Policies (#7658) ([3efb1ca](https://github.com/bitnami/charts/commit/3efb1cac924409cbda3216a2300cce031c56a1f5)), closes [#7658](https://github.com/bitnami/charts/issues/7658)

## <small>1.9.1 (2021-09-22)</small>

* [bitnami/common] fix readme for common chart (#7577) ([3f06bdd](https://github.com/bitnami/charts/commit/3f06bdd8df1c00dbdf27230bcdf925c337826abb)), closes [#7577](https://github.com/bitnami/charts/issues/7577)
* Fix typo in bitname/common README (#7529) ([fccffb3](https://github.com/bitnami/charts/commit/fccffb33391751a1bf84c53184cffe0dcac83fd6)), closes [#7529](https://github.com/bitnami/charts/issues/7529)

## 1.9.0 (2021-09-13)

* [bitnami/common] Add new dependency fullname template (#7471) ([7ca2a4b](https://github.com/bitnami/charts/commit/7ca2a4bb917ac6a276a6b30be12538f4c7c3a63d)), closes [#7471](https://github.com/bitnami/charts/issues/7471)

## 1.8.0 (2021-08-04)

* Add cronjob apiVersion capability (#7122) ([7b84a67](https://github.com/bitnami/charts/commit/7b84a674ae99fd8ddac3b5b3c859c816b87aaf51)), closes [#7122](https://github.com/bitnami/charts/issues/7122)

## <small>1.7.1 (2021-07-27)</small>

* [bitnami/*] Adapt values.yaml of common library, Tomcat, Wavefront and ZooKeeper charts (#6970) ([fb2693b](https://github.com/bitnami/charts/commit/fb2693bfe67a154b159d3998232cc613e1706c70)), closes [#6970](https://github.com/bitnami/charts/issues/6970)
* [bitnami/several] Bump version and update READMEs (#7069) ([6340bff](https://github.com/bitnami/charts/commit/6340bff66f93c8c797bda3ca0842e4bf770059f1)), closes [#7069](https://github.com/bitnami/charts/issues/7069)
* Replace <sup> strings with &trade; in the README files (#7066) ([d298b49](https://github.com/bitnami/charts/commit/d298b4996da33c9580c2594e6dc8ad665dd0ebab)), closes [#7066](https://github.com/bitnami/charts/issues/7066)

## 1.7.0 (2021-07-02)

* [bitnami/common] Add supportIngressClassname (#6828) ([0c8a455](https://github.com/bitnami/charts/commit/0c8a45546a219b4b4cd370daf0643543c92739b0)), closes [#6828](https://github.com/bitnami/charts/issues/6828)

## <small>1.6.1 (2021-06-16)</small>

* [bitnami/common] extend common.labels.matchLabels with .Values.extraMatchLabels (#6589) ([66edf04](https://github.com/bitnami/charts/commit/66edf04e3e244c343a845f9c684edf4c8ea04406)), closes [#6589](https://github.com/bitnami/charts/issues/6589)

## 1.6.0 (2021-06-15)

* bitnami/common: add version detection for policy api (#6662) ([dcacf06](https://github.com/bitnami/charts/commit/dcacf06f6f2b6d622e2226935db22d5b8efa20b3)), closes [#6662](https://github.com/bitnami/charts/issues/6662)

## <small>1.5.2 (2021-05-21)</small>

* [bitnami/common] Update _ingress.tpl (#6437) ([9048150](https://github.com/bitnami/charts/commit/90481508542c4da588e0d71944592e6c4e8d36e4)), closes [#6437](https://github.com/bitnami/charts/issues/6437)

## <small>1.5.1 (2021-05-14)</small>

* Node affinity values must be quoted. (#6348) ([f73efbe](https://github.com/bitnami/charts/commit/f73efbe074436eda6276bbf32c781fa913c6a17a)), closes [#6348](https://github.com/bitnami/charts/issues/6348)

## 1.5.0 (2021-05-13)

* [bitnami/common] pull secrets rendering (#6286) ([dfffe74](https://github.com/bitnami/charts/commit/dfffe74c212a28e27f537dbee54c3b5a81c7d572)), closes [#6286](https://github.com/bitnami/charts/issues/6286)

## <small>1.4.3 (2021-04-26)</small>

* [bitnami/common] Update Redis validation's helper (#6192) ([1e3bf03](https://github.com/bitnami/charts/commit/1e3bf03e3aad56fd4dc159744626e25ec24c5772)), closes [#6192](https://github.com/bitnami/charts/issues/6192)

## <small>1.4.2 (2021-03-25)</small>

* [bitnami/common] Common credential error (#5884) ([328ca86](https://github.com/bitnami/charts/commit/328ca863515f6ef9fe188c71110be7b951719d66)), closes [#5884](https://github.com/bitnami/charts/issues/5884)

## <small>1.4.1 (2021-02-23)</small>

* [bitnami/common] Add possibility to pull images without giving registry name (#5582) ([15ca275](https://github.com/bitnami/charts/commit/15ca27520a16b590101fa39195f55017e2935a90)), closes [#5582](https://github.com/bitnami/charts/issues/5582)

## 1.4.0 (2021-02-22)

* [bitnami/common] Add RBAC/CRD apiVersion support for versions 1.22+ (#5583) ([fda87aa](https://github.com/bitnami/charts/commit/fda87aabcd004f9a67549f5d22d273dd9fff6836)), closes [#5583](https://github.com/bitnami/charts/issues/5583)

## <small>1.3.9 (2021-02-09)</small>

* Add registered icon to all the MongoDB references (#5426) ([56f2088](https://github.com/bitnami/charts/commit/56f20884267e56175695b2917f7704b9510f4ba6)), closes [#5426](https://github.com/bitnami/charts/issues/5426)

## <small>1.3.8 (2021-02-03)</small>

* fix(common): quote namespace name (#5363) ([d27fb5e](https://github.com/bitnami/charts/commit/d27fb5e0b327728bb4304503376aaa4d2ab50619)), closes [#5363](https://github.com/bitnami/charts/issues/5363)

## <small>1.3.7 (2021-01-20)</small>

* [bitnami/*] Change helm version in the prerequisites (#5090) ([c5e67a3](https://github.com/bitnami/charts/commit/c5e67a388743cbee28439d2cabca27884b9daf97)), closes [#5090](https://github.com/bitnami/charts/issues/5090)
* [bitnami/common] Remove helm version checker from secret helper (#5156) ([20231b1](https://github.com/bitnami/charts/commit/20231b138fae524371e6b29504acd4cbd19ce697)), closes [#5156](https://github.com/bitnami/charts/issues/5156)

## <small>1.3.6 (2021-01-18)</small>

* [bitnami/common] same behavior with empty string when the secret obje… (#5057) ([0bae2bb](https://github.com/bitnami/charts/commit/0bae2bbb9b42c5a8dd2b8a144ffa55ace1c8a936)), closes [#5057](https://github.com/bitnami/charts/issues/5057)

## <small>1.3.5 (2021-01-17)</small>

* [bitnami/common] fix wrong include reference (#5056) ([11efd59](https://github.com/bitnami/charts/commit/11efd59177419d4177e59800f04b4f26ab7243f8)), closes [#5056](https://github.com/bitnami/charts/issues/5056)

## <small>1.3.4 (2021-01-15)</small>

* [bitnami/common] Fix lookup function backward compatibility and README (#5018) ([14a0042](https://github.com/bitnami/charts/commit/14a0042dc90c01fd38f814e1e43559384a3baa9f)), closes [#5018](https://github.com/bitnami/charts/issues/5018)

## <small>1.3.3 (2021-01-14)</small>

* [bitnami/several] Add Redis trademark (#5023) ([dfa89b8](https://github.com/bitnami/charts/commit/dfa89b865989da26a3c73f397fd3c402dd56ebe8)), closes [#5023](https://github.com/bitnami/charts/issues/5023)

## <small>1.3.2 (2021-01-13)</small>

* [bitnami/common] Add missing else statement to ingress apiversion ([22ab07a](https://github.com/bitnami/charts/commit/22ab07ac7d39d4153cc839de2b714086e99cfc04))

## <small>1.3.1 (2021-01-13)</small>

* [bitnami/common] Fix cases where ingress is not at the root (#4984) ([e447d9d](https://github.com/bitnami/charts/commit/e447d9d2205fc3f2f6cd990386a691fd9204b214)), closes [#4984](https://github.com/bitnami/charts/issues/4984)

## 1.3.0 (2021-01-13)

* [bitnami/*] POC Lookup function implementation (#4831) ([240dc1b](https://github.com/bitnami/charts/commit/240dc1bea80a3e121fd595636496d7941bdbc5e0)), closes [#4831](https://github.com/bitnami/charts/issues/4831)

## <small>1.2.3 (2020-12-31)</small>

* [bitnami/common] Fix incorrect backend calculation for networking/v1beta1 ([c59b869](https://github.com/bitnami/charts/commit/c59b86919f47504bc8fd06f75a024f55e58ace77))

## <small>1.2.2 (2020-12-30)</small>

* [bitnami/common] Fix typo in common.capabilities.kubeVersion ([a371b73](https://github.com/bitnami/charts/commit/a371b734b854aa81a7dec16c40d061f5e9a14875))

## <small>1.2.1 (2020-12-30)</small>

* [bitnami/common] Fix issue with global kubeversion calculation ([0bbb339](https://github.com/bitnami/charts/commit/0bbb339d60b41ab978e759863709ebb1451d07a4))

## 1.2.0 (2020-12-30)

* [bitnami/common] Make ingress rules compatible with all Kubernetes versions (#4859) ([2b22a21](https://github.com/bitnami/charts/commit/2b22a217020fe3d16ef98fdcdd4a562c43f9824a)), closes [#4859](https://github.com/bitnami/charts/issues/4859)

## <small>1.1.4 (2020-12-23)</small>

* [bitnami/common] fix: moving kube version comparison (#4804) ([cdb6ae8](https://github.com/bitnami/charts/commit/cdb6ae8f00d114f0998c604416b79f62dc27f19d)), closes [#4804](https://github.com/bitnami/charts/issues/4804)

## <small>1.1.3 (2020-12-18)</small>

* [bitnami/*] fix typos (#4699) ([49adc63](https://github.com/bitnami/charts/commit/49adc63b672da976c55af2e077aa5648a357b77f)), closes [#4699](https://github.com/bitnami/charts/issues/4699)
* [bitnami/common] Adding networking apiVersion support for versions 1.19+ (#4776) ([5ed8c54](https://github.com/bitnami/charts/commit/5ed8c54f5e0a905effc4c1ae5c4931e6669cec30)), closes [#4776](https://github.com/bitnami/charts/issues/4776)

## <small>1.1.2 (2020-12-11)</small>

* [bitnami/common] Fix node affinity templates (#4692) ([5b51a5c](https://github.com/bitnami/charts/commit/5b51a5c004b062282849a4abaaffd6840bb6c95f)), closes [#4692](https://github.com/bitnami/charts/issues/4692)

## <small>1.1.1 (2020-11-26)</small>

* fix: mongodb validation auth (#4506) ([ca3fdfb](https://github.com/bitnami/charts/commit/ca3fdfbeebeba5bd7dfa4805e1ca2411e5950b09)), closes [#4506](https://github.com/bitnami/charts/issues/4506)

## 1.1.0 (2020-11-26)

* [bitnami/common] Add mongodb validation template (#4497) ([14ece96](https://github.com/bitnami/charts/commit/14ece96c801a7326935b6269423d8854fed3a49e)), closes [#4497](https://github.com/bitnami/charts/issues/4497)

## <small>1.0.1 (2020-11-19)</small>

* [bitnami/common] existingSecret is in auth map (#4389) ([de9b217](https://github.com/bitnami/charts/commit/de9b2177465e1c56ca2aa1c4c486bd37a7104d7a)), closes [#4389](https://github.com/bitnami/charts/issues/4389)

## 1.0.0 (2020-11-10)

* bitnami/common Major version. Adapt Chart to apiVersion: v2 (#4258) ([09dbc45](https://github.com/bitnami/charts/commit/09dbc45d11c5e8fe65d6eb64dbf51571ad2c7464)), closes [#4258](https://github.com/bitnami/charts/issues/4258)

## 0.10.0 (2020-10-27)

* [bitnami/common] feat: add cassandra passwords validations (#4110) ([b4923d4](https://github.com/bitnami/charts/commit/b4923d48018dff1673a32eefcc0d62eb484b36da)), closes [#4110](https://github.com/bitnami/charts/issues/4110)

## 0.9.0 (2020-10-21)

* [bitnami/common] feat: add redis passwords validations (#4070) ([0daa8d5](https://github.com/bitnami/charts/commit/0daa8d580c06e18d94dbc0e88467347a34418596)), closes [#4070](https://github.com/bitnami/charts/issues/4070)

## <small>0.8.2 (2020-10-14)</small>

* [bitnami/common] Allow backward compatibility for existingSecret (#4006) ([aa2b3a1](https://github.com/bitnami/charts/commit/aa2b3a18610c69b2f5c76b839483db43fa3c093c)), closes [#4006](https://github.com/bitnami/charts/issues/4006)

## <small>0.8.1 (2020-10-05)</small>

* [bitnami/common] Fix secret name bug with defaulNameSuffix. (#3888) ([d114d44](https://github.com/bitnami/charts/commit/d114d446ef86cb6e7a72de6542905ec3b07d3684))

## 0.8.0 (2020-10-02)

* [bitnami/common] Add statefulset capabilities and prepare MariaDB passwords validation for new forma ([1eb4436](https://github.com/bitnami/charts/commit/1eb44366a72e39e84e33bed1a4940c1b2c6025fc)), closes [#3859](https://github.com/bitnami/charts/issues/3859)

## <small>0.7.1 (2020-09-22)</small>

* [bitnami/common] fix: evaluate enabled as string (#3733) ([048cdae](https://github.com/bitnami/charts/commit/048cdae5488cfcfe83ec698afaa8318aa3b1d0ca)), closes [#3733](https://github.com/bitnami/charts/issues/3733)

## 0.7.0 (2020-09-22)

* [bitnami/metrics-server] Add source repo (#3577) ([1ed12f9](https://github.com/bitnami/charts/commit/1ed12f96af75322b46afdb2b3d9907c11b13f765)), closes [#3577](https://github.com/bitnami/charts/issues/3577)
* PoC for pods' affinity (#3713) ([9e6a915](https://github.com/bitnami/charts/commit/9e6a915392979f0c0148875f34cca1c27e399b59)), closes [#3713](https://github.com/bitnami/charts/issues/3713)

## <small>0.6.2 (2020-09-01)</small>

* [bitnami/common] fix: wrong use of append function (#3566) ([c912fd0](https://github.com/bitnami/charts/commit/c912fd0b7378bf2d5d56182e6d2fa6bbd74df46f)), closes [#3566](https://github.com/bitnami/charts/issues/3566)

## <small>0.6.1 (2020-08-31)</small>

* [bitnami/common] fix: mariadb checks secret fields after check enabled (#3565) ([498056a](https://github.com/bitnami/charts/commit/498056ad16a6e89aa3b7cc231da7467ab5bd3986)), closes [#3565](https://github.com/bitnami/charts/issues/3565)

## 0.6.0 (2020-08-19)

* [bitnami/mariadb] Require password option at secret resource (#3411) ([a8d2464](https://github.com/bitnami/charts/commit/a8d24643756470d0280fc585b01397358c1c242d)), closes [#3411](https://github.com/bitnami/charts/issues/3411)

## <small>0.5.2 (2020-08-19)</small>

* [bitnami/common] fix: add global parameters to postgres validation (#3460) ([1c52a2a](https://github.com/bitnami/charts/commit/1c52a2a48ea65024a753eb5b32deadd46650fb18)), closes [#3460](https://github.com/bitnami/charts/issues/3460)

## <small>0.5.1 (2020-08-10)</small>

* fix(common): missing $ in required values helpers (#3376) ([c972152](https://github.com/bitnami/charts/commit/c972152762c14c5ab5e3847a4870f4f4f2a31224)), closes [#3376](https://github.com/bitnami/charts/issues/3376)

## 0.5.0 (2020-08-10)

* [bitnami/common] add psql and mysql required password validations (#3374) ([1a4419e](https://github.com/bitnami/charts/commit/1a4419e15d985f67413beff98c9fc9b9f69108fb)), closes [#3374](https://github.com/bitnami/charts/issues/3374)

## 0.4.0 (2020-08-04)

* [bitnami/*] Fix TL;DR typo in READMEs (#3280) ([3d7ab40](https://github.com/bitnami/charts/commit/3d7ab406fecd64f1af25f53e7d27f03ec95b29a4)), closes [#3280](https://github.com/bitnami/charts/issues/3280)
* [bitnami/all] Add categories (#3075) ([63bde06](https://github.com/bitnami/charts/commit/63bde066b87a140fab52264d0522401ab3d63509)), closes [#3075](https://github.com/bitnami/charts/issues/3075)
* Add common helpers to check secrets when upgrade (#3150) ([5a5807c](https://github.com/bitnami/charts/commit/5a5807c1b1db1f2337f6aa5308d3ff73a4329e6a)), closes [#3150](https://github.com/bitnami/charts/issues/3150)

## <small>0.3.1 (2020-06-05)</small>

* [bitnami/several] Fix table rendering in some hubs (#2770) ([fe9fd8c](https://github.com/bitnami/charts/commit/fe9fd8c261195385aae73e165ac6c1a666fef08e)), closes [#2770](https://github.com/bitnami/charts/issues/2770)

## 0.3.0 (2020-06-02)

* [bitnami/common]: add template function for ingress apiVersion (#2732) ([a968a50](https://github.com/bitnami/charts/commit/a968a50916ed9fa6f823a5a3ef6e4b98d615322f)), closes [#2732](https://github.com/bitnami/charts/issues/2732)

## <small>0.2.4 (2020-05-29)</small>

* [bitnami/common] Bump chart version (#2707) ([ff2c37a](https://github.com/bitnami/charts/commit/ff2c37a576191f4523c7f69504aea669ab68aba8)), closes [#2707](https://github.com/bitnami/charts/issues/2707)
* [bitnami/several] Fix trailing spaces to make helm lint work on all of them (#2705) ([bafba3f](https://github.com/bitnami/charts/commit/bafba3fc8b8949897ad2d99d437bd8fc975223e4)), closes [#2705](https://github.com/bitnami/charts/issues/2705)

## <small>0.2.3 (2020-05-26)</small>

* fix(common): add name attribute to imagePullSecrets helper (#2664) ([1ea21a9](https://github.com/bitnami/charts/commit/1ea21a92a8f44bd0d82d0fd4ed30108a89cf5b34)), closes [#2664](https://github.com/bitnami/charts/issues/2664)

## <small>0.2.2 (2020-05-19)</small>

* update bitnami/common to be compatible with helm v2.12+ (#2615) ([c7751eb](https://github.com/bitnami/charts/commit/c7751eb5764e468e1854b58a1b8491d2b13e0a4a)), closes [#2615](https://github.com/bitnami/charts/issues/2615)

## <small>0.2.1 (2020-05-13)</small>

* bump bitnami/common version number (#2580) ([1bd1e7b](https://github.com/bitnami/charts/commit/1bd1e7bc776614b6ae10f21e9c8b23fe15db5ff4)), closes [#2580](https://github.com/bitnami/charts/issues/2580)

## 0.2.0 (2020-04-17)

* [bitnami/common] add secrets and warnings helpers (#2347) ([a748ff8](https://github.com/bitnami/charts/commit/a748ff82259d6553a0d4ca56ca6d7d050de859f4)), closes [#2347](https://github.com/bitnami/charts/issues/2347)

## <small>0.1.1 (2020-04-08)</small>

* [bitnami/common] bitnami common add values yaml (#2267) ([a88c902](https://github.com/bitnami/charts/commit/a88c90212021771eacc562dd38c04381e2f63d6f)), closes [#2267](https://github.com/bitnami/charts/issues/2267)

## 0.1.0 (2020-04-03)

* [bitnami/common]: add initial functions (#2188) ([9401e13](https://github.com/bitnami/charts/commit/9401e13316992c36b0e33de75d5f249645a2924e)), closes [#2188](https://github.com/bitnami/charts/issues/2188)