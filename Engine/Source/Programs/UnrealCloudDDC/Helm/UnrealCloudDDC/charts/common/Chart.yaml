# Copyright Broadcom, Inc. All Rights Reserved.
# SPDX-License-Identifier: APACHE-2.0

annotations:
  category: Infrastructure
  licenses: Apache-2.0
apiVersion: v2
# Please make sure that version and appVersion are always the same.
appVersion: 2.31.0
description: A Library Helm Chart for grouping common logic between bitnami charts. This chart is not deployable by itself.
home: https://bitnami.com
icon: https://dyltqmyl993wv.cloudfront.net/downloads/logos/bitnami-mark.png
keywords:
  - common
  - helper
  - template
  - function
  - bitnami
maintainers:
  - name: Broadcom, Inc. All Rights Reserved.
    url: https://github.com/bitnami/charts
name: common
sources:
  - https://github.com/bitnami/charts/tree/main/bitnami/common
type: library
version: 2.31.0