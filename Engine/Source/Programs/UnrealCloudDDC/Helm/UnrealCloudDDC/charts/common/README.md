# Bitnami Common Library Chart

A [Helm Library Chart](https://helm.sh/docs/topics/library_charts/#helm) for grouping common logic between Bitnami charts.

## TL;DR

```yaml
dependencies:
  - name: common
    version: 2.x.x
    repository: oci://registry-1.docker.io/bitnamicharts
```

```console
helm dependency update
```

```yaml
apiVersion: v1
kind: ConfigMap
metadata:
	name: {{ include "common.names.fullname" . }}
data:
	myvalue: "Hello World"
```

Looking to use our applications in production? Try [VMware Tanzu Application Catalog](https://bitnami.com/enterprise), the commercial edition of the Bitnami catalog.

## Introduction

This chart provides a common template helpers which can be used to develop new charts using [Helm](https://helm.sh) package manager.

Bitnami charts can be used with [Kubeapps](https://kubeapps.dev/) for deployment and management of Helm Charts in clusters.

## Prerequisites

- Kubernetes 1.23+
- Helm 3.8.0+

## Parameters

The following table lists the helpers available in the library which are scoped in different sections.

### Affinities

| Helper identifier               | Description                                          | Expected Input                                               |
| ------------------------------- | ---------------------------------------------------- | ------------------------------------------------------------ |
| `common.affinities.nodes.soft`  | Return a soft nodeAffinity definition                | `dict "key" "FOO" "values" (list "BAR" "BAZ")`               |
| `common.affinities.nodes.hard`  | Return a hard nodeAffinity definition                | `dict "key" "FOO" "values" (list "BAR" "BAZ")`               |
| `common.affinities.nodes`       | Return a nodeAffinity definition                     | `dict "type" "soft" "key" "FOO" "values" (list "BAR" "BAZ")` |
| `common.affinities.topologyKey` | Return a topologyKey definition                      | `dict "topologyKey" "FOO"`                                   |
| `common.affinities.pods.soft`   | Return a soft podAffinity/podAntiAffinity definition | `dict "component" "FOO" "context" $`                         |
| `common.affinities.pods.hard`   | Return a hard podAffinity/podAntiAffinity definition | `dict "component" "FOO" "context" $`                         |
| `common.affinities.pods`        | Return a podAffinity/podAntiAffinity definition      | `dict "type" "soft" "key" "FOO" "values" (list "BAR" "BAZ")` |

### Capabilities

| Helper identifier                                         | Description                                                                                    | Expected Input                          |
| --------------------------------------------------------- | ---------------------------------------------------------------------------------------------- | --------------------------------------- |
| `common.capabilities.kubeVersion`                         | Return the target Kubernetes version (using client default if .Values.kubeVersion is not set). | `.` Chart context                       |
| `common.capabilities.apiVersions.has`                     | Return true if the apiVersion is supported                                                     | `dict "version" "batch/v1" "context" $` |
| `common.capabilities.job.apiVersion`                      | Return the appropriate apiVersion for job.                                                     | `.` Chart context                       |
| `common.capabilities.cronjob.apiVersion`                  | Return the appropriate apiVersion for cronjob.                                                 | `.` Chart context                       |
| `common.capabilities.daemonset.apiVersion`                | Return the appropriate apiVersion for daemonset.                                               | `.` Chart context                       |
| `common.capabilities.cronjob.apiVersion`                  | Return the appropriate apiVersion for cronjob.                                                 | `.` Chart context                       |
| `common.capabilities.deployment.apiVersion`               | Return the appropriate apiVersion for deployment.                                              | `.` Chart context                       |
| `common.capabilities.statefulset.apiVersion`              | Return the appropriate apiVersion for statefulset.                                             | `.` Chart context                       |
| `common.capabilities.ingress.apiVersion`                  | Return the appropriate apiVersion for ingress.                                                 | `.` Chart context                       |
| `common.capabilities.rbac.apiVersion`                     | Return the appropriate apiVersion for RBAC resources.                                          | `.` Chart context                       |
| `common.capabilities.crd.apiVersion`                      | Return the appropriate apiVersion for CRDs.                                                    | `.` Chart context                       |
| `common.capabilities.policy.apiVersion`                   | Return the appropriate apiVersion for podsecuritypolicy.                                       | `.` Chart context                       |
| `common.capabilities.networkPolicy.apiVersion`            | Return the appropriate apiVersion for networkpolicy.                                           | `.` Chart context                       |
| `common.capabilities.apiService.apiVersion`               | Return the appropriate apiVersion for APIService.                                              | `.` Chart context                       |
| `common.capabilities.hpa.apiVersion`                      | Return the appropriate apiVersion for Horizontal Pod Autoscaler                                | `.` Chart context                       |
| `common.capabilities.vpa.apiVersion`                      | Return the appropriate apiVersion for Vertical Pod Autoscaler.                                 | `.` Chart context                       |
| `common.capabilities.psp.supported`                       | Returns true if PodSecurityPolicy is supported                                                 | `.` Chart context                       |
| `common.capabilities.supportsHelmVersion`                 | Returns true if the used Helm version is 3.3+                                                  | `.` Chart context                       |
| `common.capabilities.admissionConfiguration.supported`    | Returns true if AdmissionConfiguration is supported                                            | `.` Chart context                       |
| `common.capabilities.admissionConfiguration.apiVersion`   | Return the appropriate apiVersion for AdmissionConfiguration.                                  | `.` Chart context                       |
| `common.capabilities.podSecurityConfiguration.apiVersion` | Return the appropriate apiVersion for PodSecurityConfiguration.                                | `.` Chart context                       |

### Compatibility

| Helper identifier                            | Description                                                                                                                                                                                                                           | Expected Input                                                   |
| -------------------------------------------- | ------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------- | ---------------------------------------------------------------- |
| `common.compatibility.isOpenshift`           | Return true if the detected platform is Openshift                                                                                                                                                                                     | `.` Chart context                                                |
| `common.compatibility.renderSecurityContext` | Render a compatible securityContext depending on the platform. By default it is maintained as it is. In other platforms like Openshift we remove default user/group values that do not work out of the box with the restricted-v1 SCC | `dict "secContext" .Values.containerSecurityContext "context" $` |

### Errors

| Helper identifier                       | Description                                                                                                                                                            | Expected Input                                                                      |
| --------------------------------------- | ---------------------------------------------------------------------------------------------------------------------------------------------------------------------- | ----------------------------------------------------------------------------------- |
| `common.errors.upgrade.passwords.empty` | It will ensure required passwords are given when we are upgrading a chart. If `validationErrors` is not empty it will throw an error and will stop the upgrade action. | `dict "validationErrors" (list $validationError00 $validationError01)  "context" $` |
| `common.errors.insecureImages`          | Throw error when original container images are replaced. The error can be bypassed by setting the `global.security.allowInsecureImages` to true.                       | `dict "images" (list .Values.path.to.the.imageRoot) "context" $`                    |

### Images

| Helper identifier                 | Description                                                                                                    | Expected Input                                                                                               |
| --------------------------------- | -------------------------------------------------------------------------------------------------------------- | ------------------------------------------------------------------------------------------------------------ |
| `common.images.image`             | Return the proper and full image name                                                                          | `dict "imageRoot" .Values.path.to.the.image "global" $`, see [ImageRoot](#imageroot) for the structure.      |
| `common.images.pullSecrets`       | Return the proper Docker Image Registry Secret Names (deprecated: use common.images.renderPullSecrets instead) | `dict "images" (list .Values.path.to.the.image1, .Values.path.to.the.image2) "global" .Values.global`        |
| `common.images.renderPullSecrets` | Return the proper Docker Image Registry Secret Names (evaluates values as templates)                           | `dict "images" (list .Values.path.to.the.image1, .Values.path.to.the.image2) "context" $`                    |
| `common.images.version`           | Return the proper image version                                                                                | `dict "imageRoot" .Values.path.to.the.image "chart" .Chart` , see [ImageRoot](#imageroot) for the structure. |

### Ingress

| Helper identifier                         | Description                                                                                                       | Expected Input                                                                                                                                                                   |
| ----------------------------------------- | ----------------------------------------------------------------------------------------------------------------- | -------------------------------------------------------------------------------------------------------------------------------------------------------------------------------- |
| `common.ingress.backend`                  | Generate a proper Ingress backend entry depending on the API version                                              | `dict "serviceName" "foo" "servicePort" "bar"`, see the [Ingress deprecation notice](https://kubernetes.io/blog/2019/07/18/api-deprecations-in-1-16/) for the syntax differences |
| `common.ingress.supportsPathType`         | Prints "true" if the pathType field is supported                                                                  | `.` Chart context                                                                                                                                                                |
| `common.ingress.supportsIngressClassname` | Prints "true" if the ingressClassname field is supported                                                          | `.` Chart context                                                                                                                                                                |
| `common.ingress.certManagerRequest`       | Prints "true" if required cert-manager annotations for TLS signed certificates are set in the Ingress annotations | `dict "annotations" .Values.path.to.the.ingress.annotations`                                                                                                                     |

### Labels

| Helper identifier           | Description                                                                 | Expected Input    |
| --------------------------- | --------------------------------------------------------------------------- | ----------------- |
| `common.labels.standard`    | Return Kubernetes standard labels                                           | `.` Chart context |
| `common.labels.matchLabels` | Labels to use on `deploy.spec.selector.matchLabels` and `svc.spec.selector` | `.` Chart context |

### Names

| Helper identifier                  | Description                                                           | Expected Input                                                                                |
| ---------------------------------- | --------------------------------------------------------------------- | --------------------------------------------------------------------------------------------- |
| `common.names.name`                | Expand the name of the chart or use `.Values.nameOverride`            | `.` Chart context                                                                             |
| `common.names.fullname`            | Create a default fully qualified app name.                            | `.` Chart context                                                                             |
| `common.names.namespace`           | Allow the release namespace to be overridden                          | `.` Chart context                                                                             |
| `common.names.fullname.namespace`  | Create a fully qualified app name adding the installation's namespace | `.` Chart context                                                                             |
| `common.names.chart`               | Chart name plus version                                               | `.` Chart context                                                                             |
| `common.names.dependency.fullname` | Create a default fully qualified dependency name.                     | `dict "chartName" "dependency-chart-name" "chartValues" .Values.dependency-chart "context" $` |

### Resources

| Helper identifier         | Description                                                                                                                                 | Expected Input       |
| ------------------------- | ------------------------------------------------------------------------------------------------------------------------------------------- | -------------------- |
| `common.resources.preset` | Return a resource request/limit object based on a given preset. These presets are for basic testing and not meant to be used in production. | `dict "type" "nano"` |

### Secrets

| Helper identifier                 | Description                                                                            | Expected Input                                                                                                                                                                                                                                                                   |
| --------------------------------- | -------------------------------------------------------------------------------------- | -------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------- |
| `common.secrets.name`             | Generate the name of the secret.                                                       | `dict "existingSecret" .Values.path.to.the.existingSecret "defaultNameSuffix" "mySuffix" "context" $` see [ExistingSecret](#existingsecret) for the structure.                                                                                                                   |
| `common.secrets.key`              | Generate secret key.                                                                   | `dict "existingSecret" .Values.path.to.the.existingSecret "key" "keyName"` see [ExistingSecret](#existingsecret) for the structure.                                                                                                                                              |
| `common.secrets.passwords.manage` | Generate secret password or retrieve one if already created.                           | `dict "secret" "secret-name" "key" "keyName" "providedValues" (list "path.to.password1" "path.to.password2") "length" 10 "strong" false "chartName" "chartName" "honorProvidedValues" false "context" $`, length, strong, honorProvidedValues and chartName fields are optional. |
| `common.secrets.exists`           | Returns whether a previous generated secret already exists.                            | `dict "secret" "secret-name" "context" $`                                                                                                                                                                                                                                        |
| `common.secrets.lookup`           | Reuses the value from an existing secret, otherwise sets its value to a default value. | `dict "secret" "secret-name" "key" "keyName" "defaultValue" .Values.myValue "context" $`                                                                                                                                                                                         |

### Storage

| Helper identifier      | Description                      | Expected Input                                                                                                      |
| ---------------------- | -------------------------------- | ------------------------------------------------------------------------------------------------------------------- |
| `common.storage.class` | Return  the proper Storage Class | `dict "persistence" .Values.path.to.the.persistence "global" $`, see [Persistence](#persistence) for the structure. |

### TplValues

| Helper identifier                  | Description                                                         | Expected Input                                                                                                                                           |
| ---------------------------------- | ------------------------------------------------------------------- | -------------------------------------------------------------------------------------------------------------------------------------------------------- |
| `common.tplvalues.render`          | Renders a value that contains template                              | `dict "value" .Values.path.to.the.Value "context" $`, value is the value should rendered as template, context frequently is the chart context `$` or `.` |
| `common.tplvalues.merge`           | Merge a list of values that contains template after rendering them. | `dict "values" (list .Values.path.to.the.Value1 .Values.path.to.the.Value2) "context" $`                                                                 |
| `common.tplvalues.merge-overwrite` | Merge a list of values that contains template after rendering them. | `dict "values" (list .Values.path.to.the.Value1 .Values.path.to.the.Value2) "context" $`                                                                 |

### Utils

| Helper identifier               | Description                                                                                                                                     | Expected Input                                                         |
| ------------------------------- | ----------------------------------------------------------------------------------------------------------------------------------------------- | ---------------------------------------------------------------------- |
| `common.utils.fieldToEnvVar`    | Build environment variable name given a field.                                                                                                  | `dict "field" "my-password"`                                           |
| `common.utils.secret.getvalue`  | Print instructions to get a secret value.                                                                                                       | `dict "secret" "secret-name" "field" "secret-value-field" "context" $` |
| `common.utils.getValueFromKey`  | Gets a value from `.Values` object given its key path                                                                                           | `dict "key" "path.to.key" "context" $`                                 |
| `common.utils.getKeyFromList`   | Returns first `.Values` key with a defined value or first of the list if all non-defined                                                        | `dict "keys" (list "path.to.key1" "path.to.key2") "context" $`         |
| `common.utils.checksumTemplate` | Checksum a template at "path" containing a *single* resource (ConfigMap,Secret) for use in pod annotations, excluding the metadata (see #18376) | `dict "path" "/configmap.yaml" "context" $`                            |

### Validations

| Helper identifier                             | Description                                                                                                        | Expected Input                                                                                                                                                                                                                                                           |
| --------------------------------------------- | ------------------------------------------------------------------------------------------------------------------ | ------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------ |
| `common.validations.values.single.empty`      | Validate a value must not be empty.                                                                                | `dict "valueKey" "path.to.value" "secret" "secret.name" "field" "my-password" "subchart" "subchart" "context" $` secret, field and subchart are optional. In case they are given, the helper will generate a how to get instruction. See [ValidateValue](#validatevalue) |
| `common.validations.values.multiple.empty`    | Validate a multiple values must not be empty. It returns a shared error for all the values.                        | `dict "required" (list $validateValueConf00 $validateValueConf01) "context" $`. See [ValidateValue](#validatevalue)                                                                                                                                                      |
| `common.validations.values.mariadb.passwords` | This helper will ensure required password for MariaDB are not empty. It returns a shared error for all the values. | `dict "secret" "mariadb-secret" "subchart" "true" "context" $` subchart field is optional and could be true or false it depends on where you will use mariadb chart and the helper.                                                                                      |

### Warnings

| Helper identifier                | Description                                                       | Expected Input                                             |
| -------------------------------- | ----------------------------------------------------------------- | ---------------------------------------------------------- |
| `common.warnings.rollingTag`     | Warning about using rolling tag.                                  | `ImageRoot` see [ImageRoot](#imageroot) for the structure. |
| `common.warnings.modifiedImages` | Warning about replaced images from the original.                  | `ImageRoot` see [ImageRoot](#imageroot) for the structure. |
| `common.warnings.resources`      | Warning about not setting the resource object in all deployments. | `dict "sections" (list "path1" "path2") context $`         |

## Special input schemas

### ImageRoot

```yaml
registry:
  type: string
  description: Docker registry where the image is located
  example: docker.io

repository:
  type: string
  description: Repository and image name
  example: bitnami/nginx

tag:
  type: string
  description: image tag
  example: 1.16.1-debian-10-r63

pullPolicy:
  type: string
  description: Specify a imagePullPolicy.'

pullSecrets:
  type: array
  items:
    type: string
  description: Optionally specify an array of imagePullSecrets (evaluated as templates).

debug:
  type: boolean
  description: Set to true if you would like to see extra information on logs
  example: false

## An instance would be:
# registry: docker.io
# repository: bitnami/nginx
# tag: 1.16.1-debian-10-r63
# pullPolicy: IfNotPresent
# debug: false
```

### Persistence

```yaml
enabled:
  type: boolean
  description: Whether enable persistence.
  example: true

storageClass:
  type: string
  description: Ghost data Persistent Volume Storage Class, If set to "-", storageClassName: "" which disables dynamic provisioning.
  example: "-"

accessMode:
  type: string
  description: Access mode for the Persistent Volume Storage.
  example: ReadWriteOnce

size:
  type: string
  description: Size the Persistent Volume Storage.
  example: 8Gi

path:
  type: string
  description: Path to be persisted.
  example: /bitnami

## An instance would be:
# enabled: true
# storageClass: "-"
# accessMode: ReadWriteOnce
# size: 8Gi
# path: /bitnami
```

### ExistingSecret

```yaml
name:
  type: string
  description: Name of the existing secret.
  example: mySecret
keyMapping:
  description: Mapping between the expected key name and the name of the key in the existing secret.
  type: object

## An instance would be:
# name: mySecret
# keyMapping:
#   password: myPasswordKey
```

#### Example of use

When we store sensitive data for a deployment in a secret, some times we want to give to users the possibility of using theirs existing secrets.

```yaml
# templates/secret.yaml
---
apiVersion: v1
kind: Secret
metadata:
  name: {{ include "common.names.fullname" . }}
  labels:
    app: {{ include "common.names.fullname" . }}
type: Opaque
data:
  password: {{ .Values.password | b64enc | quote }}

# templates/dpl.yaml
---
...
      env:
        - name: PASSWORD
          valueFrom:
            secretKeyRef:
              name: {{ include "common.secrets.name" (dict "existingSecret" .Values.existingSecret "context" $) }}
              key: {{ include "common.secrets.key" (dict "existingSecret" .Values.existingSecret "key" "password") }}
...

# values.yaml
---
name: mySecret
keyMapping:
  password: myPasswordKey
```

### ValidateValue

#### NOTES.txt

```console
{{- $validateValueConf00 := (dict "valueKey" "path.to.value00" "secret" "secretName" "field" "password-00") -}}
{{- $validateValueConf01 := (dict "valueKey" "path.to.value01" "secret" "secretName" "field" "password-01") -}}

{{ include "common.validations.values.multiple.empty" (dict "required" (list $validateValueConf00 $validateValueConf01) "context" $) }}
```

If we force those values to be empty we will see some alerts

```console
helm install test mychart --set path.to.value00="",path.to.value01=""
    'path.to.value00' must not be empty, please add '--set path.to.value00=$PASSWORD_00' to the command. To get the current value:

        export PASSWORD_00=$(kubectl get secret --namespace default secretName -o jsonpath="{.data.password-00}" | base64 -d)

    'path.to.value01' must not be empty, please add '--set path.to.value01=$PASSWORD_01' to the command. To get the current value:

        export PASSWORD_01=$(kubectl get secret --namespace default secretName -o jsonpath="{.data.password-01}" | base64 -d)
```

## Upgrading

### To 1.0.0

[On November 13, 2020, Helm v2 support was formally finished](https://github.com/helm/charts#status-of-the-project), this major version is the result of the required changes applied to the Helm Chart to be able to incorporate the different features added in Helm v3 and to be consistent with the Helm project itself regarding the Helm v2 EOL.

#### What changes were introduced in this major version?

- Previous versions of this Helm Chart use `apiVersion: v1` (installable by both Helm 2 and 3), this Helm Chart was updated to `apiVersion: v2` (installable by Helm 3 only). [Here](https://helm.sh/docs/topics/charts/#the-apiversion-field) you can find more information about the `apiVersion` field.
- Use `type: library`. [Here](https://v3.helm.sh/docs/faq/#library-chart-support) you can find more information.
- The different fields present in the *Chart.yaml* file has been ordered alphabetically in a homogeneous way for all the Bitnami Helm Charts

#### Considerations when upgrading to this version

- If you want to upgrade to this version from a previous one installed with Helm v3, you shouldn't face any issues
- If you want to upgrade to this version using Helm v2, this scenario is not supported as this version doesn't support Helm v2 anymore
- If you installed the previous version with Helm v2 and wants to upgrade to this version with Helm v3, please refer to the [official Helm documentation](https://helm.sh/docs/topics/v2_v3_migration/#migration-use-cases) about migrating from Helm v2 to v3

#### Useful links

- <https://techdocs.broadcom.com/us/en/vmware-tanzu/application-catalog/tanzu-application-catalog/services/tac-doc/apps-tutorials-resolve-helm2-helm3-post-migration-issues-index.html>
- <https://helm.sh/docs/topics/v2_v3_migration/>
- <https://helm.sh/blog/migrate-from-helm-v2-to-helm-v3/>

## License

Copyright &copy; 2025 Broadcom. The term "Broadcom" refers to Broadcom Inc. and/or its subsidiaries.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

<http://www.apache.org/licenses/LICENSE-2.0>

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.