{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}

{{/*
Return a soft nodeAffinity definition
{{ include "common.affinities.nodes.soft" (dict "key" "FOO" "values" (list "BAR" "BAZ")) -}}
*/}}
{{- define "common.affinities.nodes.soft" -}}
preferredDuringSchedulingIgnoredDuringExecution:
  - preference:
      matchExpressions:
        - key: {{ .key }}
          operator: In
          values:
            {{- range .values }}
            - {{ . | quote }}
            {{- end }}
    weight: 1
{{- end -}}

{{/*
Return a hard nodeAffinity definition
{{ include "common.affinities.nodes.hard" (dict "key" "FOO" "values" (list "BAR" "BAZ")) -}}
*/}}
{{- define "common.affinities.nodes.hard" -}}
requiredDuringSchedulingIgnoredDuringExecution:
  nodeSelectorTerms:
    - matchExpressions:
        - key: {{ .key }}
          operator: In
          values:
            {{- range .values }}
            - {{ . | quote }}
            {{- end }}
{{- end -}}

{{/*
Return a nodeAffinity definition
{{ include "common.affinities.nodes" (dict "type" "soft" "key" "FOO" "values" (list "BAR" "BAZ")) -}}
*/}}
{{- define "common.affinities.nodes" -}}
  {{- if eq .type "soft" }}
    {{- include "common.affinities.nodes.soft" . -}}
  {{- else if eq .type "hard" }}
    {{- include "common.affinities.nodes.hard" . -}}
  {{- end -}}
{{- end -}}

{{/*
Return a topologyKey definition
{{ include "common.affinities.topologyKey" (dict "topologyKey" "BAR") -}}
*/}}
{{- define "common.affinities.topologyKey" -}}
{{ .topologyKey | default "kubernetes.io/hostname" -}}
{{- end -}}

{{/*
Return a soft podAffinity/podAntiAffinity definition
{{ include "common.affinities.pods.soft" (dict "component" "FOO" "customLabels" .Values.podLabels "extraMatchLabels" .Values.extraMatchLabels "topologyKey" "BAR" "extraPodAffinityTerms" .Values.extraPodAffinityTerms "extraNamespaces" (list "namespace1" "namespace2") "context" $) -}}
*/}}
{{- define "common.affinities.pods.soft" -}}
{{- $component := default "" .component -}}
{{- $customLabels := default (dict) .customLabels -}}
{{- $extraMatchLabels := default (dict) .extraMatchLabels -}}
{{- $extraPodAffinityTerms := default (list) .extraPodAffinityTerms -}}
{{- $extraNamespaces := default (list) .extraNamespaces -}}
preferredDuringSchedulingIgnoredDuringExecution:
  - podAffinityTerm:
      labelSelector:
        matchLabels: {{- (include "common.labels.matchLabels" ( dict "customLabels" $customLabels "context" .context )) | nindent 10 }}
          {{- if not (empty $component) }}
          {{ printf "app.kubernetes.io/component: %s" $component }}
          {{- end }}
          {{- range $key, $value := $extraMatchLabels }}
          {{ $key }}: {{ $value | quote }}
          {{- end }}
      {{- if $extraNamespaces }}
      namespaces:
        - {{ .context.Release.Namespace }}
        {{- with $extraNamespaces }}
        {{- include "common.tplvalues.render" (dict "value" . "context" $) | nindent 8 }}
        {{- end }}
      {{- end }}
      topologyKey: {{ include "common.affinities.topologyKey" (dict "topologyKey" .topologyKey) }}
    weight: 1
  {{- range $extraPodAffinityTerms }}
  - podAffinityTerm:
      labelSelector:
        matchLabels: {{- (include "common.labels.matchLabels" ( dict "customLabels" $customLabels "context" $.context )) | nindent 10 }}
          {{- if not (empty $component) }}
          {{ printf "app.kubernetes.io/component: %s" $component }}
          {{- end }}
          {{- range $key, $value := .extraMatchLabels }}
          {{ $key }}: {{ $value | quote }}
          {{- end }}
      {{- if .namespaces }}
      namespaces:
        - {{ $.context.Release.Namespace }}
        {{- with .namespaces }}
        {{- include "common.tplvalues.render" (dict "value" . "context" $) | nindent 8 }}
        {{- end }}
      {{- end }}
      topologyKey: {{ include "common.affinities.topologyKey" (dict "topologyKey" .topologyKey) }}
    weight: {{ .weight | default 1 -}}
  {{- end -}}
{{- end -}}

{{/*
Return a hard podAffinity/podAntiAffinity definition
{{ include "common.affinities.pods.hard" (dict "component" "FOO" "customLabels" .Values.podLabels "extraMatchLabels" .Values.extraMatchLabels "topologyKey" "BAR" "extraPodAffinityTerms" .Values.extraPodAffinityTerms "extraNamespaces" (list "namespace1" "namespace2") "context" $) -}}
*/}}
{{- define "common.affinities.pods.hard" -}}
{{- $component := default "" .component -}}
{{- $customLabels := default (dict) .customLabels -}}
{{- $extraMatchLabels := default (dict) .extraMatchLabels -}}
{{- $extraPodAffinityTerms := default (list) .extraPodAffinityTerms -}}
{{- $extraNamespaces := default (list) .extraNamespaces -}}
requiredDuringSchedulingIgnoredDuringExecution:
  - labelSelector:
      matchLabels: {{- (include "common.labels.matchLabels" ( dict "customLabels" $customLabels "context" .context )) | nindent 8 }}
        {{- if not (empty $component) }}
        {{ printf "app.kubernetes.io/component: %s" $component }}
        {{- end }}
        {{- range $key, $value := $extraMatchLabels }}
        {{ $key }}: {{ $value | quote }}
        {{- end }}
    {{- if $extraNamespaces }}
    namespaces:
      - {{ .context.Release.Namespace }}
      {{- with $extraNamespaces }}
      {{- include "common.tplvalues.render" (dict "value" . "context" $) | nindent 6 }}
      {{- end }}
    {{- end }}
    topologyKey: {{ include "common.affinities.topologyKey" (dict "topologyKey" .topologyKey) }}
  {{- range $extraPodAffinityTerms }}
  - labelSelector:
      matchLabels: {{- (include "common.labels.matchLabels" ( dict "customLabels" $customLabels "context" $.context )) | nindent 8 }}
        {{- if not (empty $component) }}
        {{ printf "app.kubernetes.io/component: %s" $component }}
        {{- end }}
        {{- range $key, $value := .extraMatchLabels }}
        {{ $key }}: {{ $value | quote }}
        {{- end }}
    {{- if .namespaces }}
    namespaces:
      - {{ $.context.Release.Namespace }}
      {{- with .namespaces }}
      {{- include "common.tplvalues.render" (dict "value" . "context" $) | nindent 6 }}
      {{- end }}
    {{- end }}
    topologyKey: {{ include "common.affinities.topologyKey" (dict "topologyKey" .topologyKey) }}
  {{- end -}}
{{- end -}}

{{/*
Return a podAffinity/podAntiAffinity definition
{{ include "common.affinities.pods" (dict "type" "soft" "key" "FOO" "values" (list "BAR" "BAZ")) -}}
*/}}
{{- define "common.affinities.pods" -}}
  {{- if eq .type "soft" }}
    {{- include "common.affinities.pods.soft" . -}}
  {{- else if eq .type "hard" }}
    {{- include "common.affinities.pods.hard" . -}}
  {{- end -}}
{{- end -}}