{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}

{{/*
Return the target Kubernetes version
*/}}
{{- define "common.capabilities.kubeVersion" -}}
{{- default (default .Capabilities.KubeVersion.Version .Values.kubeVersion) ((.Values.global).kubeVersion) -}}
{{- end -}}

{{/*
Return true if the apiVersion is supported
Usage:
{{ include "common.capabilities.apiVersions.has" (dict "version" "batch/v1" "context" $) }}
*/}}
{{- define "common.capabilities.apiVersions.has" -}}
{{- $providedAPIVersions := default .context.Values.apiVersions ((.context.Values.global).apiVersions) -}}
{{- if and (empty $providedAPIVersions) (.context.Capabilities.APIVersions.Has .version) -}}
    {{- true -}}
{{- else if has .version $providedAPIVersions -}}
    {{- true -}}
{{- end -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for poddisruptionbudget.
*/}}
{{- define "common.capabilities.policy.apiVersion" -}}
{{- print "policy/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for networkpolicy.
*/}}
{{- define "common.capabilities.networkPolicy.apiVersion" -}}
{{- print "networking.k8s.io/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for job.
*/}}
{{- define "common.capabilities.job.apiVersion" -}}
{{- print "batch/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for cronjob.
*/}}
{{- define "common.capabilities.cronjob.apiVersion" -}}
{{- print "batch/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for daemonset.
*/}}
{{- define "common.capabilities.daemonset.apiVersion" -}}
{{- print "apps/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for deployment.
*/}}
{{- define "common.capabilities.deployment.apiVersion" -}}
{{- print "apps/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for statefulset.
*/}}
{{- define "common.capabilities.statefulset.apiVersion" -}}
{{- print "apps/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for ingress.
*/}}
{{- define "common.capabilities.ingress.apiVersion" -}}
{{- print "networking.k8s.io/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for RBAC resources.
*/}}
{{- define "common.capabilities.rbac.apiVersion" -}}
{{- print "rbac.authorization.k8s.io/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for CRDs.
*/}}
{{- define "common.capabilities.crd.apiVersion" -}}
{{- print "apiextensions.k8s.io/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for APIService.
*/}}
{{- define "common.capabilities.apiService.apiVersion" -}}
{{- print "apiregistration.k8s.io/v1" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for Horizontal Pod Autoscaler.
*/}}
{{- define "common.capabilities.hpa.apiVersion" -}}
{{- $kubeVersion := include "common.capabilities.kubeVersion" .context -}}
{{- print "autoscaling/v2" -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for Vertical Pod Autoscaler.
*/}}
{{- define "common.capabilities.vpa.apiVersion" -}}
{{- $kubeVersion := include "common.capabilities.kubeVersion" .context -}}
{{- if and (not (empty $kubeVersion)) (semverCompare "<1.25-0" $kubeVersion) -}}
{{- print "autoscaling/v1beta2" -}}
{{- else -}}
{{- print "autoscaling/v1" -}}
{{- end -}}
{{- end -}}

{{/*
Returns true if PodSecurityPolicy is supported
*/}}
{{- define "common.capabilities.psp.supported" -}}
{{- $kubeVersion := include "common.capabilities.kubeVersion" . -}}
{{- if or (empty $kubeVersion) (semverCompare "<1.25-0" $kubeVersion) -}}
  {{- true -}}
{{- end -}}
{{- end -}}

{{/*
Returns true if AdmissionConfiguration is supported
*/}}
{{- define "common.capabilities.admissionConfiguration.supported" -}}
{{- $kubeVersion := include "common.capabilities.kubeVersion" . -}}
  {{- true -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for AdmissionConfiguration.
*/}}
{{- define "common.capabilities.admissionConfiguration.apiVersion" -}}
{{- $kubeVersion := include "common.capabilities.kubeVersion" . -}}
{{- if and (not (empty $kubeVersion)) (semverCompare "<1.25-0" $kubeVersion) -}}
{{- print "apiserver.config.k8s.io/v1beta1" -}}
{{- else -}}
{{- print "apiserver.config.k8s.io/v1" -}}
{{- end -}}
{{- end -}}

{{/*
Return the appropriate apiVersion for PodSecurityConfiguration.
*/}}
{{- define "common.capabilities.podSecurityConfiguration.apiVersion" -}}
{{- $kubeVersion := include "common.capabilities.kubeVersion" . -}}
{{- if and (not (empty $kubeVersion)) (semverCompare "<1.25-0" $kubeVersion) -}}
{{- print "pod-security.admission.config.k8s.io/v1beta1" -}}
{{- else -}}
{{- print "pod-security.admission.config.k8s.io/v1" -}}
{{- end -}}
{{- end -}}

{{/*
Returns true if the used Helm version is 3.3+.
A way to check the used Helm version was not introduced until version 3.3.0 with .Capabilities.HelmVersion, which contains an additional "{}}"  structure.
This check is introduced as a regexMatch instead of {{ if .Capabilities.HelmVersion }} because checking for the key HelmVersion in <3.3 results in a "interface not found" error.
**To be removed when the catalog's minimun Helm version is 3.3**
*/}}
{{- define "common.capabilities.supportsHelmVersion" -}}
{{- if regexMatch "{(v[0-9])*[^}]*}}$" (.Capabilities | toString ) }}
  {{- true -}}
{{- end -}}
{{- end -}}