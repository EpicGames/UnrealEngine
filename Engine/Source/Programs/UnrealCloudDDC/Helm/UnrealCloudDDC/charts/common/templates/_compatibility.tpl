{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}

{{/* 
Return true if the detected platform is Openshift
Usage:
{{- include "common.compatibility.isOpenshift" . -}}
*/}}
{{- define "common.compatibility.isOpenshift" -}}
{{- if .Capabilities.APIVersions.Has "security.openshift.io/v1" -}}
{{- true -}}
{{- end -}}
{{- end -}}

{{/*
Render a compatible securityContext depending on the platform. By default it is maintained as it is. In other platforms like Openshift we remove default user/group values that do not work out of the box with the restricted-v1 SCC
Usage:
{{- include "common.compatibility.renderSecurityContext" (dict "secContext" .Values.containerSecurityContext "context" $) -}}
*/}}
{{- define "common.compatibility.renderSecurityContext" -}}
{{- $adaptedContext := .secContext -}}

{{- if (((.context.Values.global).compatibility).openshift) -}}
  {{- if or (eq .context.Values.global.compatibility.openshift.adaptSecurityContext "force") (and (eq .context.Values.global.compatibility.openshift.adaptSecurityContext "auto") (include "common.compatibility.isOpenshift" .context)) -}}
    {{/* Remove incompatible user/group values that do not work in Openshift out of the box */}}
    {{- $adaptedContext = omit $adaptedContext "fsGroup" "runAsUser" "runAsGroup" -}}
    {{- if not .secContext.seLinuxOptions -}}
    {{/* If it is an empty object, we remove it from the resulting context because it causes validation issues */}}
    {{- $adaptedContext = omit $adaptedContext "seLinuxOptions" -}}
    {{- end -}}
  {{- end -}}
{{- end -}}
{{/* Remove empty seLinuxOptions object if global.compatibility.omitEmptySeLinuxOptions is set to true */}}
{{- if and (((.context.Values.global).compatibility).omitEmptySeLinuxOptions) (not .secContext.seLinuxOptions) -}}
  {{- $adaptedContext = omit $adaptedContext "seLinuxOptions" -}}
{{- end -}}
{{/* Remove fields that are disregarded when running the container in privileged mode */}}
{{- if $adaptedContext.privileged -}}
  {{- $adaptedContext = omit $adaptedContext "capabilities" -}}
{{- end -}}
{{- omit $adaptedContext "enabled" | toYaml -}}
{{- end -}}