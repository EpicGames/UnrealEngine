{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Throw error when upgrading using empty passwords values that must not be empty.

Usage:
{{- $validationError00 := include "common.validations.values.single.empty" (dict "valueKey" "path.to.password00" "secret" "secretName" "field" "password-00") -}}
{{- $validationError01 := include "common.validations.values.single.empty" (dict "valueKey" "path.to.password01" "secret" "secretName" "field" "password-01") -}}
{{ include "common.errors.upgrade.passwords.empty" (dict "validationErrors" (list $validationError00 $validationError01) "context" $) }}

Required password params:
  - validationErrors - String - Required. List of validation strings to be return, if it is empty it won't throw error.
  - context - Context - Required. Parent context.
*/}}
{{- define "common.errors.upgrade.passwords.empty" -}}
  {{- $validationErrors := join "" .validationErrors -}}
  {{- if and $validationErrors .context.Release.IsUpgrade -}}
    {{- $errorString := "\nPASSWORDS ERROR: You must provide your current passwords when upgrading the release." -}}
    {{- $errorString = print $errorString "\n                 Note that even after reinstallation, old credentials may be needed as they may be kept in persistent volume claims." -}}
    {{- $errorString = print $errorString "\n                 Further information can be obtained at https://docs.bitnami.com/general/how-to/troubleshoot-helm-chart-issues/#credential-errors-while-upgrading-chart-releases" -}}
    {{- $errorString = print $errorString "\n%s" -}}
    {{- printf $errorString $validationErrors | fail -}}
  {{- end -}}
{{- end -}}

{{/*
Throw error when original container images are replaced.
The error can be bypassed by setting the "global.security.allowInsecureImages" to true. In this case,
a warning message will be shown instead.

Usage:
{{ include "common.errors.insecureImages" (dict "images" (list .Values.path.to.the.imageRoot) "context" $) }}
*/}}
{{- define "common.errors.insecureImages" -}}
{{- $relocatedImages := list -}}
{{- $replacedImages := list -}}
{{- $retaggedImages := list -}}
{{- $globalRegistry := ((.context.Values.global).imageRegistry) -}}
{{- $originalImages := .context.Chart.Annotations.images -}}
{{- range .images -}}
  {{- $registryName := default .registry $globalRegistry -}}
  {{- $fullImageNameNoTag := printf "%s/%s" $registryName .repository -}}
  {{- $fullImageName := printf "%s:%s" $fullImageNameNoTag .tag -}}
  {{- if not (contains $fullImageNameNoTag $originalImages) -}}
    {{- if not (contains $registryName $originalImages) -}}
      {{- $relocatedImages = append $relocatedImages $fullImageName  -}}
    {{- else if not (contains .repository $originalImages) -}}
      {{- $replacedImages = append $replacedImages $fullImageName  -}}
    {{- end -}}
  {{- end -}}
  {{- if not (contains (printf "%s:%s" .repository .tag) $originalImages) -}}
    {{- $retaggedImages = append $retaggedImages $fullImageName  -}}
  {{- end -}}
{{- end -}}

{{- if and (or (gt (len $relocatedImages) 0) (gt (len $replacedImages) 0)) (((.context.Values.global).security).allowInsecureImages) -}}
  {{- print "\n\n⚠ SECURITY WARNING: Verifying original container images was skipped. Please note this Helm chart was designed, tested, and validated on multiple platforms using a specific set of Bitnami and Tanzu Application Catalog containers. Substituting other containers is likely to cause degraded security and performance, broken chart features, and missing environment variables.\n" -}}
{{- else if (or (gt (len $relocatedImages) 0) (gt (len $replacedImages) 0)) -}}
  {{- $errorString := "Original containers have been substituted for unrecognized ones. Deploying this chart with non-standard containers is likely to cause degraded security and performance, broken chart features, and missing environment variables." -}}
  {{- $errorString = print $errorString "\n\nUnrecognized images:" -}}
  {{- range (concat $relocatedImages $replacedImages) -}}
    {{- $errorString = print $errorString "\n  - " . -}}
  {{- end -}}
  {{- if or (contains "docker.io/bitnami/" $originalImages) (contains "docker.io/bitnamiprem/" $originalImages) -}}
    {{- $errorString = print "\n\n⚠ ERROR: " $errorString -}}
    {{- $errorString = print $errorString "\n\nIf you are sure you want to proceed with non-standard containers, you can skip container image verification by setting the global parameter 'global.security.allowInsecureImages' to true." -}}
    {{- $errorString = print $errorString "\nFurther information can be obtained at https://github.com/bitnami/charts/issues/30850" -}}
    {{- print $errorString | fail -}}
  {{- else if gt (len $replacedImages) 0 -}}
    {{- $errorString = print "\n\n⚠ WARNING: " $errorString -}}
    {{- print $errorString -}}
  {{- end -}}
{{- else if gt (len $retaggedImages) 0 -}}
  {{- $warnString := "\n\n⚠ WARNING: Original containers have been retagged. Please note this Helm chart was tested, and validated on multiple platforms using a specific set of Tanzu Application Catalog containers. Substituting original image tags could cause unexpected behavior." -}}
  {{- $warnString = print $warnString "\n\nRetagged images:" -}}
  {{- range $retaggedImages -}}
    {{- $warnString = print $warnString "\n  - " . -}}
  {{- end -}}
  {{- print $warnString -}}
{{- end -}}
{{- end -}}