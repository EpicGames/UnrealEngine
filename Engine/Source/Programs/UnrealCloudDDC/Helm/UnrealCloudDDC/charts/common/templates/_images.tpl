{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Return the proper image name.
If image tag and digest are not defined, termination fallbacks to chart appVersion.
{{ include "common.images.image" ( dict "imageRoot" .Values.path.to.the.image "global" .Values.global "chart" .Chart ) }}
*/}}
{{- define "common.images.image" -}}
{{- $registryName := default .imageRoot.registry ((.global).imageRegistry) -}}
{{- $repositoryName := .imageRoot.repository -}}
{{- $separator := ":" -}}
{{- $termination := .imageRoot.tag | toString -}}

{{- if not .imageRoot.tag }}
  {{- if .chart }}
    {{- $termination = .chart.AppVersion | toString -}}
  {{- end -}}
{{- end -}}
{{- if .imageRoot.digest }}
    {{- $separator = "@" -}}
    {{- $termination = .imageRoot.digest | toString -}}
{{- end -}}
{{- if $registryName }}
    {{- printf "%s/%s%s%s" $registryName $repositoryName $separator $termination -}}
{{- else -}}
    {{- printf "%s%s%s"  $repositoryName $separator $termination -}}
{{- end -}}
{{- end -}}

{{/*
Return the proper Docker Image Registry Secret Names (deprecated: use common.images.renderPullSecrets instead)
{{ include "common.images.pullSecrets" ( dict "images" (list .Values.path.to.the.image1, .Values.path.to.the.image2) "global" .Values.global) }}
*/}}
{{- define "common.images.pullSecrets" -}}
  {{- $pullSecrets := list }}

  {{- range ((.global).imagePullSecrets) -}}
    {{- if kindIs "map" . -}}
      {{- $pullSecrets = append $pullSecrets .name -}}
    {{- else -}}
      {{- $pullSecrets = append $pullSecrets . -}}
    {{- end }}
  {{- end -}}

  {{- range .images -}}
    {{- range .pullSecrets -}}
      {{- if kindIs "map" . -}}
        {{- $pullSecrets = append $pullSecrets .name -}}
      {{- else -}}
        {{- $pullSecrets = append $pullSecrets . -}}
      {{- end -}}
    {{- end -}}
  {{- end -}}

  {{- if (not (empty $pullSecrets)) -}}
imagePullSecrets:
    {{- range $pullSecrets | uniq }}
  - name: {{ . }}
    {{- end }}
  {{- end }}
{{- end -}}

{{/*
Return the proper Docker Image Registry Secret Names evaluating values as templates
{{ include "common.images.renderPullSecrets" ( dict "images" (list .Values.path.to.the.image1, .Values.path.to.the.image2) "context" $) }}
*/}}
{{- define "common.images.renderPullSecrets" -}}
  {{- $pullSecrets := list }}
  {{- $context := .context }}

  {{- range (($context.Values.global).imagePullSecrets) -}}
    {{- if kindIs "map" . -}}
      {{- $pullSecrets = append $pullSecrets (include "common.tplvalues.render" (dict "value" .name "context" $context)) -}}
    {{- else -}}
      {{- $pullSecrets = append $pullSecrets (include "common.tplvalues.render" (dict "value" . "context" $context)) -}}
    {{- end -}}
  {{- end -}}

  {{- range .images -}}
    {{- range .pullSecrets -}}
      {{- if kindIs "map" . -}}
        {{- $pullSecrets = append $pullSecrets (include "common.tplvalues.render" (dict "value" .name "context" $context)) -}}
      {{- else -}}
        {{- $pullSecrets = append $pullSecrets (include "common.tplvalues.render" (dict "value" . "context" $context)) -}}
      {{- end -}}
    {{- end -}}
  {{- end -}}

  {{- if (not (empty $pullSecrets)) -}}
imagePullSecrets:
    {{- range $pullSecrets | uniq }}
  - name: {{ . }}
    {{- end }}
  {{- end }}
{{- end -}}

{{/*
Return the proper image version (ingores image revision/prerelease info & fallbacks to chart appVersion)
{{ include "common.images.version" ( dict "imageRoot" .Values.path.to.the.image "chart" .Chart ) }}
*/}}
{{- define "common.images.version" -}}
{{- $imageTag := .imageRoot.tag | toString -}}
{{/* regexp from https://github.com/Masterminds/semver/blob/23f51de38a0866c5ef0bfc42b3f735c73107b700/version.go#L41-L44 */}}
{{- if regexMatch `^([0-9]+)(\.[0-9]+)?(\.[0-9]+)?(-([0-9A-Za-z\-]+(\.[0-9A-Za-z\-]+)*))?(\+([0-9A-Za-z\-]+(\.[0-9A-Za-z\-]+)*))?$` $imageTag -}}
    {{- $version := semver $imageTag -}}
    {{- printf "%d.%d.%d" $version.Major $version.Minor $version.Patch -}}
{{- else -}}
    {{- print .chart.AppVersion -}}
{{- end -}}
{{- end -}}
