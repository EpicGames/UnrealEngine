{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}

{{/*
Generate backend entry that is compatible with all Kubernetes API versions.

Usage:
{{ include "common.ingress.backend" (dict "serviceName" "backendName" "servicePort" "backendPort" "context" $) }}

Params:
  - serviceName - String. Name of an existing service backend
  - servicePort - String/Int. Port name (or number) of the service. It will be translated to different yaml depending if it is a string or an integer.
  - context - Dict - Required. The context for the template evaluation.
*/}}
{{- define "common.ingress.backend" -}}
service:
  name: {{ .serviceName }}
  port:
    {{- if typeIs "string" .servicePort }}
    name: {{ .servicePort }}
    {{- else if or (typeIs "int" .servicePort) (typeIs "float64" .servicePort) }}
    number: {{ .servicePort | int }}
    {{- end }}
{{- end -}}

{{/*
TODO: Remove as soon it is removed from the rest of the charts
Print "true" if the API pathType field is supported
Usage:
{{ include "common.ingress.supportsPathType" . }}
*/}}
{{- define "common.ingress.supportsPathType" -}}
{{- print "true" -}}
{{- end -}}

{{/*
TODO: Remove as soon it is removed from the rest of the charts
Returns true if the ingressClassname field is supported
Usage:
{{ include "common.ingress.supportsIngressClassname" . }}
*/}}
{{- define "common.ingress.supportsIngressClassname" -}}
{{- print "true" -}}
{{- end -}}

{{/*
Return true if cert-manager required annotations for TLS signed
certificates are set in the Ingress annotations
Ref: https://cert-manager.io/docs/usage/ingress/#supported-annotations
Usage:
{{ include "common.ingress.certManagerRequest" ( dict "annotations" .Values.path.to.the.ingress.annotations ) }}
*/}}
{{- define "common.ingress.certManagerRequest" -}}
{{ if or (hasKey .annotations "cert-manager.io/cluster-issuer") (hasKey .annotations "cert-manager.io/issuer") (hasKey .annotations "kubernetes.io/tls-acme") }}
    {{- true -}}
{{- end -}}
{{- end -}}