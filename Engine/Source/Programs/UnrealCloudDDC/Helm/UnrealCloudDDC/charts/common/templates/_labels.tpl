{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}

{{/*
Kubernetes standard labels
{{ include "common.labels.standard" (dict "customLabels" .Values.commonLabels "context" $) -}}
*/}}
{{- define "common.labels.standard" -}}
{{- if and (hasKey . "customLabels") (hasKey . "context") -}}
{{- $default := dict "app.kubernetes.io/name" (include "common.names.name" .context) "helm.sh/chart" (include "common.names.chart" .context) "app.kubernetes.io/instance" .context.Release.Name "app.kubernetes.io/managed-by" .context.Release.Service -}}
{{- with .context.Chart.AppVersion -}}
{{- $_ := set $default "app.kubernetes.io/version" . -}}
{{- end -}}
{{ template "common.tplvalues.merge" (dict "values" (list .customLabels $default) "context" .context) }}
{{- else -}}
app.kubernetes.io/name: {{ include "common.names.name" . }}
helm.sh/chart: {{ include "common.names.chart" . }}
app.kubernetes.io/instance: {{ .Release.Name }}
app.kubernetes.io/managed-by: {{ .Release.Service }}
{{- with .Chart.AppVersion }}
app.kubernetes.io/version: {{ . | quote }}
{{- end -}}
{{- end -}}
{{- end -}}

{{/*
Labels used on immutable fields such as deploy.spec.selector.matchLabels or svc.spec.selector
{{ include "common.labels.matchLabels" (dict "customLabels" .Values.podLabels "context" $) -}}

We don't want to loop over custom labels appending them to the selector
since it's very likely that it will break deployments, services, etc.
However, it's important to overwrite the standard labels if the user
overwrote them on metadata.labels fields.
*/}}
{{- define "common.labels.matchLabels" -}}
{{- if and (hasKey . "customLabels") (hasKey . "context") -}}
{{ merge (pick (include "common.tplvalues.render" (dict "value" .customLabels "context" .context) | fromYaml) "app.kubernetes.io/name" "app.kubernetes.io/instance") (dict "app.kubernetes.io/name" (include "common.names.name" .context) "app.kubernetes.io/instance" .context.Release.Name ) | toYaml }}
{{- else -}}
app.kubernetes.io/name: {{ include "common.names.name" . }}
app.kubernetes.io/instance: {{ .Release.Name }}
{{- end -}}
{{- end -}}