{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Expand the name of the chart.
*/}}
{{- define "common.names.name" -}}
{{- default .Chart.Name .Values.nameOverride | trunc 63 | trimSuffix "-" -}}
{{- end -}}

{{/*
Create chart name and version as used by the chart label.
*/}}
{{- define "common.names.chart" -}}
{{- printf "%s-%s" .Chart.Name .Chart.Version | replace "+" "_" | trunc 63 | trimSuffix "-" -}}
{{- end -}}

{{/*
Create a default fully qualified app name.
We truncate at 63 chars because some Kubernetes name fields are limited to this (by the DNS naming spec).
If release name contains chart name it will be used as a full name.
*/}}
{{- define "common.names.fullname" -}}
{{- if .Values.fullnameOverride -}}
{{- .Values.fullnameOverride | trunc 63 | trimSuffix "-" -}}
{{- else -}}
{{- $name := default .Chart.Name .Values.nameOverride -}}
{{- if contains $name .Release.Name -}}
{{- .Release.Name | trunc 63 | trimSuffix "-" -}}
{{- else -}}
{{- printf "%s-%s" .Release.Name $name | trunc 63 | trimSuffix "-" -}}
{{- end -}}
{{- end -}}
{{- end -}}

{{/*
Create a default fully qualified dependency name.
We truncate at 63 chars because some Kubernetes name fields are limited to this (by the DNS naming spec).
If release name contains chart name it will be used as a full name.
Usage:
{{ include "common.names.dependency.fullname" (dict "chartName" "dependency-chart-name" "chartValues" .Values.dependency-chart "context" $) }}
*/}}
{{- define "common.names.dependency.fullname" -}}
{{- if .chartValues.fullnameOverride -}}
{{- .chartValues.fullnameOverride | trunc 63 | trimSuffix "-" -}}
{{- else -}}
{{- $name := default .chartName .chartValues.nameOverride -}}
{{- if contains $name .context.Release.Name -}}
{{- .context.Release.Name | trunc 63 | trimSuffix "-" -}}
{{- else -}}
{{- printf "%s-%s" .context.Release.Name $name | trunc 63 | trimSuffix "-" -}}
{{- end -}}
{{- end -}}
{{- end -}}

{{/*
Allow the release namespace to be overridden for multi-namespace deployments in combined charts.
*/}}
{{- define "common.names.namespace" -}}
{{- default .Release.Namespace .Values.namespaceOverride | trunc 63 | trimSuffix "-" -}}
{{- end -}}

{{/*
Create a fully qualified app name adding the installation's namespace.
*/}}
{{- define "common.names.fullname.namespace" -}}
{{- printf "%s-%s" (include "common.names.fullname" .) (include "common.names.namespace" .) | trunc 63 | trimSuffix "-" -}}
{{- end -}}