{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}

{{/*
Return a resource request/limit object based on a given preset.
These presets are for basic testing and not meant to be used in production
{{ include "common.resources.preset" (dict "type" "nano") -}}
*/}}
{{- define "common.resources.preset" -}}
{{/* The limits are the requests increased by 50% (except ephemeral-storage and xlarge/2xlarge sizes)*/}}
{{- $presets := dict 
  "nano" (dict 
      "requests" (dict "cpu" "100m" "memory" "128Mi" "ephemeral-storage" "50Mi")
      "limits" (dict "cpu" "150m" "memory" "192Mi" "ephemeral-storage" "2Gi")
   )
  "micro" (dict 
      "requests" (dict "cpu" "250m" "memory" "256Mi" "ephemeral-storage" "50Mi")
      "limits" (dict "cpu" "375m" "memory" "384Mi" "ephemeral-storage" "2Gi")
   )
  "small" (dict 
      "requests" (dict "cpu" "500m" "memory" "512Mi" "ephemeral-storage" "50Mi")
      "limits" (dict "cpu" "750m" "memory" "768Mi" "ephemeral-storage" "2Gi")
   )
  "medium" (dict 
      "requests" (dict "cpu" "500m" "memory" "1024Mi" "ephemeral-storage" "50Mi")
      "limits" (dict "cpu" "750m" "memory" "1536Mi" "ephemeral-storage" "2Gi")
   )
  "large" (dict 
      "requests" (dict "cpu" "1.0" "memory" "2048Mi" "ephemeral-storage" "50Mi")
      "limits" (dict "cpu" "1.5" "memory" "3072Mi" "ephemeral-storage" "2Gi")
   )
  "xlarge" (dict 
      "requests" (dict "cpu" "1.0" "memory" "3072Mi" "ephemeral-storage" "50Mi")
      "limits" (dict "cpu" "3.0" "memory" "6144Mi" "ephemeral-storage" "2Gi")
   )
  "2xlarge" (dict 
      "requests" (dict "cpu" "1.0" "memory" "3072Mi" "ephemeral-storage" "50Mi")
      "limits" (dict "cpu" "6.0" "memory" "12288Mi" "ephemeral-storage" "2Gi")
   )
 }}
{{- if hasKey $presets .type -}}
{{- index $presets .type | toYaml -}}
{{- else -}}
{{- printf "ERROR: Preset key '%s' invalid. Allowed values are %s" .type (join "," (keys $presets)) | fail -}}
{{- end -}}
{{- end -}}