{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Generate secret name.

Usage:
{{ include "common.secrets.name" (dict "existingSecret" .Values.path.to.the.existingSecret "defaultNameSuffix" "mySuffix" "context" $) }}

Params:
  - existingSecret - ExistingSecret/String - Optional. The path to the existing secrets in the values.yaml given by the user
    to be used instead of the default one. Allows for it to be of type String (just the secret name) for backwards compatibility.
    +info: https://github.com/bitnami/charts/tree/main/bitnami/common#existingsecret
  - defaultNameSuffix - String - Optional. It is used only if we have several secrets in the same deployment.
  - context - Dict - Required. The context for the template evaluation.
*/}}
{{- define "common.secrets.name" -}}
{{- $name := (include "common.names.fullname" .context) -}}

{{- if .defaultNameSuffix -}}
{{- $name = printf "%s-%s" $name .defaultNameSuffix | trunc 63 | trimSuffix "-" -}}
{{- end -}}

{{- with .existingSecret -}}
{{- if not (typeIs "string" .) -}}
{{- with .name -}}
{{- $name = . -}}
{{- end -}}
{{- else -}}
{{- $name = . -}}
{{- end -}}
{{- end -}}

{{- printf "%s" $name -}}
{{- end -}}

{{/*
Generate secret key.

Usage:
{{ include "common.secrets.key" (dict "existingSecret" .Values.path.to.the.existingSecret "key" "keyName") }}

Params:
  - existingSecret - ExistingSecret/String - Optional. The path to the existing secrets in the values.yaml given by the user
    to be used instead of the default one. Allows for it to be of type String (just the secret name) for backwards compatibility.
    +info: https://github.com/bitnami/charts/tree/main/bitnami/common#existingsecret
  - key - String - Required. Name of the key in the secret.
*/}}
{{- define "common.secrets.key" -}}
{{- $key := .key -}}

{{- if .existingSecret -}}
  {{- if not (typeIs "string" .existingSecret) -}}
    {{- if .existingSecret.keyMapping -}}
      {{- $key = index .existingSecret.keyMapping $.key -}}
    {{- end -}}
  {{- end }}
{{- end -}}

{{- printf "%s" $key -}}
{{- end -}}

{{/*
Generate secret password or retrieve one if already created.

Usage:
{{ include "common.secrets.passwords.manage" (dict "secret" "secret-name" "key" "keyName" "providedValues" (list "path.to.password1" "path.to.password2") "length" 10 "strong" false "chartName" "chartName" "honorProvidedValues" false "context" $) }}

Params:
  - secret - String - Required - Name of the 'Secret' resource where the password is stored.
  - key - String - Required - Name of the key in the secret.
  - providedValues - List<String> - Required - The path to the validating value in the values.yaml, e.g: "mysql.password". Will pick first parameter with a defined value.
  - length - int - Optional - Length of the generated random password.
  - strong - Boolean - Optional - Whether to add symbols to the generated random password.
  - chartName - String - Optional - Name of the chart used when said chart is deployed as a subchart.
  - context - Context - Required - Parent context.
  - failOnNew - Boolean - Optional - Default to true. If set to false, skip errors adding new keys to existing secrets.
  - skipB64enc - Boolean - Optional - Default to false. If set to true, no the secret will not be base64 encrypted.
  - skipQuote - Boolean - Optional - Default to false. If set to true, no quotes will be added around the secret.
  - honorProvidedValues - Boolean - Optional - Default to false. If set to true, the values in providedValues have higher priority than an existing secret
The order in which this function returns a secret password:
  1. Password provided via the values.yaml if honorProvidedValues = true
     (If one of the keys passed to the 'providedValues' parameter to this function is a valid path to a key in the values.yaml and has a value, the value of the first key with a value will be returned)
  2. Already existing 'Secret' resource
     (If a 'Secret' resource is found under the name provided to the 'secret' parameter to this function and that 'Secret' resource contains a key with the name passed as the 'key' parameter to this function then the value of this existing secret password will be returned)
  3. Password provided via the values.yaml if honorProvidedValues = false
     (If one of the keys passed to the 'providedValues' parameter to this function is a valid path to a key in the values.yaml and has a value, the value of the first key with a value will be returned)
  4. Randomly generated secret password
     (A new random secret password with the length specified in the 'length' parameter will be generated and returned)

*/}}
{{- define "common.secrets.passwords.manage" -}}

{{- $password := "" }}
{{- $subchart := "" }}
{{- $chartName := default "" .chartName }}
{{- $passwordLength := default 10 .length }}
{{- $providedPasswordKey := include "common.utils.getKeyFromList" (dict "keys" .providedValues "context" $.context) }}
{{- $providedPasswordValue := include "common.utils.getValueFromKey" (dict "key" $providedPasswordKey "context" $.context) }}
{{- $secretData := (lookup "v1" "Secret" (include "common.names.namespace" .context) .secret).data }}
{{- if $secretData }}
  {{- if hasKey $secretData .key }}
    {{- $password = index $secretData .key | b64dec }}
  {{- else if not (eq .failOnNew false) }}
    {{- printf "\nPASSWORDS ERROR: The secret \"%s\" does not contain the key \"%s\"\n" .secret .key | fail -}}
  {{- end -}}
{{- end }}

{{- if and $providedPasswordValue .honorProvidedValues }}
  {{- $password = tpl ($providedPasswordValue | toString) .context }}
{{- end }}

{{- if not $password }}
  {{- if $providedPasswordValue }}
    {{- $password = tpl ($providedPasswordValue | toString) .context }}
  {{- else }}
    {{- if .context.Values.enabled }}
      {{- $subchart = $chartName }}
    {{- end -}}

    {{- if not (eq .failOnNew false) }}
      {{- $requiredPassword := dict "valueKey" $providedPasswordKey "secret" .secret "field" .key "subchart" $subchart "context" $.context -}}
      {{- $requiredPasswordError := include "common.validations.values.single.empty" $requiredPassword -}}
      {{- $passwordValidationErrors := list $requiredPasswordError -}}
      {{- include "common.errors.upgrade.passwords.empty" (dict "validationErrors" $passwordValidationErrors "context" $.context) -}}
    {{- end }}

    {{- if .strong }}
      {{- $subStr := list (lower (randAlpha 1)) (randNumeric 1) (upper (randAlpha 1)) | join "_" }}
      {{- $password = randAscii $passwordLength }}
      {{- $password = regexReplaceAllLiteral "\\W" $password "@" | substr 5 $passwordLength }}
      {{- $password = printf "%s%s" $subStr $password | toString | shuffle }}
    {{- else }}
      {{- $password = randAlphaNum $passwordLength }}
    {{- end }}
  {{- end -}}
{{- end -}}
{{- if not .skipB64enc }}
{{- $password = $password | b64enc }}
{{- end -}}
{{- if .skipQuote -}}
{{- printf "%s" $password -}}
{{- else -}}
{{- printf "%s" $password | quote -}}
{{- end -}}
{{- end -}}

{{/*
Reuses the value from an existing secret, otherwise sets its value to a default value.

Usage:
{{ include "common.secrets.lookup" (dict "secret" "secret-name" "key" "keyName" "defaultValue" .Values.myValue "context" $) }}

Params:
  - secret - String - Required - Name of the 'Secret' resource where the password is stored.
  - key - String - Required - Name of the key in the secret.
  - defaultValue - String - Required - The path to the validating value in the values.yaml, e.g: "mysql.password". Will pick first parameter with a defined value.
  - context - Context - Required - Parent context.

*/}}
{{- define "common.secrets.lookup" -}}
{{- $value := "" -}}
{{- $secretData := (lookup "v1" "Secret" (include "common.names.namespace" .context) .secret).data -}}
{{- if and $secretData (hasKey $secretData .key) -}}
  {{- $value = index $secretData .key -}}
{{- else if .defaultValue -}}
  {{- $value = .defaultValue | toString | b64enc -}}
{{- end -}}
{{- if $value -}}
{{- printf "%s" $value -}}
{{- end -}}
{{- end -}}

{{/*
Returns whether a previous generated secret already exists

Usage:
{{ include "common.secrets.exists" (dict "secret" "secret-name" "context" $) }}

Params:
  - secret - String - Required - Name of the 'Secret' resource where the password is stored.
  - context - Context - Required - Parent context.
*/}}
{{- define "common.secrets.exists" -}}
{{- $secret := (lookup "v1" "Secret" (include "common.names.namespace" .context) .secret) }}
{{- if $secret }}
  {{- true -}}
{{- end -}}
{{- end -}}