{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}

{{/*
Return  the proper Storage Class
{{ include "common.storage.class" ( dict "persistence" .Values.path.to.the.persistence "global" $) }}
*/}}
{{- define "common.storage.class" -}}
{{- $storageClass := (.global).storageClass | default .persistence.storageClass | default (.global).defaultStorageClass | default "" -}}
{{- if $storageClass -}}
  {{- if (eq "-" $storageClass) -}}
      {{- printf "storageClassName: \"\"" -}}
  {{- else -}}
      {{- printf "storageClassName: %s" $storageClass -}}
  {{- end -}}
{{- end -}}
{{- end -}}