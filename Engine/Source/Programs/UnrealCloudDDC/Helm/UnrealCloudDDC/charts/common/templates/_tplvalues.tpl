{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Renders a value that contains template perhaps with scope if the scope is present.
Usage:
{{ include "common.tplvalues.render" ( dict "value" .Values.path.to.the.Value "context" $ ) }}
{{ include "common.tplvalues.render" ( dict "value" .Values.path.to.the.Value "context" $ "scope" $app ) }}
*/}}
{{- define "common.tplvalues.render" -}}
{{- $value := typeIs "string" .value | ternary .value (.value | toYaml) }}
{{- if contains "{{" (toJson .value) }}
  {{- if .scope }}
      {{- tpl (cat "{{- with $.RelativeScope -}}" $value "{{- end }}") (merge (dict "RelativeScope" .scope) .context) }}
  {{- else }}
    {{- tpl $value .context }}
  {{- end }}
{{- else }}
    {{- $value }}
{{- end }}
{{- end -}}

{{/*
Merge a list of values that contains template after rendering them.
Merge precedence is consistent with http://masterminds.github.io/sprig/dicts.html#merge-mustmerge
Usage:
{{ include "common.tplvalues.merge" ( dict "values" (list .Values.path.to.the.Value1 .Values.path.to.the.Value2) "context" $ ) }}
*/}}
{{- define "common.tplvalues.merge" -}}
{{- $dst := dict -}}
{{- range .values -}}
{{- $dst = include "common.tplvalues.render" (dict "value" . "context" $.context "scope" $.scope) | fromYaml | merge $dst -}}
{{- end -}}
{{ $dst | toYaml }}
{{- end -}}

{{/*
Merge a list of values that contains template after rendering them.
Merge precedence is consistent with https://masterminds.github.io/sprig/dicts.html#mergeoverwrite-mustmergeoverwrite
Usage:
{{ include "common.tplvalues.merge-overwrite" ( dict "values" (list .Values.path.to.the.Value1 .Values.path.to.the.Value2) "context" $ ) }}
*/}}
{{- define "common.tplvalues.merge-overwrite" -}}
{{- $dst := dict -}}
{{- range .values -}}
{{- $dst = include "common.tplvalues.render" (dict "value" . "context" $.context "scope" $.scope) | fromYaml | mergeOverwrite $dst -}}
{{- end -}}
{{ $dst | toYaml }}
{{- end -}}