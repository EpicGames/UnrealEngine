{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Print instructions to get a secret value.
Usage:
{{ include "common.utils.secret.getvalue" (dict "secret" "secret-name" "field" "secret-value-field" "context" $) }}
*/}}
{{- define "common.utils.secret.getvalue" -}}
{{- $varname := include "common.utils.fieldToEnvVar" . -}}
export {{ $varname }}=$(kubectl get secret --namespace {{ include "common.names.namespace" .context | quote }} {{ .secret }} -o jsonpath="{.data.{{ .field }}}" | base64 -d)
{{- end -}}

{{/*
Build env var name given a field
Usage:
{{ include "common.utils.fieldToEnvVar" dict "field" "my-password" }}
*/}}
{{- define "common.utils.fieldToEnvVar" -}}
  {{- $fieldNameSplit := splitList "-" .field -}}
  {{- $upperCaseFieldNameSplit := list -}}

  {{- range $fieldNameSplit -}}
    {{- $upperCaseFieldNameSplit = append $upperCaseFieldNameSplit ( upper . ) -}}
  {{- end -}}

  {{ join "_" $upperCaseFieldNameSplit }}
{{- end -}}

{{/*
Gets a value from .Values given
Usage:
{{ include "common.utils.getValueFromKey" (dict "key" "path.to.key" "context" $) }}
*/}}
{{- define "common.utils.getValueFromKey" -}}
{{- $splitKey := splitList "." .key -}}
{{- $value := "" -}}
{{- $latestObj := $.context.Values -}}
{{- range $splitKey -}}
  {{- if not $latestObj -}}
    {{- printf "please review the entire path of '%s' exists in values" $.key | fail -}}
  {{- end -}}
  {{- $value = ( index $latestObj . ) -}}
  {{- $latestObj = $value -}}
{{- end -}}
{{- printf "%v" (default "" $value) -}} 
{{- end -}}

{{/*
Returns first .Values key with a defined value or first of the list if all non-defined
Usage:
{{ include "common.utils.getKeyFromList" (dict "keys" (list "path.to.key1" "path.to.key2") "context" $) }}
*/}}
{{- define "common.utils.getKeyFromList" -}}
{{- $key := first .keys -}}
{{- $reverseKeys := reverse .keys }}
{{- range $reverseKeys }}
  {{- $value := include "common.utils.getValueFromKey" (dict "key" . "context" $.context ) }}
  {{- if $value -}}
    {{- $key = . }}
  {{- end -}}
{{- end -}}
{{- printf "%s" $key -}} 
{{- end -}}

{{/*
Checksum a template at "path" containing a *single* resource (ConfigMap,Secret) for use in pod annotations, excluding the metadata (see #18376).
Usage:
{{ include "common.utils.checksumTemplate" (dict "path" "/configmap.yaml" "context" $) }}
*/}}
{{- define "common.utils.checksumTemplate" -}}
{{- $obj := include (print .context.Template.BasePath .path) .context | fromYaml -}}
{{ omit $obj "apiVersion" "kind" "metadata" | toYaml | sha256sum }}
{{- end -}}