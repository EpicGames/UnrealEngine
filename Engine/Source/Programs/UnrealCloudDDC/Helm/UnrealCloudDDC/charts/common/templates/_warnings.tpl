{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Warning about using rolling tag.
Usage:
{{ include "common.warnings.rollingTag" .Values.path.to.the.imageRoot }}
*/}}
{{- define "common.warnings.rollingTag" -}}

{{- if and (contains "bitnami/" .repository) (not (.tag | toString | regexFind "-r\\d+$|sha256:")) }}
WARNING: Rolling tag detected ({{ .repository }}:{{ .tag }}), please note that it is strongly recommended to avoid using rolling tags in a production environment.
+info https://techdocs.broadcom.com/us/en/vmware-tanzu/application-catalog/tanzu-application-catalog/services/tac-doc/apps-tutorials-understand-rolling-tags-containers-index.html
{{- end }}
{{- end -}}

{{/*
Warning about replaced images from the original.
Usage:
{{ include "common.warnings.modifiedImages" (dict "images" (list .Values.path.to.the.imageRoot) "context" $) }}
*/}}
{{- define "common.warnings.modifiedImages" -}}
{{- $affectedImages := list -}}
{{- $printMessage := false -}}
{{- $originalImages := .context.Chart.Annotations.images -}}
{{- range .images -}}
  {{- $fullImageName := printf (printf "%s/%s:%s" .registry .repository .tag) -}}
  {{- if not (contains $fullImageName $originalImages) }}
    {{- $affectedImages = append $affectedImages (printf "%s/%s:%s" .registry .repository .tag) -}}
    {{- $printMessage = true -}}
  {{- end -}}
{{- end -}}
{{- if $printMessage }}

⚠ SECURITY WARNING: Original containers have been substituted. This Helm chart was designed, tested, and validated on multiple platforms using a specific set of Bitnami and Tanzu Application Catalog containers. Substituting other containers is likely to cause degraded security and performance, broken chart features, and missing environment variables.

Substituted images detected:
{{- range $affectedImages }}
  - {{ . }}
{{- end }}
{{- end -}}
{{- end -}}

{{/*
Warning about not setting the resource object in all deployments.
Usage:
{{ include "common.warnings.resources" (dict "sections" (list "path1" "path2") context $) }}
Example:
{{- include "common.warnings.resources" (dict "sections" (list "csiProvider.provider" "server" "volumePermissions" "") "context" $) }}
The list in the example assumes that the following values exist:
  - csiProvider.provider.resources
  - server.resources
  - volumePermissions.resources
  - resources
*/}}
{{- define "common.warnings.resources" -}}
{{- $values := .context.Values -}}
{{- $printMessage := false -}}
{{ $affectedSections := list -}}
{{- range .sections -}}
  {{- if eq . "" -}}
    {{/* Case where the resources section is at the root (one main deployment in the chart) */}}
    {{- if not (index $values "resources") -}}
    {{- $affectedSections = append $affectedSections "resources" -}}
    {{- $printMessage = true -}}
    {{- end -}}
  {{- else -}}
    {{/* Case where the are multiple resources sections (more than one main deployment in the chart) */}}
    {{- $keys := split "." . -}}
    {{/* We iterate through the different levels until arriving to the resource section. Example: a.b.c.resources */}}
    {{- $section := $values -}}
    {{- range $keys -}}
      {{- $section = index $section . -}}
    {{- end -}}
    {{- if not (index $section "resources") -}}
      {{/* If the section has enabled=false or replicaCount=0, do not include it */}}
      {{- if and (hasKey $section "enabled") -}}
        {{- if index $section "enabled" -}}
          {{/* enabled=true */}}
          {{- $affectedSections = append $affectedSections (printf "%s.resources" .) -}}
          {{- $printMessage = true -}}
        {{- end -}}
      {{- else if and (hasKey $section "replicaCount")  -}}
        {{/* We need a casting to int because number 0 is not treated as an int by default */}}
        {{- if (gt (index $section "replicaCount" | int) 0) -}}
          {{/* replicaCount > 0 */}}
          {{- $affectedSections = append $affectedSections (printf "%s.resources" .) -}}
          {{- $printMessage = true -}}
        {{- end -}}
      {{- else -}}
        {{/* Default case, add it to the affected sections */}}
        {{- $affectedSections = append $affectedSections (printf "%s.resources" .) -}}
        {{- $printMessage = true -}}
      {{- end -}}
    {{- end -}}
  {{- end -}}
{{- end -}}
{{- if $printMessage }}

WARNING: There are "resources" sections in the chart not set. Using "resourcesPreset" is not recommended for production. For production installations, please set the following values according to your workload needs:
{{- range $affectedSections }}
  - {{ . }}
{{- end }}
+info https://kubernetes.io/docs/concepts/configuration/manage-resources-containers/
{{- end -}}
{{- end -}}