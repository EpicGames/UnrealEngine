{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Auxiliary function to get the right value for existingSecret.

Usage:
{{ include "common.cassandra.values.existingSecret" (dict "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether Cassandra is used as subchart or not. Default: false
*/}}
{{- define "common.cassandra.values.existingSecret" -}}
  {{- if .subchart -}}
    {{- .context.Values.cassandra.dbUser.existingSecret | quote -}}
  {{- else -}}
    {{- .context.Values.dbUser.existingSecret | quote -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for enabled cassandra.

Usage:
{{ include "common.cassandra.values.enabled" (dict "context" $) }}
*/}}
{{- define "common.cassandra.values.enabled" -}}
  {{- if .subchart -}}
    {{- printf "%v" .context.Values.cassandra.enabled -}}
  {{- else -}}
    {{- printf "%v" (not .context.Values.enabled) -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for the key dbUser

Usage:
{{ include "common.cassandra.values.key.dbUser" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether Cassandra is used as subchart or not. Default: false
*/}}
{{- define "common.cassandra.values.key.dbUser" -}}
  {{- if .subchart -}}
    cassandra.dbUser
  {{- else -}}
    dbUser
  {{- end -}}
{{- end -}}