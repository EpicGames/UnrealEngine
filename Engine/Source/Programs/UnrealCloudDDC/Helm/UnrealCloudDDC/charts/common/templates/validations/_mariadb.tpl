{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Validate MariaDB required passwords are not empty.

Usage:
{{ include "common.validations.values.mariadb.passwords" (dict "secret" "secretName" "subchart" false "context" $) }}
Params:
  - secret - String - Required. Name of the secret where MariaDB values are stored, e.g: "mysql-passwords-secret"
  - subchart - Boolean - Optional. Whether MariaDB is used as subchart or not. Default: false
*/}}
{{- define "common.validations.values.mariadb.passwords" -}}
  {{- $existingSecret := include "common.mariadb.values.auth.existingSecret" . -}}
  {{- $enabled := include "common.mariadb.values.enabled" . -}}
  {{- $architecture := include "common.mariadb.values.architecture" . -}}
  {{- $authPrefix := include "common.mariadb.values.key.auth" . -}}
  {{- $valueKeyRootPassword := printf "%s.rootPassword" $authPrefix -}}
  {{- $valueKeyUsername := printf "%s.username" $authPrefix -}}
  {{- $valueKeyPassword := printf "%s.password" $authPrefix -}}
  {{- $valueKeyReplicationPassword := printf "%s.replicationPassword" $authPrefix -}}

  {{- if and (or (not $existingSecret) (eq $existingSecret "\"\"")) (eq $enabled "true") -}}
    {{- $requiredPasswords := list -}}

    {{- $requiredRootPassword := dict "valueKey" $valueKeyRootPassword "secret" .secret "field" "mariadb-root-password" -}}
    {{- $requiredPasswords = append $requiredPasswords $requiredRootPassword -}}

    {{- $valueUsername := include "common.utils.getValueFromKey" (dict "key" $valueKeyUsername "context" .context) }}
    {{- if not (empty $valueUsername) -}}
        {{- $requiredPassword := dict "valueKey" $valueKeyPassword "secret" .secret "field" "mariadb-password" -}}
        {{- $requiredPasswords = append $requiredPasswords $requiredPassword -}}
    {{- end -}}

    {{- if (eq $architecture "replication") -}}
        {{- $requiredReplicationPassword := dict "valueKey" $valueKeyReplicationPassword "secret" .secret "field" "mariadb-replication-password" -}}
        {{- $requiredPasswords = append $requiredPasswords $requiredReplicationPassword -}}
    {{- end -}}

    {{- include "common.validations.values.multiple.empty" (dict "required" $requiredPasswords "context" .context) -}}

  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for existingSecret.

Usage:
{{ include "common.mariadb.values.auth.existingSecret" (dict "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether MariaDB is used as subchart or not. Default: false
*/}}
{{- define "common.mariadb.values.auth.existingSecret" -}}
  {{- if .subchart -}}
    {{- .context.Values.mariadb.auth.existingSecret | quote -}}
  {{- else -}}
    {{- .context.Values.auth.existingSecret | quote -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for enabled mariadb.

Usage:
{{ include "common.mariadb.values.enabled" (dict "context" $) }}
*/}}
{{- define "common.mariadb.values.enabled" -}}
  {{- if .subchart -}}
    {{- printf "%v" .context.Values.mariadb.enabled -}}
  {{- else -}}
    {{- printf "%v" (not .context.Values.enabled) -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for architecture

Usage:
{{ include "common.mariadb.values.architecture" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether MariaDB is used as subchart or not. Default: false
*/}}
{{- define "common.mariadb.values.architecture" -}}
  {{- if .subchart -}}
    {{- .context.Values.mariadb.architecture -}}
  {{- else -}}
    {{- .context.Values.architecture -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for the key auth

Usage:
{{ include "common.mariadb.values.key.auth" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether MariaDB is used as subchart or not. Default: false
*/}}
{{- define "common.mariadb.values.key.auth" -}}
  {{- if .subchart -}}
    mariadb.auth
  {{- else -}}
    auth
  {{- end -}}
{{- end -}}