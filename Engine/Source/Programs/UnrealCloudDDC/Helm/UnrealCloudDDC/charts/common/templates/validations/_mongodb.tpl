{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Auxiliary function to get the right value for existingSecret.

Usage:
{{ include "common.mongodb.values.auth.existingSecret" (dict "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether MongoDb is used as subchart or not. Default: false
*/}}
{{- define "common.mongodb.values.auth.existingSecret" -}}
  {{- if .subchart -}}
    {{- .context.Values.mongodb.auth.existingSecret | quote -}}
  {{- else -}}
    {{- .context.Values.auth.existingSecret | quote -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for enabled mongodb.

Usage:
{{ include "common.mongodb.values.enabled" (dict "context" $) }}
*/}}
{{- define "common.mongodb.values.enabled" -}}
  {{- if .subchart -}}
    {{- printf "%v" .context.Values.mongodb.enabled -}}
  {{- else -}}
    {{- printf "%v" (not .context.Values.enabled) -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for the key auth

Usage:
{{ include "common.mongodb.values.key.auth" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether MongoDB&reg; is used as subchart or not. Default: false
*/}}
{{- define "common.mongodb.values.key.auth" -}}
  {{- if .subchart -}}
    mongodb.auth
  {{- else -}}
    auth
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for architecture

Usage:
{{ include "common.mongodb.values.architecture" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether MongoDB&reg; is used as subchart or not. Default: false
*/}}
{{- define "common.mongodb.values.architecture" -}}
  {{- if .subchart -}}
    {{- .context.Values.mongodb.architecture -}}
  {{- else -}}
    {{- .context.Values.architecture -}}
  {{- end -}}
{{- end -}}