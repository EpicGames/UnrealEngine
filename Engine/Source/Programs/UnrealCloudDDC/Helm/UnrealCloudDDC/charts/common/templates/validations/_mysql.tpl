{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Auxiliary function to get the right value for existingSecret.

Usage:
{{ include "common.mysql.values.auth.existingSecret" (dict "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether MySQL is used as subchart or not. Default: false
*/}}
{{- define "common.mysql.values.auth.existingSecret" -}}
  {{- if .subchart -}}
    {{- .context.Values.mysql.auth.existingSecret | quote -}}
  {{- else -}}
    {{- .context.Values.auth.existingSecret | quote -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for enabled mysql.

Usage:
{{ include "common.mysql.values.enabled" (dict "context" $) }}
*/}}
{{- define "common.mysql.values.enabled" -}}
  {{- if .subchart -}}
    {{- printf "%v" .context.Values.mysql.enabled -}}
  {{- else -}}
    {{- printf "%v" (not .context.Values.enabled) -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for architecture

Usage:
{{ include "common.mysql.values.architecture" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether MySQL is used as subchart or not. Default: false
*/}}
{{- define "common.mysql.values.architecture" -}}
  {{- if .subchart -}}
    {{- .context.Values.mysql.architecture -}}
  {{- else -}}
    {{- .context.Values.architecture -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for the key auth

Usage:
{{ include "common.mysql.values.key.auth" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether MySQL is used as subchart or not. Default: false
*/}}
{{- define "common.mysql.values.key.auth" -}}
  {{- if .subchart -}}
    mysql.auth
  {{- else -}}
    auth
  {{- end -}}
{{- end -}}