{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Auxiliary function to decide whether evaluate global values.

Usage:
{{ include "common.postgresql.values.use.global" (dict "key" "key-of-global" "context" $) }}
Params:
  - key - String - Required. Field to be evaluated within global, e.g: "existingSecret"
*/}}
{{- define "common.postgresql.values.use.global" -}}
  {{- if .context.Values.global -}}
    {{- if .context.Values.global.postgresql -}}
      {{- index .context.Values.global.postgresql .key | quote -}}
    {{- end -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for existingSecret.

Usage:
{{ include "common.postgresql.values.existingSecret" (dict "context" $) }}
*/}}
{{- define "common.postgresql.values.existingSecret" -}}
  {{- $globalValue := include "common.postgresql.values.use.global" (dict "key" "existingSecret" "context" .context) -}}

  {{- if .subchart -}}
    {{- default (.context.Values.postgresql.existingSecret | quote) $globalValue -}}
  {{- else -}}
    {{- default (.context.Values.existingSecret | quote) $globalValue -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for enabled postgresql.

Usage:
{{ include "common.postgresql.values.enabled" (dict "context" $) }}
*/}}
{{- define "common.postgresql.values.enabled" -}}
  {{- if .subchart -}}
    {{- printf "%v" .context.Values.postgresql.enabled -}}
  {{- else -}}
    {{- printf "%v" (not .context.Values.enabled) -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for the key postgressPassword.

Usage:
{{ include "common.postgresql.values.key.postgressPassword" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether postgresql is used as subchart or not. Default: false
*/}}
{{- define "common.postgresql.values.key.postgressPassword" -}}
  {{- $globalValue := include "common.postgresql.values.use.global" (dict "key" "postgresqlUsername" "context" .context) -}}

  {{- if not $globalValue -}}
    {{- if .subchart -}}
      postgresql.postgresqlPassword
    {{- else -}}
      postgresqlPassword
    {{- end -}}
  {{- else -}}
    global.postgresql.postgresqlPassword
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for enabled.replication.

Usage:
{{ include "common.postgresql.values.enabled.replication" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether postgresql is used as subchart or not. Default: false
*/}}
{{- define "common.postgresql.values.enabled.replication" -}}
  {{- if .subchart -}}
    {{- printf "%v" .context.Values.postgresql.replication.enabled -}}
  {{- else -}}
    {{- printf "%v" .context.Values.replication.enabled -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right value for the key replication.password.

Usage:
{{ include "common.postgresql.values.key.replicationPassword" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether postgresql is used as subchart or not. Default: false
*/}}
{{- define "common.postgresql.values.key.replicationPassword" -}}
  {{- if .subchart -}}
    postgresql.replication.password
  {{- else -}}
    replication.password
  {{- end -}}
{{- end -}}