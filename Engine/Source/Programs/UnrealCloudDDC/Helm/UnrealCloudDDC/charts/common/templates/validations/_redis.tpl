{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}


{{/* vim: set filetype=mustache: */}}
{{/*
Auxiliary function to get the right value for enabled redis.

Usage:
{{ include "common.redis.values.enabled" (dict "context" $) }}
*/}}
{{- define "common.redis.values.enabled" -}}
  {{- if .subchart -}}
    {{- printf "%v" .context.Values.redis.enabled -}}
  {{- else -}}
    {{- printf "%v" (not .context.Values.enabled) -}}
  {{- end -}}
{{- end -}}

{{/*
Auxiliary function to get the right prefix path for the values

Usage:
{{ include "common.redis.values.key.prefix" (dict "subchart" "true" "context" $) }}
Params:
  - subchart - Boolean - Optional. Whether redis is used as subchart or not. Default: false
*/}}
{{- define "common.redis.values.keys.prefix" -}}
  {{- if .subchart -}}redis.{{- else -}}{{- end -}}
{{- end -}}

{{/*
Checks whether the redis chart's includes the standarizations (version >= 14)

Usage:
{{ include "common.redis.values.standarized.version" (dict "context" $) }}
*/}}
{{- define "common.redis.values.standarized.version" -}}

  {{- $standarizedAuth := printf "%s%s" (include "common.redis.values.keys.prefix" .) "auth" -}}
  {{- $standarizedAuthValues := include "common.utils.getValueFromKey" (dict "key" $standarizedAuth "context" .context) }}

  {{- if $standarizedAuthValues -}}
    {{- true -}}
  {{- end -}}
{{- end -}}