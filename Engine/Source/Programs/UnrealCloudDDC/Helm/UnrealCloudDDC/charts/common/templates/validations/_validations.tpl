{{/*
Copyright Broadcom, Inc. All Rights Reserved.
SPDX-License-Identifier: APACHE-2.0
*/}}

{{/* vim: set filetype=mustache: */}}
{{/*
Validate values must not be empty.

Usage:
{{- $validateValueConf00 := (dict "valueKey" "path.to.value" "secret" "secretName" "field" "password-00") -}}
{{- $validateValueConf01 := (dict "valueKey" "path.to.value" "secret" "secretName" "field" "password-01") -}}
{{ include "common.validations.values.empty" (dict "required" (list $validateValueConf00 $validateValueConf01) "context" $) }}

Validate value params:
  - valueKey - String - Required. The path to the validating value in the values.yaml, e.g: "mysql.password"
  - secret - String - Optional. Name of the secret where the validating value is generated/stored, e.g: "mysql-passwords-secret"
  - field - String - Optional. Name of the field in the secret data, e.g: "mysql-password"
*/}}
{{- define "common.validations.values.multiple.empty" -}}
  {{- range .required -}}
    {{- include "common.validations.values.single.empty" (dict "valueKey" .valueKey "secret" .secret "field" .field "context" $.context) -}}
  {{- end -}}
{{- end -}}

{{/*
Validate a value must not be empty.

Usage:
{{ include "common.validations.value.empty" (dict "valueKey" "mariadb.password" "secret" "secretName" "field" "my-password" "subchart" "subchart" "context" $) }}

Validate value params:
  - valueKey - String - Required. The path to the validating value in the values.yaml, e.g: "mysql.password"
  - secret - String - Optional. Name of the secret where the validating value is generated/stored, e.g: "mysql-passwords-secret"
  - field - String - Optional. Name of the field in the secret data, e.g: "mysql-password"
  - subchart - String - Optional - Name of the subchart that the validated password is part of.
*/}}
{{- define "common.validations.values.single.empty" -}}
  {{- $value := include "common.utils.getValueFromKey" (dict "key" .valueKey "context" .context) }}
  {{- $subchart := ternary "" (printf "%s." .subchart) (empty .subchart) }}

  {{- if not $value -}}
    {{- $varname := "my-value" -}}
    {{- $getCurrentValue := "" -}}
    {{- if and .secret .field -}}
      {{- $varname = include "common.utils.fieldToEnvVar" . -}}
      {{- $getCurrentValue = printf " To get the current value:\n\n        %s\n" (include "common.utils.secret.getvalue" .) -}}
    {{- end -}}
    {{- printf "\n    '%s' must not be empty, please add '--set %s%s=$%s' to the command.%s" .valueKey $subchart .valueKey $varname $getCurrentValue -}}
  {{- end -}}
{{- end -}}