# Copyright Broadcom, Inc. All Rights Reserved.
# SPDX-License-Identifier: APACHE-2.0

## bitnami/common
## It is required by CI/CD tools and processes.
## @skip exampleValue
##
exampleValue: common-chart