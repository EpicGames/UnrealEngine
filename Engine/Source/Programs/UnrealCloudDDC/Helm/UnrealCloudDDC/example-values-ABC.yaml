# this file contains the setting specific to Region ABC - we recommend using the same region names as your cloud provider does
config:
  Scylla: &scylla-connection-info
    LocalKeyspaceReplicationStrategy:
      class : "NetworkTopologyStrategy"
      region-A : 2
      region-B: 0
    LocalDatacenterName: "region-ABC"
    LocalKeyspaceSuffix: "abc"
  S3: &s3-connection-info
    BucketName: ddc-storage-abc

worker:
  config:
    GC:
      CleanOldRefRecords: true # this should be enabled in one region - it will delete old ref records no longer in use across the entire system.

    Scylla: *scylla-connection-info
    S3: *s3-connection-info
    
    # A section like this can be used to define a replicator that replicates blobs from region DEF to ABC (current file)
    # Replication:
    #  Enabled: true
    #  Replicators: 
    #  - ReplicatorName: DEF-to-ABC-test-namespace
    #    Namespace: test-namespace
    #    ConnectionString: http://url-to-region-DEF.com

global:
  # AWS specific configuration
  awsRegion: "us-east-1"
  
  siteName: "ABC"