# This is the base settings of your deployment, most things should go into here except for anything that is unique to a single region
config:
  Scylla: &scylla-connection-info
    ConnectionString: Contact Points=your-scylla-dns.your-domain.com
  # if you have a locally attached filesystem for caching (we recommend you do) then this configures it
  #Filesystem:
  #  MaxSizeBytes: 800000000000 # 800 Gb - the amount of storage we can use on the storage
  #  PerNamespaceGC: true - set this to do GC per namespace instead, at which point the MaxSizeBytes above sets the default value allowed per namespace instead of the total each namespace policy can override this by setting MaxFilesystemStorageBytes

# This enables a worker deployment for your scheduled tasks - we recommend running with this to keep the api more responsive
worker:
  enabled: true

  # Any configuration specified in the api deployment that also applies to workers needs to be duplicated that is not part of the global scope
  config:
    Scylla: *scylla-connection-info
    GC:
      CleanOldRefRecords: false # this should be enabled in one region - it will delete old ref records no longer in use across the entire system.
      CleanOldBlobs: true # this will delete old blobs that are no longer referenced by any ref - this runs in each region to cleanup that regions blob stores

global:
  # AWS specific configuration
  # cloudProvider: "AWS"
  # awsRegion: "us-east-1"
  # awsRole: AssumeRoleWebIdentity # service accounts are IAM mapped
  

  # Example auth configurations
  auth:
    enabled: false # disabled authentication - see section below for a example of a okta setup
  
  # This is a example using Okta for authentication
  #auth:
  #  enabled: true
  #  defaultScheme: Bearer # this should match the name under schemes
  #  schemes:
  #    Bearer: 
  #      implementation: "Okta"
  #      oktaDomain: https://your-okta-server.okta.com
  #      oktaAuthorizationServerId: abc123 # this will be given to you by your okta admin
  #      jwtAudience: "api://unreal" # the audience defined for the authorization server 
  #      jwtAuthority: "https://your-okta-server.okta.com/oauth2/abc123" # the full path to your authorization server - is usually <domain>/oauth2/<auth-server>

    policies:
      # This is an example of how to setup a claim to grant special privileges
      # In this case we expect a group called app-ddc-storage-admin to indicate admins which are granted full access to everything (all namespaces and buckets)
    - claims: 
      - groups=app-ddc-storage-admin
      actions:
      - ReadObject
      - WriteObject
      - DeleteObject
      - DeleteBucket
      - DeleteNamespace
      - AdminAction
      scope:
        namespaces:
        - values: 
          - "*"
        buckets:
        - values:
          - "*"
      
      # This grants access to the ue.ddc namespace (for normal read and write oeprations) for anyone with the app-ddc-storage-user group
    - claims:
      - groups=app-ddc-storage-user
      actions:
      - ReadObject
      - WriteObject
      scope:
        namespaces:
        - values: 
          - ue.ddc
        buckets:
        - values: ["*"]

  
  # Specify credentials used by UnrealCloudDDC to connect to other instances
  #ServiceCredentials:
  #  OAuthClientId: <path-to-arn>
  #  OAuthClientSecret: <path-to-arn>
  #  OAuthLoginUrl: "https://your-okta-server.okta.com/oauth2/abc123/v1/token"
  
  # Namespaces can have certain settings overriden in their policy, note that the acl fields are legacy and have been replaced by the AclPolicies as described above
  #namespaces:
  #  Policies:
      # This is an example of a namespace used for oplog storage
      #ue.oplog:
      #  storagePool: "oplog"
      #  UseBlobIndexForExists: true # Enabling blob index usage witin oplogs as a experiment
      #  GcMethod: TTL # use TTL (time to live) GC instead of last access tracking
      #  OnDemandReplication: true # replicate blobs on demand