Thank you for installing {{ .Chart.Name }}.

Your release is named {{ .Release.Name }}.

To learn more about the release, try:

  $ helm status {{ .Release.Name }}
  $ helm get {{ .Release.Name }}

To verify that your setup works you can run the following commands, note that they assume you do not have authenication enabled and that you have a test-namespace namespace
To put new content:
  $ kubectl exec <pod-name> -- curl http://localhost/api/v1/refs/test-namespace/default/00000000000000000000000000000000000000aa -X PUT --data 'test' -H 'content-type: application/octet-stream' -H 'X-Jupiter-IoHash: 4878CA0425C739FA427F7EDA20FE845F6B2E46BA' -i

To fetch content that you have uploaded:
  $ kubectl exec <pod-name> -- curl http://localhost/api/v1/refs/test-namespace/default/00000000000000000000000000000000000000aa.raw -i
