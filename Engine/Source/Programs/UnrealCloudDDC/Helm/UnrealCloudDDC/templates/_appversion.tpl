{{- define "jupiter.appversion" -}}
{{- $versionFile := .Files.Get "version.yaml" | fromYaml -}}
{{- $globals := default nil .Values.global -}}
{{- $globalOverride := default nil $globals.OverrideAppVersion -}}
{{- $versionFileVersion := $versionFile.version -}}
{{- $override := default $versionFileVersion $globalOverride -}}
{{- default .Chart.AppVersion $override -}}
{{- end -}}