{{- define "jupiter.applicationSettings" -}}
{{- $config := .Values.config | default dict -}}
{{- $auth := .Values.auth | default dict -}}

Jupiter:
  HostSwaggerDocumentation: {{ $config.HostSwaggerDocumentation }}
  CurrentSite: {{ required "A site has to be specified" .Values.global.siteName }}

{{- if .Values.global.auth }}
Auth:
{{ .Values.global.auth | toYaml | nindent 2 }}
{{- end -}}

{{- if .Values.global.serviceAccounts }}
ServiceAccounts:
{{ .Values.global.serviceAccounts | toYaml | nindent 2 }}
{{- end }}

{{ if .Values.global.ServiceCredentials }}
ServiceCredentials:
  SchemeName: {{ default "Bearer" .Values.global.ServiceCredentials.SchemeName }}
{{ if .Values.global.ServiceCredentials.AccessToken }}
  AccessToken: {{ .Values.global.ServiceCredentials.AccessToken }}
{{ end }}
{{ if .Values.global.ServiceCredentials.OAuthLoginUrl }}
  OAuthClientId: {{ required "OAuthClientId must be specified when using OAuth logins" .Values.global.ServiceCredentials.OAuthClientId }}
  OAuthClientSecret: {{ required "OAuthClientSecret must be specified when using OAuth logins" .Values.global.ServiceCredentials.OAuthClientSecret }}
  OAuthLoginUrl: {{ .Values.global.ServiceCredentials.OAuthLoginUrl }}
  OAuthScope: {{ default "cache_access" .Values.global.ServiceCredentials.OAuthScope }}
{{ end }}
{{ end }}

{{- if .Values.global.namespaces }}
Namespaces:
{{ .Values.global.namespaces | toYaml | nindent 2 }}
{{- end }}

{{- if .Values.global.cluster }}
Cluster:
{{ .Values.global.cluster | toYaml | nindent 2 }}
{{- end }}

{{- include "jupiter.aws.settings" . -}}

{{ end }}

{{- define "jupiter.aws.settings" -}}
{{- if and (.Values.global.cloudProvider) (eq .Values.global.cloudProvider "AWS") }}
AWS:
  Region: {{ required "You have to specify a global with the current aws region to use (.global.awsRegion)" .Values.global.awsRegion }}

AWSCredentials:
{{- if eq .Values.global.awsRole "AssumeRole" }}
  AWSCredentialsType: AssumeRole
  AssumeRoleArn:  {{ required "You have to specify ARN to a AWS Role to assume" .Values.global.awsRoleArn}}
{{- else if eq .Values.global.awsRole "AssumeRoleWebIdentity" }}
  AWSCredentialsType: AssumeRoleWebIdentity
{{- else if eq .Values.global.awsRole "Basic" }}
  AWSCredentialsType: Basic
{{ end }}
{{ end }}
{{ end }}