{{- if .Values.dashboard.enabled -}}
{{- $config := include "dashboard-config" . | fromYaml -}}
apiVersion: v1
kind: ConfigMap
metadata:
  name: {{ include "common.names.fullname" . }}-dashboard
  labels: {{- include "common.labels.standard" . | nindent 4 }}
data:
  config.json: |
{{- $config | toJson | nindent 4}} 
{{- end -}}

{{- define "dashboard-config" -}}
{{- $config := .Values.dashboard.config | default dict -}}
UpstreamUrl: {{ required "Upstream Url must be specified" $config.UpstreamUrl }}

OidcAuthority: {{ default "" $config.OidcAuthority }}
OidcClientId: {{ default "" $config.OidcClientId }}
{{ end }}
  