{{- if .Values.prepareNvmeFilesystem.enabled -}}
{{- $drivesCount := len .Values.prepareNvmeFilesystem.drives -}}
{{- if and (not .Values.prepareNvmeFilesystem.customScript) (eq $drivesCount 0) }}{{ fail "You have to specify at least one drive path when enabling prepareNvmeFilesystem" }}{{- end -}}
apiVersion: v1
kind: ConfigMap
metadata:
  name: {{ include "common.names.fullname" . }}-init
  labels: {{- include "common.labels.standard" . | nindent 4 }}
data:
  init.sh: |    
    {{- if .Values.prepareNvmeFilesystem.customScript }}
    {{ tpl .Values.prepareNvmeFilesystem.customScript . | nindent 4 }}
    {{- else }}
    #!/bin/sh
    
    # Exit if the node has already mounted the target
    mountpoint -q /data-nvme && exit 0
    
    {{- if gt $drivesCount 1 }}
    apt-get update && apt-get -y install mdadm && mdadm --create --verbose /dev/md0 --level=0 --raid-devices={{ $drivesCount }} {{ join " " .Values.prepareNvmeFilesystem.drives }} && mkfs.ext4 -E nodiscard /dev/md0 && mount /dev/md0 /data-nvme
    {{- else }}
    mkfs.ext4 -E nodiscard {{ first .Values.prepareNvmeFilesystem.drives }} && mount {{ first .Values.prepareNvmeFilesystem.drives }} /data-nvme
    {{- end }}
    
    {{- end }}
{{- end }}
