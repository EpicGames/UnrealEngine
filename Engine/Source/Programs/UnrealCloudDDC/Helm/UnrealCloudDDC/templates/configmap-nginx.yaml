{{- if .Values.nginx.enabled }}
apiVersion: v1
kind: ConfigMap
metadata:
  name: {{ include "common.names.fullname" . }}-nginx
  labels: {{- include "common.labels.standard" . | nindent 4 }}
data:
  nginx.conf: |
    worker_processes auto;
    worker_rlimit_nofile {{ mul .Values.nginx.workerConnections 2 }};
    events {
      worker_connections {{ .Values.nginx.workerConnections }};
    }

    http {
      {{ if .Values.nginx.accessLogging -}}
      log_format jupiter '$remote_addr - $remote_user [$time_local] '
                         '"$request" $status $body_bytes_sent rt=$request_time '
                         '"$http_referer" "$http_user_agent"';
      access_log /var/log/nginx/access.log jupiter;
      {{- else -}}
      access_log off;
      {{- end }}
      client_max_body_size 0;
      server_tokens off;
      sendfile on;
      tcp_nopush on;

      # increases the number of requests allowed per keepalive connection, reduces connection churn at a cost of some memory usage
      keepalive_requests 10000;

      upstream jupiter_backend {
        zone upstreams 64K;
        server {{ if .Values.nginx.useDomainSockets -}} unix:/nginx/jupiter-http.sock {{- else -}} localhost:80 {{- end }} max_fails=1 fail_timeout=2s;
        keepalive {{ .Values.nginx.keepAliveConnections }};
      }

      # http 1.1 public port
      server {
        listen 81 backlog={{ .Values.nginx.backlog }};
        server_name public_http1;

        proxy_buffering on;
        proxy_request_buffering off;
        proxy_http_version 1.1;
        
        location /api {
          {{ if .Values.nginx.readTimeout }}
          proxy_read_timeout {{ .Values.nginx.readTimeout }};
          {{- end }}
          proxy_set_header X-Jupiter-XAccel-Supported true;
          proxy_set_header X-Jupiter-Port Public;
          proxy_set_header Connection "";
    
          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;
          
          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;
        }
        location /health {
          proxy_set_header X-Jupiter-Port Public;
          proxy_set_header Connection "";

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;

          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;
         }
        location /nginx-blobs {
          internal;
          alias {{ printf "%s/Blobs" .Values.persistence.mountPath }};
          {{ if .Values.nginx.addXAccelDebugHeader }} add_header X-Jupiter-XAccel-Used true; {{ end }}

          add_header X-Jupiter-IoHash $upstream_http_x_jupiter_iohash;
          add_header X-Jupiter-LastAccess $upstream_http_x_jupiter_lastaccess;
          add_header X-Jupiter-InlinePayloadHash $upstream_http_x_jupiter_inlinepayloadhash;
          add_header Server-Timing $upstream_http_server_timing;
        }

        location / {
            return 404;
        }

        error_page 404 502 504 @error_page;
        location @error_page {
          default_type "application/json";
          return 200 '{"error_code": $status}'; # return_code is ignored
        }
      }

      # http 1.1 corp port
      server {
        listen 8009 backlog={{ .Values.nginx.backlog }};
        server_name corp_http1;
        
        proxy_buffering on;
        proxy_request_buffering off;
        proxy_http_version 1.1;
        
        location /api {
          {{ if .Values.nginx.readTimeout }}
          proxy_read_timeout {{ .Values.nginx.readTimeout }};
          {{- end }}
          proxy_set_header X-Jupiter-XAccel-Supported true;
          proxy_set_header X-Jupiter-Port Corp;
          proxy_set_header Connection "";

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;

          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;  
        }
        location /health {
          proxy_set_header X-Jupiter-Port Corp;
          proxy_set_header Connection "";

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;

          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;
        }
        location /nginx-blobs {
          internal;
          alias {{ printf "%s/Blobs" .Values.persistence.mountPath }};
          {{ if .Values.nginx.addXAccelDebugHeader }} add_header X-Jupiter-XAccel-Used true; {{ end }}

          add_header X-Jupiter-IoHash $upstream_http_x_jupiter_iohash;
          add_header X-Jupiter-LastAccess $upstream_http_x_jupiter_lastaccess;
          add_header X-Jupiter-InlinePayloadHash $upstream_http_x_jupiter_inlinepayloadhash;
          add_header Server-Timing $upstream_http_server_timing;
        }

        location / {
            return 404;
        }

        error_page 404 502 504 @error_page;
        location @error_page {
          default_type "application/json";
          return 200 '{"error_code": $status}'; # return_code is ignored
        }
      }

      # public http2 port
      server {
        listen 8091 backlog={{ .Values.nginx.backlog }};
        server_name public_http2;
        http2 on;

        proxy_buffering on;
        proxy_request_buffering off;
        proxy_http_version 1.1;
        
        location /api {
          {{ if .Values.nginx.readTimeout }}
          proxy_read_timeout {{ .Values.nginx.readTimeout }};
          {{- end }}
          proxy_set_header X-Jupiter-XAccel-Supported true;
          proxy_set_header X-Jupiter-Port Public;
          proxy_set_header Connection "";

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;

          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;
        }
        location /health {
          proxy_set_header X-Jupiter-Port Public;
          proxy_set_header Connection "";

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;

          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;
        }
        location /nginx-blobs {
          internal;
          alias {{ printf "%s/Blobs" .Values.persistence.mountPath }};
          {{ if .Values.nginx.addXAccelDebugHeader }} add_header X-Jupiter-XAccel-Used true; {{ end }}

          add_header X-Jupiter-IoHash $upstream_http_x_jupiter_iohash;
          add_header X-Jupiter-LastAccess $upstream_http_x_jupiter_lastaccess;
          add_header X-Jupiter-InlinePayloadHash $upstream_http_x_jupiter_inlinepayloadhash;
          add_header Server-Timing $upstream_http_server_timing;
        }

        location / {
            return 404;
        }

        error_page 404 502 504 @error_page;
        location @error_page {
          default_type "application/json";
          return 200 '{"error_code": $status}'; # return_code is ignored
        }
      }

      # corp port http2
      server {
        listen 8092 backlog={{ .Values.nginx.backlog }};
        server_name corp_http2;
        http2 on;

        proxy_buffering on;
        proxy_request_buffering off;
        proxy_http_version 1.1;
        
        location /api {
          {{ if .Values.nginx.readTimeout }}
          proxy_read_timeout {{ .Values.nginx.readTimeout }};
          {{- end }}
          proxy_set_header X-Jupiter-XAccel-Supported true;
          proxy_set_header X-Jupiter-Port Corp;
          proxy_set_header Connection "";

          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;

          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;
        }
        location /health {
          proxy_set_header X-Jupiter-Port Corp;
          proxy_set_header Connection "";
          
          # Always set the x-forwarded-for header so we can extract the proper ip of the sender for logging purposes
          proxy_set_header X-Forwarded-For $proxy_add_x_forwarded_for;

          proxy_pass http://jupiter_backend;
          proxy_next_upstream error timeout http_500;
        }
        location /nginx-blobs {
            internal;
            alias {{ printf "%s/Blobs" .Values.persistence.mountPath }};
            {{ if .Values.nginx.addXAccelDebugHeader }}add_header X-Jupiter-XAccel-Used true; {{ end }}

            add_header X-Jupiter-IoHash $upstream_http_x_jupiter_iohash;
            add_header X-Jupiter-LastAccess $upstream_http_x_jupiter_lastaccess;
            add_header X-Jupiter-InlinePayloadHash $upstream_http_x_jupiter_inlinepayloadhash;
            add_header Server-Timing $upstream_http_server_timing;
        }

        location / {
            return 404;
        }

        error_page 404 502 504 @error_page;
        location @error_page {
          default_type "application/json";
          return 200 '{"error_code": $status}'; # return_code is ignored
        }
      }

      # nginx status
      server {
        listen 18080;

        # ensures the version information can be retrieved
        server_tokens on;
        
        location /nginx_status {
          stub_status on;
        }

        location / {
          return 404;
        }
      }

      # https redirect
      server {
          listen 18090;
          return 301 https://$host$request_uri;
      }
    }  
{{ end }}
