{{- if .Values.worker.enabled -}}
{{- $baseSettings := include "jupiter.applicationSettings" . | fromYaml -}}
{{- $overrides := include "storage.worker-settings" . | fromYaml -}}
{{- $newDict := mergeOverwrite $baseSettings $overrides -}}
apiVersion: v1
kind: ConfigMap
metadata:
  name: {{ include "common.names.fullname" . }}-worker
  labels: {{- include "common.labels.standard" . | nindent 4 }}
data:
  appsettings.Local.yaml: |
{{ $newDict | toYaml | indent 4 }} 
{{- end -}}

{{- define "storage.worker-settings" -}}
{{- $serviceBaseName := .Release.Name | lower -}}
{{- $worker := .Values.worker | default dict -}}
{{- $config := $worker.config | default dict -}}
{{- $replication := $config.Replication | default dict -}}
{{- $filesystem := $config.Filesystem | default dict -}}

UnrealCloudDDC:
  # always assume you are the leader when you are the worker
  LeaderElectionImplementation: "Static"

  ReferencesDbImplementation:  {{ default "Scylla" $config.ReferencesDbImplementation }}
  ReplicationLogWriterImplementation:  {{ default "Scylla" $config.ReplicationLogWriterImplementation }}
  ContentIdStoreImplementation :  {{ default "Scylla" $config.ContentIdStoreImplementation }}
  BlobIndexImplementation :  {{ default "Scylla" $config.BlobIndexImplementation }}
  BuildStoreImplementation :  {{ default "Scylla" $config.BuildStoreImplementation }}
  
  # Disable last access tracking in the worker by default 
  # Typically operations in the worker do not bump the last accessed field so this just adds overhead
  # And you would typically not want to extend the lifetime of refs based on them being accessed by scheduled tasks
  EnableLastAccessTracking: {{ default "false" $config.EnableLastAccessTracking }}

{{- if hasKey $config "RemoveReferencesOnBlobDeletion" }}
  RemoveReferencesOnBlobDeletion: {{ $config.RemoveReferencesOnBlobDeletion }}
{{- end }}

{{ if $config.StorageImplementations }}
  StorageImplementations:
  {{- range $config.StorageImplementations -}}
  - {{ . }}
  {{- end -}}
{{ else }}
  StorageImplementations:
  {{- if or ($worker.persistence.enabled) ($worker.persistence.volume) }}
  - "FileSystem"
  {{- end -}}
  {{- if and (.Values.global.cloudProvider) (eq .Values.global.cloudProvider "Azure") }}
  - "Azure"
  {{- else if and (.Values.global.cloudProvider) (eq .Values.global.cloudProvider "AWS") }}
  - "S3"
  {{- else -}}
  {{- fail "You have to specify a cloud provider as global.cloudProvider"}}
  {{- end -}}
{{- end }}

{{- if $config.Jupiter }}
Jupiter: 
{{ $config.Jupiter | toYaml | nindent 2 }}
{{- end }}

Replication:
  Enabled: {{ default "false" $replication.Enabled }}
{{ with $replication.Replicators }}
  Replicators:
{{ range . }}
  - ReplicatorName: {{ required "ReplicatorName must be specified for a replicator" .ReplicatorName }}
    NamespaceToReplicate: {{ required "Namespace must be specified for a replicator" .Namespace }}
    MaxParallelReplications: {{ default 128  .MaxParallelReplications }}
    EnableMultipartReplications: {{ default false  .EnableMultipartReplications }}
    SkipSnapshot: {{ default true  .SkipSnapshot }}
    PageSize: {{ default 1000  .PageSize }}
    Version: {{ default "Refs"  .Version }}
    {{- if .SourceSite }}
    SourceSite: {{ .SourceSite }}
    {{- else if .ConnectionString }}
    ConnectionString: {{ .ConnectionString }}
    {{- else }}
    {{- fail "Replicator must specify a SourceSite or ConnectionString" }}
    {{- end }}
{{- end -}}
{{- end }}

{{- if or ($worker.persistence.enabled) ($worker.persistence.volume) }}
FileSystem:
  RootDir:  {{ printf "%s/Blobs" $worker.persistence.mountPath }}
  MaxSizeBytes: {{ required "MaxSizeBytes of the cache volume must be specified" $filesystem.MaxSizeBytes }}
 {{- if $filesystem.TriggerThresholdPercentage }}
  TriggerThresholdPercentage: {{ $filesystem.TriggerThresholdPercentage }}
{{- end }}
{{- if $filesystem.TargetThresholdPercentage }}
  TargetThresholdPercentage: {{ $filesystem.TargetThresholdPercentage }}
{{- end }}
{{- if $filesystem.PerNamespaceGC }}
  PerNamespaceGC: {{ $filesystem.PerNamespaceGC }}
{{- end }}
{{- end }}

{{- if $config.Scylla }}
Scylla: 
{{ $config.Scylla | toYaml | nindent 2 }}
{{- end }}

{{- if $config.Metrics }}
Metrics: 
{{ $config.Metrics | toYaml | nindent 2 }}
{{- end }}

{{- if $config.Serilog }}
Serilog: 
{{ $config.Serilog | toYaml | nindent 2 }}
{{- end }}

{{- if $config.ConsistencyCheck }}
ConsistencyCheck: 
{{ $config.ConsistencyCheck | toYaml | nindent 2 }}
{{- end }}

{{- if $config.CacheRef }}
CacheRef:
{{ $config.CacheRef | toYaml | nindent 2 }}
{{- end }}

{{- if $config.GC }}
GC: 
{{ $config.GC | toYaml | nindent 2 }}

{{- if or ($worker.persistence.enabled) ($worker.persistence.volume) }}
  RunFilesystemCleanup: true
{{- end }}
{{- end }}

{{- if $config.Snapshot }}
Snapshot: 
{{ $config.Snapshot | toYaml | nindent 2 }}
{{- end }}

{{- if $config.PayloadBuffering }}
PayloadBuffering: 
{{ $config.PayloadBuffering | toYaml | nindent 2 }}
{{- end }}

{{- if and (.Values.global.cloudProvider) (eq .Values.global.cloudProvider "Azure") }}
{{ include "cloud.settings.azure" .Values.worker }}
{{- else if and (.Values.global.cloudProvider) (eq .Values.global.cloudProvider "AWS") }}
{{ include "cloud.settings.aws" .Values.worker }}
{{- else -}}
{{- fail "You have to specify a cloud provider as global.cloudProvider"}}
{{- end }}

{{ end }}


