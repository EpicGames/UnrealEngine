{{- if .Values.enabled -}}
{{- $baseSettings := include "jupiter.applicationSettings" . | fromYaml -}}
{{- $overrides := include "storage.settings" . | fromYaml -}}
{{- $newDict := merge $overrides $baseSettings -}}
apiVersion: v1
kind: ConfigMap
metadata:
  name: {{ include "common.names.fullname" . }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
data:
  appsettings.Local.yaml: |
{{ $newDict | toYaml | indent 4 }} 
{{ end }}

{{- define "storage.settings" -}}
{{- $serviceBaseName := .Release.Name | lower -}}
{{- $config := .Values.config | default dict -}}
{{- $replication := $config.Replication | default dict -}}
{{- $filesystem := $config.Filesystem | default dict -}}
UnrealCloudDDC:
{{ if .Values.worker.enabled }}
  # if using the workers we do not attempt to participate in the leader election as a normal deployment
  LeaderElectionImplementation: "Disabled"
{{ else }}
  LeaderElectionImplementation: "Kubernetes"
{{ end }}
  ServiceDiscoveryImplementation: "Kubernetes"
  
  ReferencesDbImplementation:  {{ default "Scylla" $config.ReferencesDbImplementation }}
  ReplicationLogWriterImplementation:  {{ default "Scylla" $config.ReplicationLogWriterImplementation }}
  ContentIdStoreImplementation :  {{ default "Scylla" $config.ContentIdStoreImplementation }}
  BlobIndexImplementation :  {{ default "Scylla" $config.BlobIndexImplementation }}
  BuildStoreImplementation :  {{ default "Scylla" $config.BuildStoreImplementation }}

{{- if hasKey $config "EnableOnDemandReplication" }}
  EnableOnDemandReplication :  {{ $config.EnableOnDemandReplication }}  
{{- end }}

{{- if hasKey $config "EnableLastAccessTracking" }}
  EnableLastAccessTracking: {{ $config.EnableLastAccessTracking }}
{{- end }}

{{- if hasKey $config "EnableBucketStatsTracking" }}
  EnableBucketStatsTracking: {{ $config.EnableBucketStatsTracking }}
{{- end }}

{{- if hasKey $config "EnableForceSubmitRefBlobToBlobStore" }}
  EnableForceSubmitRefBlobToBlobStore: {{ $config.EnableForceSubmitRefBlobToBlobStore }}
{{- end }}

{{- if hasKey $config "EnableInlineSmallBlobs" }}
  EnableInlineSmallBlobs: {{ $config.EnableInlineSmallBlobs }}
{{- end }}

{{- if hasKey $config "UseLastAccessBackgroundService" }}
  UseLastAccessBackgroundService: {{ $config.UseLastAccessBackgroundService }}
{{- end }}

{{- if hasKey $config "RemoveReferencesOnBlobDeletion" }}
  RemoveReferencesOnBlobDeletion: {{ $config.RemoveReferencesOnBlobDeletion }}
{{- end }}

{{ if $config.StorageImplementations }}
  StorageImplementations:
  {{- range $config.StorageImplementations -}}
  - {{ . }}
  {{- end -}}
{{ else }}
  StorageImplementations:
  {{- if or (.Values.persistence.enabled) (.Values.persistence.volume) }}
  - "FileSystem"
  {{- end }}
  {{- if .Values.fetchBlobsFromPeerPods }}
  - "Peer"
  {{- end }}
  {{- if and (.Values.global.cloudProvider) (eq .Values.global.cloudProvider "Azure") }}
  - "Azure"
  {{- else if and (.Values.global.cloudProvider) (eq .Values.global.cloudProvider "AWS") }}
  - "S3"
  {{- else -}}
  {{- fail "You have to specify a cloud provider as global.cloudProvider"}}
  {{- end -}}
{{- end }}

{{- if or ($config.Jupiter) (.Values.nginx.useDomainSockets) }}
Jupiter: 
{{ if $config.Jupiter }}
{{ $config.Jupiter | toYaml | nindent 2 }}
{{- end }}
{{ if .Values.nginx.useDomainSockets }}
  UseDomainSockets: true
  DomainSocketsRoot: "/nginx"
  ChmodDomainSockets: true
{{- end }}
{{- end }}

{{- if $config.Debug }}
Debug: 
{{ $config.Debug | toYaml | nindent 2 }}
{{- end }}

{{ if .Values.nginx.enabled }}
Nginx:
  UseNginxRedirect: true
{{ end }}

Replication:
  Enabled: {{ default "false" $replication.Enabled }}
{{ with $replication.Replicators }}
  Replicators:
{{ range . }}
  - ReplicatorName: {{ required "ReplicatorName must be specified for a replicator" .ReplicatorName }}
    NamespaceToReplicate: {{ required "Namespace must be specified for a replicator" .Namespace }}
    MaxParallelReplications: {{ default 64  .MaxParallelReplications }}
    EnableMultipartReplications: {{ default false  .EnableMultipartReplications }}
    SkipSnapshot: {{ default true  .SkipSnapshot }}
    PageSize: {{ default 1000  .PageSize }}
    Version: {{ default "Refs"  .Version }}
    {{- if .SourceSite }}
    SourceSite: {{ .SourceSite }}
    {{- else if .ConnectionString }}
    ConnectionString: {{ .ConnectionString }}
    {{- else }}
    {{- fail "Replicator must specify a SourceSite or ConnectionString" }}
    {{- end }}
{{- end -}}
{{- end }}

{{- if $config.ConsistencyCheck }}
ConsistencyCheck: 
{{ $config.ConsistencyCheck | toYaml | nindent 2 }}
{{- end }}

{{- if $config.ReferenceResolver }}
ReferenceResolver: 
{{ $config.ReferenceResolver | toYaml | nindent 2 }}
{{- end }}

{{- if $config.PayloadBuffering }}
PayloadBuffering:
{{ $config.PayloadBuffering | toYaml | nindent 2 }}
{{- end }}

{{- if $config.ServiceAccounts }}
ServiceAccounts: 
{{ $config.ServiceAccounts | toYaml | nindent 2 }}
{{- end }}

{{- if $config.GC }}
GC: 
{{ $config.GC | toYaml | nindent 2 }}

{{- if or (.Values.persistence.enabled) (.Values.persistence.volume) }}
  RunFilesystemCleanup: true
{{- end }}
{{- end }}

{{- if or (.Values.persistence.enabled) (.Values.persistence.volume) }}
FileSystem:
  RootDir:  {{ printf "%s/Blobs" .Values.persistence.mountPath }}
  MaxSizeBytes: {{ default .Values.persistence.size $filesystem.MaxSizeBytes }}
 {{- if $filesystem.TriggerThresholdPercentage }}
  TriggerThresholdPercentage: {{ $filesystem.TriggerThresholdPercentage }}
{{- end }}
{{- if $filesystem.TargetThresholdPercentage }}
  TargetThresholdPercentage: {{ $filesystem.TargetThresholdPercentage }}
{{- end }}
{{- if $filesystem.PerNamespaceGC }}
  PerNamespaceGC: {{ $filesystem.PerNamespaceGC }}
{{- end }}
{{- end }}

{{- if $config.Scylla }}
Scylla: 
{{ $config.Scylla | toYaml | nindent 2 }}
{{- end }}

{{- if $config.Metrics }}
Metrics: 
{{ $config.Metrics | toYaml | nindent 2 }}
{{- end }}

{{- if $config.Serilog }}
Serilog: 
{{ $config.Serilog | toYaml | nindent 2 }}
{{- end }}

{{- if $config.CacheRef }}
CacheRef:
{{ $config.CacheRef | toYaml | nindent 2 }}
{{- end }}

{{- if $config.Symbols }}
Symbols:
{{ $config.Symbols | toYaml | nindent 2 }}
{{- end }}

{{- if $config.Builds }}
Builds:
{{ $config.Builds | toYaml | nindent 2 }}
{{- end }}

{{- if and (.Values.global.cloudProvider) (eq .Values.global.cloudProvider "Azure") }}
{{ include "cloud.settings.azure" .Values }}
{{- else if and (.Values.global.cloudProvider) (eq .Values.global.cloudProvider "AWS") }}
{{ include "cloud.settings.aws" .Values }}
{{- else -}}
{{- fail "You have to specify a cloud provider as global.cloudProvider"}}
{{- end }}

{{ end }}

{{- define "cloud.settings.azure" -}}
{{- $config := .config -}}
{{- $azure := default dict $config.Azure -}}
Azure:
  ConnectionString: {{ required "Blob Storage connection string has to be specified" $azure.ConnectionString }}
{{ end }}

{{- define "cloud.settings.aws" -}}
{{- $config := .config -}}
{{- $s3 := default dict $config.S3 -}}
S3:
  ConnectionString: {{ default "AWS" $s3.ConnectionString }} 
  BucketName: {{ required "The name of the S3 bucket to use has to be specified (.config.S3.BucketName)" $s3.BucketName }}
  # We assume that you are creating your buckets yourself for production deployments like this 
  CreateBucketIfMissing: {{ default  false $s3.CreateBucketIfMissing }}
  SetBucketPolicies: {{ default false $s3.SetBucketPolicies }}

  UseBlobIndexForExistsCheck: {{ default false $s3.UseBlobIndexForExistsCheck }}

  {{- if hasKey $s3 "UseArnRegion" }}
  UseArnRegion: {{ $s3.UseArnRegion }}
  {{- end }}

  {{- if $s3.SiteBucketsPerStoragePool }}
  SiteBucketsPerStoragePool: {{ $s3.SiteBucketsPerStoragePool | toYaml | nindent 4 }}
  {{- end }}

  {{- if $s3.StoragePoolBucketOverride }}
  StoragePoolBucketOverride: {{ $s3.StoragePoolBucketOverride | toYaml | nindent 4  }}
  {{- end }}

  {{- if hasKey $s3 "UseMultiPartUpload" }}
  UseMultiPartUpload: {{ $s3.UseMultiPartUpload }}
  {{- end }}

  {{- if hasKey $s3 "UseChunkEncoding" }}
  UseChunkEncoding: {{ $s3.UseChunkEncoding }}
  {{- end }}

  {{- if hasKey $s3 "MultiPartMaxMemoryBufferSize" }}
  MultiPartMaxMemoryBufferSize: {{ $s3.MultiPartMaxMemoryBufferSize }}
  {{- end }}
  
{{ end }}
