{{- if .Values.dashboard.enabled -}}
{{ $fullname := include "common.names.fullname" . }}
{{ $dashboard := .Values.dashboard }}
apiVersion: apps/v1
kind: Deployment
metadata:
  name: {{ $fullname }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
spec:
  selector:
    matchLabels: {{- include "common.labels.matchLabels" . | nindent 6 }}
  replicas: {{ $dashboard.replicaCount }}
  template:
    metadata:
      labels: {{- include "common.labels.standard" . | nindent 8 }}
      {{- if $dashboard.podLabels }}
      {{- include "common.tplvalues.render" ( dict "value" $dashboard.podLabels "context" $ ) | nindent 8 }}
      {{- end }}

      {{- if or $dashboard.podAnnotations $dashboard.restartPodOnConfigMapChange }}
      annotations:
        {{- if $dashboard.podAnnotations }}
        {{- include "common.tplvalues.render" (dict "value" $dashboard.podAnnotations "context" $) | nindent 8 }}
        {{- end }}
        {{- if $dashboard.restartPodOnConfigMapChange }}
        checksum/config: {{ include (print $.Template.BasePath "/configmap.yaml") . | sha256sum }}
        {{- end }}
      {{- end }}
    spec:
      serviceAccountName: {{ include "jupiter.serviceAccountName" . }}
      {{- with $dashboard.imagePullSecrets }}
      imagePullSecrets:
        {{- toYaml . | nindent 8 }}
      {{- end }}
      {{- if $dashboard.terminationGracePeriodSeconds }}
      terminationGracePeriodSeconds: {{ $dashboard.terminationGracePeriodSeconds }}
      {{- end }}
      {{- if $dashboard.affinity }}
      affinity: {{- include "common.tplvalues.render" (dict "value" $dashboard.affinity "context" $) | nindent 8 }}
      {{- else }}
      affinity:
        podAffinity: {{- include "common.affinities.pods" (dict "type" $dashboard.podAffinityPreset "context" $) | nindent 10 }}
        podAntiAffinity: {{- include "common.affinities.pods" (dict "type" $dashboard.podAntiAffinityPreset "context" $) | nindent 10 }}
        nodeAffinity: {{- include "common.affinities.nodes" (dict "type" $dashboard.nodeAffinityPreset.type "key" $dashboard.nodeAffinityPreset.key "values" $dashboard.nodeAffinityPreset.values) | nindent 10 }}
      {{- end }}
      {{- if $dashboard.securityContext.enabled }}
      {{ $securityContext := unset $dashboard.securityContext "enabled" }}
      securityContext: {{- include "common.tplvalues.render" (dict "value" $securityContext "context" $) | nindent 8 }}
      {{- end }}
      {{- if $dashboard.nodeSelector }}
      nodeSelector: {{- include "common.tplvalues.render" (dict "value" $dashboard.nodeSelector "context" $) | nindent 8 }}
      {{- end }}
      {{- if $dashboard.tolerations }}
      tolerations: {{- include "common.tplvalues.render" (dict "value" $dashboard.tolerations "context" $) | nindent 8 }}
      {{- end }}
      containers:
      - name: {{ include "common.names.name" . }}-dashboard
        image: {{ printf "%s:%s" $dashboard.image.repository (include "jupiter.appversion" . ) }}
        imagePullPolicy: {{ $dashboard.image.pullPolicy }}
        env:
        - name: Kubernetes__Namespace
          valueFrom:
            fieldRef:
              fieldPath: metadata.namespace
        - name: Kubernetes__PodName
          valueFrom:
            fieldRef:
              fieldPath: metadata.name
        {{- if $dashboard.env }}
        {{- include "common.tplvalues.render" ( dict "value" $dashboard.env "context" $ ) | nindent 8 }}
        {{- end }}
        ports:
        - name: http
          containerPort: 80
        {{- if $dashboard.livenessProbe.enabled }}
        livenessProbe:
          httpGet:
            path: {{ default "/index.html" $dashboard.livenessProbe.path }}
            port: http
          initialDelaySeconds: {{ $dashboard.livenessProbe.initialDelaySeconds }}
          periodSeconds: {{ $dashboard.livenessProbe.periodSeconds }}
          timeoutSeconds: {{ $dashboard.livenessProbe.timeoutSeconds }}
          successThreshold: {{ $dashboard.livenessProbe.successThreshold }}
          failureThreshold: {{ $dashboard.livenessProbe.failureThreshold }}
        {{- end }}
        {{- if $dashboard.readinessProbe.enabled }}
        readinessProbe:
          httpGet:
            path:  {{ default "/index.html" $dashboard.readinessProbe.path }}
            port: http
          initialDelaySeconds: {{ $dashboard.readinessProbe.initialDelaySeconds }}
          periodSeconds: {{ $dashboard.readinessProbe.periodSeconds }}
          timeoutSeconds: {{ $dashboard.readinessProbe.timeoutSeconds }}
          successThreshold: {{ $dashboard.readinessProbe.successThreshold }}
          failureThreshold: {{ $dashboard.readinessProbe.failureThreshold }}
        {{- end }}
        {{- if $dashboard.resources }}
        resources: {{- toYaml $dashboard.resources | nindent 12 }}
        {{- end }}
        volumeMounts:
        - name: config-volume
          mountPath: /config
      volumes:
      - name: config-volume
        configMap:
          name: {{ include "common.names.fullname" . }}-dashboard
{{- end -}}