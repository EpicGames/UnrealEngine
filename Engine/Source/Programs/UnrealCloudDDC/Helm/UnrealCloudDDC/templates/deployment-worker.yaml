{{- if .Values.worker.enabled -}}
{{ $fullname := include "common.names.fullname" . }}
{{ $worker := $.Values.worker }}
apiVersion: apps/v1
kind: Deployment
metadata:
  name: {{ $fullname }}-worker
  labels: &worker-labels
    app.kubernetes.io/name: {{ include "common.names.name" . }}-worker
    helm.sh/chart: {{ include "common.names.chart" . }}
    app.kubernetes.io/instance: {{ .Release.Name }}
    app.kubernetes.io/managed-by: {{ .Release.Service }}
spec:
  selector:
    matchLabels: 
      app.kubernetes.io/name: {{ include "common.names.name" . }}-worker
      app.kubernetes.io/instance: {{ .Release.Name }}
  replicas: {{ $worker.replicaCount }}
  strategy:
   type: Recreate
  template:
    metadata:
      labels: 
        app.kubernetes.io/name: {{ include "common.names.name" . }}-worker
        helm.sh/chart: {{ include "common.names.chart" . }}
        app.kubernetes.io/instance: {{ .Release.Name }}
        app.kubernetes.io/managed-by: {{ .Release.Service }}
      {{- if .Values.podLabels }}
      {{- include "common.tplvalues.render" ( dict "value" $worker.podLabels "context" $ ) | nindent 8 }}
      {{- end }}

      {{- if or $worker.podAnnotations .Values.restartPodOnConfigMapChange }}
      annotations:
        {{- if $worker.podAnnotations }}
        {{- include "common.tplvalues.render" (dict "value" $worker.podAnnotations "context" $) | nindent 8 }}
        {{- end }}
        {{- if .Values.restartPodOnConfigMapChange }}
        checksum/config: {{ include (print $.Template.BasePath "/configmap-worker.yaml") . | sha256sum }}
        {{- end }}
      {{- end }}
    spec:
      serviceAccountName: {{ include "jupiter.serviceAccountName" . }}
      {{- with $worker.imagePullSecrets }}
      imagePullSecrets:
        {{- toYaml . | nindent 8 }}
      {{- end }}
      {{- if $worker.affinity }}
      affinity: {{- include "common.tplvalues.render" (dict "value" .affinity "context" $) | nindent 8 }}
      {{- else }}
      affinity:
        podAffinity: {{- include "common.affinities.pods" (dict "type" $worker.podAffinityPreset "context" $) | nindent 10 }}
        podAntiAffinity: {{- include "common.affinities.pods" (dict "type" $worker.podAntiAffinityPreset "context" $) | nindent 10 }}
        nodeAffinity: {{- include "common.affinities.nodes" (dict "type" $worker.nodeAffinityPreset.type "key" $worker.nodeAffinityPreset.key "values" $worker.nodeAffinityPreset.values) | nindent 10 }}
      {{- end }}
      {{- if $worker.nodeSelector }}
      nodeSelector: {{- include "common.tplvalues.render" (dict "value" $worker.nodeSelector "context" $) | nindent 8 }}
      {{- end }}
      {{- if $worker.tolerations }}
      tolerations: {{- include "common.tplvalues.render" (dict "value" $worker.tolerations "context" $) | nindent 8 }}
      {{- end }}
      containers:
      - name: {{ include "common.names.name" . }}-worker
        image: {{ printf "%s:%s" $worker.image.repository (include "jupiter.appversion" . ) }}
        imagePullPolicy: {{ $worker.image.pullPolicy }}
        env:
        - name: Kubernetes__Namespace
          valueFrom:
            fieldRef:
              fieldPath: metadata.namespace
        {{- if $worker.useDotnetMonitorSidecar }}
        - name: DOTNET_DiagnosticPorts
          value: /tmp/dotnet-monitor.sock
        {{- end }}
        {{- if $worker.env }}
        {{- include "common.tplvalues.render" ( dict "value" $worker.env "context" $ ) | nindent 8 }}
        {{- end }}
        ports:
        - name: http
          containerPort: 80
        {{- if $worker.livenessProbe.enabled }}
        livenessProbe:
          httpGet:
            path: {{ default "health/liveness" $worker.livenessProbe.path }}
            port: http
          initialDelaySeconds: {{ $worker.livenessProbe.initialDelaySeconds }}
          periodSeconds: {{ $worker.livenessProbe.periodSeconds }}
          timeoutSeconds: {{ $worker.livenessProbe.timeoutSeconds }}
          successThreshold: {{ $worker.livenessProbe.successThreshold }}
          failureThreshold: {{ $worker.livenessProbe.failureThreshold }}
        {{- end }}
        {{- if $worker.readinessProbe.enabled }}
        readinessProbe:
          httpGet:
            path:  {{ default "health/readiness" $worker.readinessProbe.path }}
            port: http
          initialDelaySeconds: {{ $worker.readinessProbe.initialDelaySeconds }}
          periodSeconds: {{ $worker.readinessProbe.periodSeconds }}
          timeoutSeconds: {{ $worker.readinessProbe.timeoutSeconds }}
          successThreshold: {{ $worker.readinessProbe.successThreshold }}
          failureThreshold: {{ $worker.readinessProbe.failureThreshold }}
        {{- end }}
        {{- if $worker.resources }}
        resources: {{- toYaml $worker.resources | nindent 12 }}
        {{- end }}
        volumeMounts:
        - name: config-volume
          mountPath: /config
        {{- if $worker.useDotnetMonitorSidecar }}
        - name: diagnostics
          mountPath: /tmp
        {{- end }}
        {{- if and $worker.persistence.volume $worker.persistence.enabled }}
        - name: data
          mountPath: {{ .Values.persistence.mountPath }}
        {{- end }}
          {{- if .Values.extraVolumeMounts }}
          {{- include "common.tplvalues.render" ( dict "value" .Values.extraVolumeMounts "context" $) | nindent 8 }}
          {{- end }}
    {{- if $worker.useDotnetMonitorSidecar }}
      - name: sidecar
        image: mcr.microsoft.com/dotnet/monitor:8.0.5
        securityContext:
          # since we run cloud ddc as root we need to run dotnet monitor as root
          runAsNonRoot: false
          runAsUser: 0
          runAsGroup: 0
        env:
        - name: DOTNETMONITOR_DiagnosticPort__ConnectionMode
          value: Listen
        - name: DOTNETMONITOR_Storage__DefaultSharedPath
          value: /tmp
        - name: DOTNETMONITOR_Urls
          value: http://localhost:52323
        - name: DOTNETMONITOR_Metrics__Endpoints
          value: http://+:52325
        args: ["collect", "--temp-apikey"]
        volumeMounts:
          - name: diagnostics
            mountPath: /tmp
        resources:
          requests:
            memory: "32Mi"
            cpu: "50m"
          limits:
            memory: "256Mi"
            cpu: "250m"
    {{- end }}
      volumes:
      - name: config-volume
        configMap:
          name: {{ $fullname }}-worker
    {{- if $worker.useDotnetMonitorSidecar }}
      - name: diagnostics
        emptyDir: {}
    {{- end }}
      {{- if or $worker.persistence.volume $worker.persistence.enabled }}
      - name: data
      {{- if $worker.persistence.enabled }}
        persistentVolumeClaim:
          claimName: {{ if $worker.persistence.existingClaim }}{{ $worker.persistence.existingClaim }}{{- else }}{{ include "common.names.fullname" . }}{{- end }}
      {{- else if $worker.persistence.volume }}
      {{- include "common.tplvalues.render" ( dict "value" $worker.persistence.volume "context" $) | nindent 8 }}
      {{- end }}
      {{- end }}
      {{- if .Values.extraVolumes }}
      {{- include "common.tplvalues.render" ( dict "value" .Values.extraVolumes "context" $) | nindent 6 }}
      {{- end }}
{{- end -}}