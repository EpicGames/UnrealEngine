{{- if .Values.enabled -}}
{{ $fullname := include "common.names.fullname" . }}
apiVersion: apps/v1
kind: Deployment
metadata:
  name: {{ $fullname }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
spec:
  selector:
    matchLabels: {{- include "common.labels.matchLabels" . | nindent 6 }}
  replicas: {{ .Values.replicaCount }}
  template:
    metadata:
      labels: {{- include "common.labels.standard" . | nindent 8 }}
      {{- if .Values.podLabels }}
      {{- include "common.tplvalues.render" ( dict "value" .Values.podLabels "context" $ ) | nindent 8 }}
      {{- end }}

      {{- if or .Values.podAnnotations .Values.restartPodOnConfigMapChange }}
      annotations:
        {{- if .Values.podAnnotations }}
        {{- include "common.tplvalues.render" (dict "value" .Values.podAnnotations "context" $) | nindent 8 }}
        {{- end }}
        {{- if .Values.restartPodOnConfigMapChange }}
        checksum/config: {{ include (print $.Template.BasePath "/configmap.yaml") . | sha256sum }}
        {{- end }}
      {{- end }}
    spec:
      serviceAccountName: {{ include "jupiter.serviceAccountName" . }}
      {{- with .Values.imagePullSecrets }}
      imagePullSecrets:
        {{- toYaml . | nindent 8 }}
      {{- end }}
      {{- if .Values.terminationGracePeriodSeconds }}
      terminationGracePeriodSeconds: {{ .Values.terminationGracePeriodSeconds }}
      {{- end }}
      {{- if .Values.affinity }}
      affinity: {{- include "common.tplvalues.render" (dict "value" .Values.affinity "context" $) | nindent 8 }}
      {{- else }}
      affinity:
        podAffinity: {{- include "common.affinities.pods" (dict "type" .Values.podAffinityPreset "context" $) | nindent 10 }}
        podAntiAffinity: {{- include "common.affinities.pods" (dict "type" .Values.podAntiAffinityPreset "context" $) | nindent 10 }}
        nodeAffinity: {{- include "common.affinities.nodes" (dict "type" .Values.nodeAffinityPreset.type "key" .Values.nodeAffinityPreset.key "values" .Values.nodeAffinityPreset.values) | nindent 10 }}
      {{- end }}
      {{- if .Values.securityContext.enabled }}
      {{ $securityContext := unset .Values.securityContext "enabled" }}
      securityContext: {{- include "common.tplvalues.render" (dict "value" $securityContext "context" $) | nindent 8 }}
      {{- end }}
      {{- if .Values.nodeSelector }}
      nodeSelector: {{- include "common.tplvalues.render" (dict "value" .Values.nodeSelector "context" $) | nindent 8 }}
      {{- end }}
      {{- if .Values.tolerations }}
      tolerations: {{- include "common.tplvalues.render" (dict "value" .Values.tolerations "context" $) | nindent 8 }}
      {{- end }}
      {{- if or .Values.initContainers .Values.prepareNvmeFilesystem.enabled }}
      initContainers:
      {{- if .Values.initContainers }}
      {{- include "common.tplvalues.render" (dict "value" .Values.initContainers "context" $) | nindent 6 }}
      {{- end }}
      {{- if .Values.prepareNvmeFilesystem.enabled }}
      - name: nvme-mount
        image: ubuntu
        securityContext:
          privileged: true
        command: ["/bin/sh", "/init-scripts/init.sh"]
        volumeMounts:
        - mountPath: /data-nvme
          name: data
          mountPropagation: Bidirectional
        - mountPath: /init-scripts
          name: init-scripts
          readOnly: true
      {{- end }}
      {{- end }}
      containers:
      - name: {{ include "common.names.name" . }}
        image: {{ printf "%s:%s" .Values.image.repository (include "jupiter.appversion" . ) }}
        imagePullPolicy: {{ .Values.image.pullPolicy }}
        env:
        - name: Kubernetes__Namespace
          valueFrom:
            fieldRef:
              fieldPath: metadata.namespace
        - name: Kubernetes__PodName
          valueFrom:
            fieldRef:
              fieldPath: metadata.name
        {{- if .Values.useDotnetMonitorSidecar }}
        - name: DOTNET_DiagnosticPorts
          value: /tmp/dotnet-monitor.sock
        {{- end }}
        {{- if .Values.env }}
        {{- include "common.tplvalues.render" ( dict "value" .Values.env "context" $ ) | nindent 8 }}
        {{- end }}
        ports:
        - name: http
          containerPort: 80
        - name: corp-http
          containerPort: 8008
        - name: internal-http
          containerPort: 8080
        - name: http2
          containerPort: 8081
        - name: corp-http2
          containerPort: 8082
        - name: internal-http2
          containerPort: 8083
        {{- if .Values.livenessProbe.enabled }}
        livenessProbe:
          {{- if .Values.livenessProbe.useCurlHealthCheck }}
          exec:
            command: [ "sh", "-c", "curl --fail localhost:80/{{ default "health/live" .Values.livenessProbe.path }}"]
          {{- else }}
          httpGet:
            path: {{ default "health/live" .Values.livenessProbe.path }}
            port: http
          {{- end }}
          initialDelaySeconds: {{ .Values.livenessProbe.initialDelaySeconds }}
          periodSeconds: {{ .Values.livenessProbe.periodSeconds }}
          timeoutSeconds: {{ .Values.livenessProbe.timeoutSeconds }}
          successThreshold: {{ .Values.livenessProbe.successThreshold }}
          failureThreshold: {{ .Values.livenessProbe.failureThreshold }}
        {{- end }}
        {{- if .Values.readinessProbe.enabled }}
        readinessProbe:
          {{- if .Values.readinessProbe.useCurlHealthCheck }}
          exec:
            command: [ "sh", "-c", "curl --fail localhost:80/{{ default "health/ready" .Values.readinessProbe.path }}"]
          {{- else }}
          httpGet:
            path: {{ default "health/ready" .Values.readinessProbe.path }}
            port: http
          {{- end }}
          initialDelaySeconds: {{ .Values.readinessProbe.initialDelaySeconds }}
          periodSeconds: {{ .Values.readinessProbe.periodSeconds }}
          timeoutSeconds: {{ .Values.readinessProbe.timeoutSeconds }}
          successThreshold: {{ .Values.readinessProbe.successThreshold }}
          failureThreshold: {{ .Values.readinessProbe.failureThreshold }}
        {{- end }}
        {{- if .Values.preStopHookDelay }}
        lifecycle:
          preStop:
            exec:
              command:
              # Load balancers can be slow to deregister instances, this can be used to introduce a delay to provide zero downtime
              - "sleep"
              - "{{ .Values.preStopHookDelay }}"
        {{- end }}
        {{- if .Values.resources }}
        resources: {{- toYaml .Values.resources | nindent 12 }}
        {{- end }}
        volumeMounts:
        - name: config-volume
          mountPath: /config
        {{- if .Values.useDotnetMonitorSidecar }}
        - name: diagnostics
          mountPath: /tmp
        {{- end }}
    {{- if .Values.nginx.enabled }}
        - name: nginx
          mountPath: /nginx
    {{- end }}
        {{- if or .Values.persistence.volume .Values.persistence.enabled }}
        - name: data
          mountPath: {{ .Values.persistence.mountPath }}
        {{- end }}
          {{- if .Values.extraVolumeMounts }}
          {{- include "common.tplvalues.render" ( dict "value" .Values.extraVolumeMounts "context" $) | nindent 8 }}
          {{- end }}
    {{- if .Values.useDotnetMonitorSidecar }}
      - name: sidecar
        image: mcr.microsoft.com/dotnet/monitor:8.0.5
        securityContext:
          # since we run cloud ddc as root we need to run dotnet monitor as root
          runAsNonRoot: false
          runAsUser: 0
          runAsGroup: 0
        env:
        - name: DOTNETMONITOR_DiagnosticPort__ConnectionMode
          value: Listen
        - name: DOTNETMONITOR_Storage__DefaultSharedPath
          value: /tmp
        - name: DOTNETMONITOR_Urls
          value: http://localhost:52323
        - name: DOTNETMONITOR_Metrics__Endpoints
          value: http://+:52325
        args: ["collect", "--temp-apikey"]
        volumeMounts:
          - name: diagnostics
            mountPath: /tmp
        resources:
          requests:
            memory: "32Mi"
            cpu: "50m"
          limits:
            memory: "256Mi"
            cpu: "250m"
    {{- end }}
    {{- if .Values.nginx.enabled }}
      - name: nginx
        image: nginx:1.29.5
        {{- if .Values.nginx.env }}
        env:
        {{- include "common.tplvalues.render" ( dict "value" .Values.nginx.env "context" $ ) | nindent 8 }}
        {{- end }}
        ports:
        - name: nginx-http
          containerPort: 81
        - name: nginx-corp-http
          containerPort: 8009
        - name: nginx-h2
          containerPort: 8091
        - name: nginx-corp-h2
          containerPort: 8092
        - containerPort: 18080
        - name: nginx-redirect
          containerPort: 18090
        {{- if .Values.nginx.livenessProbe.enabled }}
        livenessProbe:
          httpGet:
            path: {{ default "health/live" .Values.nginx.livenessProbe.path }}
            port: nginx-http
          initialDelaySeconds: {{ .Values.nginx.livenessProbe.initialDelaySeconds }}
          periodSeconds: {{ .Values.nginx.livenessProbe.periodSeconds }}
          timeoutSeconds: {{ .Values.nginx.livenessProbe.timeoutSeconds }}
          successThreshold: {{ .Values.nginx.livenessProbe.successThreshold }}
          failureThreshold: {{ .Values.nginx.livenessProbe.failureThreshold }}
        {{- end }}
        {{- if .Values.nginx.readinessProbe.enabled }}
        readinessProbe:
          httpGet:
            path:  {{ default "health/ready" .Values.nginx.readinessProbe.path }}
            port: nginx-http
          initialDelaySeconds: {{ .Values.nginx.readinessProbe.initialDelaySeconds }}
          periodSeconds: {{ .Values.nginx.readinessProbe.periodSeconds }}
          timeoutSeconds: {{ .Values.nginx.readinessProbe.timeoutSeconds }}
          successThreshold: {{ .Values.nginx.readinessProbe.successThreshold }}
          failureThreshold: {{ .Values.nginx.readinessProbe.failureThreshold }}
        {{- end }}
        {{- if .Values.preStopHookDelay }}
        lifecycle:
          preStop:
            exec:
              command:
              # Load balancers can be slow to deregister instances, this can be used to introduce a delay to provide zero downtime
              - "sleep"
              - "{{ .Values.preStopHookDelay }}"
        {{- end }}
        {{- if .Values.nginx.resources }}
        resources: {{- toYaml .Values.nginx.resources | nindent 12 }}
        {{- end }}
        volumeMounts:
          - name: nginx
            mountPath: /nginx
          - name: nginx-config-volume
            mountPath: /etc/nginx/nginx.conf
            subPath: nginx.conf
            readOnly: true
        {{- if or .Values.persistence.volume .Values.persistence.enabled }}
          - name: data
            mountPath: {{ .Values.persistence.mountPath }}
        {{- end }}
    {{- end }}
      volumes:
      - name: config-volume
        configMap:
          name: {{ $fullname }}
    {{- if .Values.useDotnetMonitorSidecar }}
      - name: diagnostics
        emptyDir: {}
    {{- end }}
    {{- if .Values.nginx.enabled }}
      - name: nginx
        emptyDir: {}
      - name: nginx-config-volume
        configMap:
          name: {{ $fullname }}-nginx
    {{- end }}
      {{- if or .Values.persistence.enabled .Values.persistence.volume }}
      - name: data
        {{- if .Values.persistence.enabled }}
        persistentVolumeClaim:
          claimName: {{ if .Values.persistence.existingClaim }}{{ .Values.persistence.existingClaim }}{{- else }}{{ include "common.names.fullname" . }}{{- end }}
        {{- else if .Values.persistence.volume }}
        {{- include "common.tplvalues.render" ( dict "value" .Values.persistence.volume "context" $) | nindent 8 }}
        {{- end }}
      {{- end }}
      {{- if .Values.prepareNvmeFilesystem.enabled }}
      - name: init-scripts
        configMap:
          name: {{ $fullname }}-init
          defaultMode: 0755
      {{- end }}
      {{- if .Values.extraVolumes }}
      {{- include "common.tplvalues.render" ( dict "value" .Values.extraVolumes "context" $) | nindent 6 }}
      {{- end }}
{{- end -}}