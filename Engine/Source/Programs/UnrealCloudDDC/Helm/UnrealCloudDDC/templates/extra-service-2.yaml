{{- if .Values.extraService2.enabled }}
apiVersion: v1
kind: Service
metadata:
  name: {{ include "common.names.fullname" . }}-extra-2
  namespace: {{ .Release.Namespace | quote }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
  {{- if .Values.extraService2.annotations }}
  annotations: {{- include "common.tplvalues.render" ( dict "value" .Values.extraService2.annotations "context" $) | nindent 4 }}
  {{- end }}
spec:
  type: {{ .Values.extraService2.type }}
  {{- if and (eq .Values.extraService2.type "LoadBalancer") (not (empty .Values.extraService2.loadBalancerIP)) }}
  loadBalancerIP: {{ .Values.extraService2.loadBalancerIP }}
  {{- end }}
  ports:
    - name: {{ .Values.extraService2.portName }}
      port: {{ .Values.extraService2.port }}
      targetPort: {{ default "http" .Values.extraService2.targetPort }}
  {{- if .Values.extraService2.extraPort }}
  {{- toYaml .Values.extraService2.extraPort | nindent 4 }}
  {{- end }}
  selector: {{- include "common.labels.matchLabels" . | nindent 4 }}
{{- end }}