{{- if .Values.extraService3.enabled }}
apiVersion: v1
kind: Service
metadata:
  name: {{ include "common.names.fullname" . }}-extra-3
  namespace: {{ .Release.Namespace | quote }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
  {{- if .Values.extraService3.annotations }}
  annotations: {{- include "common.tplvalues.render" ( dict "value" .Values.extraService3.annotations "context" $) | nindent 4 }}
  {{- end }}
spec:
  type: {{ .Values.extraService3.type }}
  {{- if and (eq .Values.extraService3.type "LoadBalancer") (not (empty .Values.extraService3.loadBalancerIP)) }}
  loadBalancerIP: {{ .Values.extraService3.loadBalancerIP }}
  {{- end }}
  ports:
    - name: {{ .Values.extraService3.portName }}
      port: {{ .Values.extraService3.port }}
      targetPort: {{ default "http" .Values.extraService3.targetPort }}
  {{- if .Values.extraService3.extraPort }}
  {{- toYaml .Values.extraService3.extraPort | nindent 4 }}
  {{- end }}
  selector: {{- include "common.labels.matchLabels" . | nindent 4 }}
{{- end }}