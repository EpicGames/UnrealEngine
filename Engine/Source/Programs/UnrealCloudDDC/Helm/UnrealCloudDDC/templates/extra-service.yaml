{{- if .Values.extraService.enabled }}
apiVersion: v1
kind: Service
metadata:
  name: {{ include "common.names.fullname" . }}-extra
  namespace: {{ .Release.Namespace | quote }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
  {{- if .Values.extraService.annotations }}
  annotations: {{- include "common.tplvalues.render" ( dict "value" .Values.extraService.annotations "context" $) | nindent 4 }}
  {{- end }}
spec:
  type: {{ .Values.extraService.type }}
  {{- if and (eq .Values.extraService.type "LoadBalancer") (not (empty .Values.extraService.loadBalancerIP)) }}
  loadBalancerIP: {{ .Values.extraService.loadBalancerIP }}
  {{- end }}
  ports:
    - name: {{ .Values.extraService.portName }}
      port: {{ .Values.extraService.port }}
      targetPort: {{ default "http" .Values.extraService.targetPort }}
  {{- if .Values.extraService.extraPort }}
  {{- toYaml .Values.extraService.extraPort | nindent 4 }}
  {{- end }}
  selector: {{- include "common.labels.matchLabels" . | nindent 4 }}
{{- end }}