{{- if .Values.autoscaling.hpa.enabled  }}
apiVersion: {{ include "common.capabilities.hpa.apiVersion" ( dict "context" $ ) }}
kind: HorizontalPodAutoscaler
metadata:
  name: {{ include "common.names.fullname" . }}
  namespace: {{ include "common.names.namespace" . | quote }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
  {{- if or .Values.autoscaling.hpa.annotations .Values.commonAnnotations }}
  {{- $annotations := include "common.tplvalues.merge" ( dict "values" ( list .Values.autoscaling.hpa.annotations .Values.commonAnnotations ) "context" . ) }}
  annotations: {{- include "common.tplvalues.render" ( dict "value" $annotations "context" $) | nindent 4 }}
  {{- end }}
spec:
  {{- if .Values.autoscaling.hpa.behavior }}
  behavior:
    {{- toYaml .Values.autoscaling.hpa.behavior | nindent 4 }}
  {{- end }}
  scaleTargetRef:
    apiVersion: apps/v1
    kind: Deployment
    name: {{ include "common.names.fullname" . }}
  minReplicas: {{ .Values.autoscaling.hpa.minReplicas }}
  maxReplicas: {{ .Values.autoscaling.hpa.maxReplicas }}
  metrics:
  {{- if .Values.autoscaling.hpa.targetCPUUtilizationPercentage }}
    - type: ContainerResource
      containerResource:
        name: cpu
        container: {{ include "common.names.name" . }}
        target:
          averageUtilization: {{ .Values.autoscaling.hpa.targetCPUUtilizationPercentage }}
          type: Utilization
  {{- end }}
  {{- if .Values.autoscaling.hpa.targetMemoryUtilizationPercentage }}
    - type: ContainerResource
      containerResource:
        name: memory
        container: {{ include "common.names.name" . }}
        target:
          averageUtilization: {{ .Values.autoscaling.hpa.targetMemoryUtilizationPercentage }}
          type: Utilization
  {{- end }}
  {{- if .Values.autoscaling.hpa.customRules -}}
    {{- toYaml .Values.autoscaling.hpa.customRules | nindent 4}}
  {{- end -}}
{{- end }}