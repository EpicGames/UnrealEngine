{{- if .Values.dashboard.ingress.enabled }}
{{- $ingress := .Values.dashboard.ingress | default dict -}}
{{- $servicePort := default "http" .Values.dashbord.ingress.servicePort -}}
apiVersion: {{ include "common.capabilities.ingress.apiVersion" . }}
kind: Ingress
metadata:
  name: {{ include "common.names.fullname" . }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
  annotations:
    {{- range $key, $value := $ingress.annotations }}
    {{ $key }}: {{ $value | quote }}
    {{- end }}
spec:
  {{- if and $ingress.ingressClassName (include "common.ingress.supportsIngressClassname" .) }}
  ingressClassName: {{ $ingress.ingressClassName | quote }}
  {{- end }}
  rules:
    {{- if $ingress.hostname }}
    - host: {{ $ingress.hostname | quote }}
      http:
        paths:
          {{- if $ingress.extraPaths }}
          {{- toYaml $ingress.extraPaths | nindent 10 }}
          {{- end }}
          - path: {{ $ingress.path }}
            {{- if eq "true" (include "common.ingress.supportsPathType" .) }}
            pathType: {{ $ingress.pathType }}
            {{- end }}
            backend: {{- include "common.ingress.backend" (dict "serviceName" (include "common.names.fullname" .) "servicePort" $servicePort "context" $)  | nindent 14 }}
    {{- end }}
    {{- range $ingress.extraHosts }}
    - host: {{ .name }}
      http:
        paths:
          - path: {{ default "/" .path }}
            {{- if eq "true" (include "common.ingress.supportsPathType" $) }}
            pathType: {{ default "ImplementationSpecific" .pathType }}
            {{- end }}
            backend: {{- include "common.ingress.backend" (dict "serviceName" (include "common.names.fullname" $) "servicePort" $servicePort "context" $) | nindent 14 }}
    {{- end }}
  {{- if or $ingress.tls .Values.ingress.extraTls .Values.ingress.hosts }}
  tls:
    {{- if $ingress.tls }}
    - hosts:
        - {{ $ingress.hostname }}
      secretName: {{ printf "%s-tls" $ingress.hostname | trunc 63 | trimSuffix "-" }}
    {{- end }}
    {{- if $ingress.extraTls }}
    {{- toYaml $ingress.extraTls | nindent 4 }}
    {{- end }}
  {{- end }}
{{- end }}