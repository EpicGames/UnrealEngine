{{- if .Values.ingressExtra.enabled }}
apiVersion: {{ include "common.capabilities.ingress.apiVersion" . }}
kind: Ingress
metadata:
  name: {{ include "common.names.fullname" . }}-extra
  labels: {{- include "common.labels.standard" . | nindent 4 }}
  annotations:
    {{- range $key, $value := .Values.ingressExtra.annotations }}
    {{ $key }}: {{ $value | quote }}
    {{- end }}
spec:
  rules:
    {{- if .Values.ingressExtra.hostname }}
    - host: {{ .Values.ingressExtra.hostname }}
      http:
        paths:
          {{- if .Values.ingressExtra.extraPaths }}
          {{- toYaml .Values.ingressExtra.extraPaths | nindent 10 }}
          {{- end }}
          - path: {{ .Values.ingressExtra.path }}
            {{- if eq "true" (include "common.ingress.supportsPathType" .) }}
            pathType: {{ .Values.ingressExtra.pathType }}
            {{- end }}
            backend: {{- include "common.ingress.backend" (dict "serviceName" (include "common.names.fullname" .) "servicePort" "http" "context" $)  | nindent 14 }}
    {{- end }}
    {{- range .Values.ingressExtra.extraHosts }}
    - host: {{ .name }}
      http:
        paths:
          - path: {{ default "/" .path }}
            {{- if eq "true" (include "common.ingress.supportsPathType" $) }}
            pathType: {{ default "ImplementationSpecific" .pathType }}
            {{- end }}
            {{- $servicePort := default "http" .servicePort }}
            backend: {{- include "common.ingress.backend" (dict "serviceName" (include "common.names.fullname" $) "servicePort" .servicePort "context" $) | nindent 14 }}
    {{- end }}
{{- end }}