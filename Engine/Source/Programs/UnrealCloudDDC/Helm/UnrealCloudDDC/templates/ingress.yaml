{{- if .Values.ingress.enabled }}
{{- $servicePort := default "http" .Values.ingress.servicePort -}}
apiVersion: {{ include "common.capabilities.ingress.apiVersion" . }}
kind: Ingress
metadata:
  name: {{ include "common.names.fullname" . }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
  annotations:
    {{- range $key, $value := .Values.ingress.annotations }}
    {{ $key }}: {{ $value | quote }}
    {{- end }}
spec:
  {{- if and .Values.ingress.ingressClassName (include "common.ingress.supportsIngressClassname" .) }}
  ingressClassName: {{ .Values.ingress.ingressClassName | quote }}
  {{- end }}
  rules:
    {{- if .Values.ingress.hostname }}
    - host: {{ .Values.ingress.hostname | quote }}
      http:
        paths:
          {{- if .Values.ingress.extraPaths }}
          {{- toYaml .Values.ingress.extraPaths | nindent 10 }}
          {{- end }}
          - path: {{ .Values.ingress.path }}
            {{- if eq "true" (include "common.ingress.supportsPathType" .) }}
            pathType: {{ .Values.ingress.pathType }}
            {{- end }}
            backend: {{- include "common.ingress.backend" (dict "serviceName" (include "common.names.fullname" .) "servicePort" $servicePort "context" $)  | nindent 14 }}
    {{- end }}
    {{- range .Values.ingress.extraHosts }}
    - host: {{ .name }}
      http:
        paths:
          - path: {{ default "/" .path }}
            {{- if eq "true" (include "common.ingress.supportsPathType" $) }}
            pathType: {{ default "ImplementationSpecific" .pathType }}
            {{- end }}
            backend: {{- include "common.ingress.backend" (dict "serviceName" (include "common.names.fullname" $) "servicePort" $servicePort "context" $) | nindent 14 }}
    {{- end }}
  {{- if or .Values.ingress.tls .Values.ingress.extraTls .Values.ingress.hosts }}
  tls:
    {{- if .Values.ingress.tls }}
    - hosts:
        - {{ .Values.ingress.hostname }}
      secretName: {{ printf "%s-tls" .Values.ingress.hostname | trunc 63 | trimSuffix "-" }}
    {{- end }}
    {{- if .Values.ingress.extraTls }}
    {{- toYaml .Values.ingress.extraTls | nindent 4 }}
    {{- end }}
  {{- end }}
{{- end }}