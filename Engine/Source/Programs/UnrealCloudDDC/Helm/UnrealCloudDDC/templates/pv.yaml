{{- $persistence := default dict .Values.persistence -}}
{{ if and ($persistence.volume) ($persistence.enabled) }}
apiVersion: v1
kind: PersistentVolume
{{ with $persistence.volume -}}
metadata:
  name: {{ include "common.names.fullname" $ }}
spec:
  {{- if .size }}
  capacity:
    storage: {{ .size | quote }}
  {{- end }}
  volumeMode: {{ .volumeMode }}
  accessModes:
  {{- range .accessModes }}
  - {{ . | quote }}
  {{- end }}
  persistentVolumeReclaimPolicy: {{ .reclaimPolicy }}
  {{- if .csi }}
  csi: {{ toYaml .csi | nindent 4 }}
  {{- end -}}
  {{- include "common.storage.class" (dict "persistence" . "global" $.Values.global) | nindent 2 }}
{{ end }}
{{ end }}