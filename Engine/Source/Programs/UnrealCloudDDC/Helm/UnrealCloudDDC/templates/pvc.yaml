{{- if and .Values.persistence.enabled (not .Values.persistence.existingClaim) }}
{{ with .Values.persistence }}
kind: PersistentVolumeClaim
apiVersion: v1
metadata:
  name: {{ include "common.names.fullname" $ }}
  namespace: {{ $.Release.Namespace | quote }}
  labels: {{- include "common.labels.standard" $ | nindent 4 }}
  {{- if or .annotations $.Values.commonAnnotations }}
  annotations:
  {{- if .annotations }}
  {{- include "common.tplvalues.render" ( dict "value" .annotations "context" $ ) | nindent 4 }}
  {{- end }}
  {{- if $.Values.commonAnnotations }}
  {{- include "common.tplvalues.render" ( dict "value" $.Values.commonAnnotations "context" $ ) | nindent 4 }}
  {{- end }}
  {{- end }}
spec:
  accessModes:
  {{- range .accessModes }}
    - {{ . | quote }}
  {{- end }}
  resources:
    requests:
      storage: {{ .size | quote }}
  {{- include "common.storage.class" (dict "persistence" . "global" $.Values.global) | nindent 2 }}
{{- end }}
{{- end }}