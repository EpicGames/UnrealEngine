{{- if .Values.rbac.create -}}
apiVersion: rbac.authorization.k8s.io/v1
kind: Role
metadata:
  name: {{ template "common.names.fullname" . }}
  namespace: {{ .Release.Namespace | quote }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
rules:
  # when using kubernetes for leader election UnrealCloudDDC will allocate a configmap to coordinate access between replicas
  - apiGroups:
      - ""
    resources:
      - configmaps
    verbs:
      - create
      - list
      - watch
      - patch
      - get
      - delete
      - update
  # kubernetes is also used to find other pods of the same deployment
  - apiGroups:
      - ""
    resources:
      - pods
    verbs:
      - list
      - get
{{- end -}}