{{- if .Values.rbac.create -}}
apiVersion: rbac.authorization.k8s.io/v1
kind: RoleBinding
metadata:
  name: {{ template "common.names.fullname" . }}
  namespace: {{ .Release.Namespace | quote }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
roleRef:
  apiGroup: rbac.authorization.k8s.io
  kind: Role
  name: {{ template "common.names.fullname" . }}
subjects:
  - kind: ServiceAccount
    name: {{ include "jupiter.serviceAccountName" . }}
    namespace: {{ .Release.Namespace | quote }}
{{- end -}}