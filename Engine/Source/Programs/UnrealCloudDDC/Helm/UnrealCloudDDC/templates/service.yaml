{{- if .Values.enabled -}}
apiVersion: v1
kind: Service
metadata:
  name: {{ include "common.names.fullname" . }}
  namespace: {{ .Release.Namespace | quote }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
  {{- if .Values.service.annotations }}
  annotations: {{- include "common.tplvalues.render" ( dict "value" .Values.service.annotations "context" $) | nindent 4 }}
  {{- end }}
spec:
  type: {{ .Values.service.type }}
  {{- if and (eq .Values.service.type "LoadBalancer") (not (empty .Values.service.loadBalancerIP)) }}
  loadBalancerIP: {{ .Values.service.loadBalancerIP }}
  {{- end }}
  ports:
    - name: http
      port: {{ .Values.service.port }}
      targetPort: {{ default "http" .Values.service.targetPort }}
      {{- if and (or (eq .Values.service.type "NodePort") (eq .Values.service.type "LoadBalancer")) (not (empty .Values.service.nodePort)) }}
      nodePort: {{ .Values.service.nodePort }}
      {{- else if eq .Values.service.type "ClusterIP" }}
      nodePort: null
      {{- end }}
  {{- if .Values.service.extraPort }}
  {{- toYaml .Values.service.extraPort | nindent 4 }}
  {{- end }}

  selector: {{- include "common.labels.matchLabels" . | nindent 4 }}

{{ end }}
