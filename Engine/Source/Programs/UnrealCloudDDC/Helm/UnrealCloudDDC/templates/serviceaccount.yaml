{{- if .Values.serviceAccount.create }}
apiVersion: v1
kind: ServiceAccount
metadata:
  name: {{ template "jupiter.serviceAccountName" . }}
  namespace: {{ .Release.Namespace | quote }}
  labels: {{- include "common.labels.standard" . | nindent 4 }}
    {{- if .Values.commonLabels }}
    {{- include "common.tplvalues.render" ( dict "value" .Values.commonLabels "context" $ ) | nindent 4 }}
    {{- end }}
  annotations:
    {{- if .Values.serviceAccount.annotations }}
    {{- include "common.tplvalues.render" ( dict "value" .Values.serviceAccount.annotations "context" $) | nindent 4 }}
    {{- end }}
    {{- if .Values.commonAnnotations }}
    {{- include "common.tplvalues.render" ( dict "value" .Values.commonAnnotations "context" $ ) | nindent 4 }}
    {{- end }}
secrets:
  - name: {{ include "common.names.fullname" . }}
{{- end }}

{{- define "jupiter.serviceAccountName" -}}
{{- if .Values.serviceAccount.create -}}
    {{ default (include "common.names.fullname" .) .Values.serviceAccount.name }}
{{- else -}}
    {{ default "default" .Values.serviceAccount.name }}
{{- end -}}
{{- end -}}
