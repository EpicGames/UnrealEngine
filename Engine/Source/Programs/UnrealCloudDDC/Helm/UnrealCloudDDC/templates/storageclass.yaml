{{- $persistence := default dict .Values.persistence -}}
{{- $storageClasses := default list $persistence.storageClasses -}}
{{ range $storageClasses }}
---
kind: StorageClass
apiVersion: storage.k8s.io/v1
metadata:
  name: {{ .name }}
allowVolumeExpansion: true

provisioner: {{ required "Provisioner must be specified when using a specific storage class" .provisioner }}
# we retain volumes by default so that we can manually migrate data should we want to
reclaimPolicy: {{ default "Retain" .reclaimPolicy }} 

{{- with .parameters }}
parameters:
  {{- toYaml . | nindent 2 }}
{{- end }}
{{- with .mountOptions }}
parameters:
  {{- toYaml . | nindent 2 }}
{{- end -}}
{{- with .allowedTopologies }}
allowedTopologies:
  {{- toYaml . | nindent 2 }}
{{- end -}}

{{- end -}}