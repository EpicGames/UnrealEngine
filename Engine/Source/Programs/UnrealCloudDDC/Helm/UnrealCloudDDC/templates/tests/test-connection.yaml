{{ $fullname := include "common.names.fullname" . }}
apiVersion: v1
kind: Pod
metadata:
  name: {{ $fullname }}
  annotations:
    "helm.sh/hook": test-success
  labels: {{- include "common.labels.standard" . | nindent 4 }}
spec:
  containers:
    - name: wget
      image: busybox
      command: ['wget']
      args:  ['{{ $fullname }}:{{ .Values.service.port }}']
  restartPolicy: Never