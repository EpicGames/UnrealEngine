## App configuration
config:

## Flag to enable hosting swagger documentation - you will likely not want to host this on a public endpoint
##
  HostSwaggerDocumentation: false


## Number of replicas
##

replicaCount: 2

## Docker image configuration
image:
  repository: "ghcr.io/epicgames/unreal-cloud-ddc"
  pullPolicy: IfNotPresent

## Force target Kubernetes version (using Helm capabilites if not set)
##
kubeVersion:

## Enable the main workload (cloud ddc) - mostly used if you want to break the install into multiple parts
## 
enabled: true

## String to partially override common.names.fullname template (will maintain the release name)
##
# nameOverride:

## String to fully override common.names.fullname template
##
# fullnameOverride:

serviceAccount:
  # Specifies whether a ServiceAccount should be created
  create: true
  # The name of the ServiceAccount to use.
  # If not set and create is true, a name is generated using the fullname template
  name:

## Define custom environment variables to pass to the image here
##
env: []

## Security Context
## ref: https://kubernetes.io/docs/tasks/configure-pod-container/security-context/
##
securityContext: {}

## Pod annotations
## ref: https://kubernetes.io/docs/concepts/overview/working-with-objects/annotations/
##
podAnnotations: {}

## Pod affinity preset
## ref: https://kubernetes.io/docs/concepts/scheduling-eviction/assign-pod-node/#inter-pod-affinity-and-anti-affinity
## Allowed values: soft, hard
##
podAffinityPreset: ""

## Pod anti-affinity preset
## ref: https://kubernetes.io/docs/concepts/scheduling-eviction/assign-pod-node/#inter-pod-affinity-and-anti-affinity
## Allowed values: soft, hard
##
podAntiAffinityPreset: soft

## Node affinity preset
## ref: https://kubernetes.io/docs/concepts/scheduling-eviction/assign-pod-node/#node-affinity
## Allowed values: soft, hard
##
nodeAffinityPreset:
  ## Node affinity type
  ## Allowed values: soft, hard
  ##
  type: ""
  ## Node label key to match
  ## E.g.
  ## key: "kubernetes.io/e2e-az-name"
  ##
  key: ""
  ## Node label values to match
  ## E.g.
  ## values:
  ##   - e2e-az1
  ##   - e2e-az2
  ##
  values: []

## Affinity for pod assignment. Evaluated as a template.
## ref: https://kubernetes.io/docs/concepts/configuration/assign-pod-node/#affinity-and-anti-affinity
## Note: podAffinityPreset, podAntiAffinityPreset, and nodeAffinityPreset will be ignored when it's set
##
affinity: {}

## Node labels for pod assignment. Evaluated as a template.
## ref: https://kubernetes.io/docs/user-guide/node-selection/
##
nodeSelector: {}

## Tolerations for pod assignment. Evaluated as a template.
## ref: https://kubernetes.io/docs/concepts/configuration/taint-and-toleration/
##
tolerations: []

## Container resource requests and limits
## ref: http://kubernetes.io/docs/user-guide/compute-resources/
##
resources:
  # We usually recommend not to specify default resources and to leave this as a conscious
  # choice for the user. This also increases chances charts run on environments with little
  # resources, such as Minikube. If you do want to specify resources, uncomment the following
  # lines, adjust them as necessary, and remove the curly braces after 'resources:'.
  limits: {}
  #   cpu: 250m
  #   memory: 256Mi
  requests: {}
  #   cpu: 250m
  #   memory: 256Mi

## Liveness and readiness probes, defaults to corresponding endpoints
##
livenessProbe:
  enabled: true
  initialDelaySeconds: 5
  periodSeconds: 10
  timeoutSeconds: 5
  successThreshold: 1
  failureThreshold: 5
readinessProbe:
  enabled: true
  initialDelaySeconds: 5
  periodSeconds: 10
  timeoutSeconds: 3
  successThreshold: 1
  failureThreshold: 5

## Enable persistence using Persistent Volume Claims
## ref: http://kubernetes.io/docs/user-guide/persistent-volumes/
##
persistence:
  ## If true, use a Persistent Volume Claim, If false, use emptyDir
  ##
  enabled: true

  ## Enable persistence using an existing PVC
  ##
  # existingClaim:

  ## Data volume mount path
  ##
  mountPath: /data

  ## Persistent Volume Access Mode
  ##
  accessModes:
    - ReadWriteOnce

  ## Persistent Volume size
  ##
  size: 5242880 # 5 Gb specified in bytes so it can be used as a setting MaxSizeBytes

  ## Persistent Volume Storage Class
  ## If defined, storageClassName: <storageClass>
  ## If set to "-", storageClassName: "", which disables dynamic provisioning
  ## If undefined (the default) or set to null, no storageClassName spec is
  ##   set, choosing the default provisioner.  (gp2 on AWS, standard on
  ##   GKE, AWS & OpenStack)
  ##
  # storageClass: "-"

  ## Persistent Volume Claim annotations
  ##
  annotations: {}


autoscaling:
  ## Horizontal Pod Autoscaler configuration
  ## ref: https://kubernetes.io/docs/tasks/run-application/horizontal-pod-autoscale/
  ##
  hpa:
    ## Annotations for HPA resource
    ##
    annotations: {}
    ## Enable HPA
    ##
    enabled: false
    ## Min replicas
    ##
    minReplicas: 2
    ## Max replicas
    ##
    maxReplicas: 4
    ## Target CPU utilization percentage
    ##
    targetCPUUtilizationPercentage: 75
    ## Target Memory utilization percentage
    ##
    targetMemoryUtilizationPercentage: ""
    ## Custom rules
    ##
    customRules: []
    ## HPA Behavior
    ##
    behavior: {}

## Service properties
##
service:
  ## Service type
  ##
  type: NodePort

  ## Service port
  ##
  port: 80

  ## Specify the nodePort value for the LoadBalancer and NodePort service types.
  ## ref: https://kubernetes.io/docs/concepts/services-networking/service/#type-nodeport
  ##
  # nodePort:

  ## Provide any additional annotations which may be required. This can be used to
  ## set the LoadBalancer service type to internal only.
  ## ref: https://kubernetes.io/docs/concepts/services-networking/service/#internal-load-balancer
  ##
  annotations: {}

  ## loadBalancerIP for the PrestaShop Service (optional, cloud specific)
  ## ref: http://kubernetes.io/docs/user-guide/services/#type-loadbalancer
  ##
  # loadBalancerIP:

extraService:
  enabled: false

extraService2:
  enabled: false

extraService3:
  enabled: false

## Configure the ingress resource that allows you to access the pod
##
ingress:
  ## Set to true to enable ingress record generation
  ##
  enabled: false

  ## Ingress Path type
  ##
  pathType: ImplementationSpecific

  ## Override API Version (automatically detected if not set)
  ##
  apiVersion:

  ## When the ingress is enabled, a host pointing to this will be created
  ##
  # hostname: hostname.example

  ## The Path to host the ingress under. You may need to set this to '/*' in order to use this
  ## with ALB ingress controllers.
  ##
  path: /api

  ## Ingress annotations done as key:value pairs
  ##
  annotations: {}

  ## The list of additional hostnames to be covered with this ingress record.
  ## Most likely the hostname above will be enough, but in the event more hosts are needed, this is an array
  ## extraHosts:
  ## - name: hostname.example
  ##   path: /
  ##

  ## Any additional arbitrary paths that may need to be added to the ingress under the main host.
  ## For example: The ALB ingress controller requires a special rule for handling SSL redirection.
  ## extraPaths:
  ## - path: /*
  ##   backend:
  ##     name: ssl-redirect
  ##     port: 
  ##       name: use-annotation
  ##

## A secondary ingress, identical to the first ingress
##
ingressExtra:
  ## Set to true to enable ingress record generation
  ##
  enabled: false

  ## Ingress Path type
  ##
  pathType: ImplementationSpecific

  ## Override API Version (automatically detected if not set)
  ##
  apiVersion:

  ## When the ingress is enabled, a host pointing to this will be created
  ##
  # hostname: hostname.example

  ## The Path to host the ingress under. You may need to set this to '/*' in order to use this
  ## with ALB ingress controllers.
  ##
  path: /api

  ## Ingress annotations done as key:value pairs
  ##
  annotations: {}

  ## The list of additional hostnames to be covered with this ingress record.
  ## Most likely the hostname above will be enough, but in the event more hosts are needed, this is an array
  ## extraHosts:
  ## - name: hostname.example
  ##   path: /
  ##

  ## Any additional arbitrary paths that may need to be added to the ingress under the main host.
  ## For example: The ALB ingress controller requires a special rule for handling SSL redirection.
  ## extraPaths:
  ## - path: /*
  ##   backend:
  ##     name: ssl-redirect
  ##     port: 
  ##       name: use-annotation
  ##


## Role Based Access
## Ref: https://kubernetes.io/docs/admin/authorization/rbac/
##
rbac:
  ## Specifies whether RBAC rules should be created
  ##
  create: true


## Dotnet debugging
## Enable to install a sidecar container which allows you to remoteley run dotnet monitoring tools
## see https://devblogs.microsoft.com/dotnet/introducing-dotnet-monitor/
useDotnetMonitorSidecar : false

## Peer pod blobs
## Attempt to fetch blobs from other pods in the same deployment before hitting blob store
## this is typically faster then falling back to global blob store
fetchBlobsFromPeerPods: false

## restartPodOnConfigMapChange
## Adds a annotation that will include the checksum of the config map to all pods, causing them to restart if the configmap changes
## This is a bit pessimistic as config changes can be dynamically reloaded but its inconsistent
restartPodOnConfigMapChange: false

## Nginx configuration
nginx:

  ## Enable this option to run a nginx proxy in the all pods that is able to serve content directly from the filesystem
  enabled: false

  ## Set this to add a header whenever X-Accel is used from nginx to send blobs
  addXAccelDebugHeader: false

  ## Set this to use Unix domain sockets rather then ports for inter process communication
  useDomainSockets: false

  ## Set to enable access log output, can be useful for debugging but will generate a lot of logs in prod
  accessLogging: false

  ## Control the number of worker connections used for nginx
  workerConnections: 1024

  ## Number of connections to keep alive between nginx and Jupiter
  keepAliveConnections: 2048

  ## The length of the nginx backlog, bump this if you increase `net.core.somaxconn` beyond 4096
  backlog: 4096
  
  ## Proxy read timeout. Increase this if experiencing 504 errors during large operations.
  readTimeout: 60s

  ## Liveness and readiness probes, defaults to corresponding endpoints
  ##
  livenessProbe:
    enabled: true
    initialDelaySeconds: 5
    periodSeconds: 10
    timeoutSeconds: 5
    successThreshold: 1
    failureThreshold: 5
  readinessProbe:
    enabled: true
    initialDelaySeconds: 5
    periodSeconds: 10
    timeoutSeconds: 3
    successThreshold: 1
    failureThreshold: 5

## Dashboard configuration
##
dashboard:
  ## Set to enable the admin dashboard webpage
  ##
  enabled: False

  service:
    enabled: False
  
  ingress:
    ## Set to enable creating a ingress (load balancer)
    enabled: False
  ## Liveness and readiness probes, defaults to corresponding endpoints
  ##
  livenessProbe:
    enabled: true
    initialDelaySeconds: 5
    periodSeconds: 10
    timeoutSeconds: 5
    successThreshold: 1
    failureThreshold: 5
  readinessProbe:
    enabled: true
    initialDelaySeconds: 5
    periodSeconds: 10
    timeoutSeconds: 3
    successThreshold: 1
    failureThreshold: 5
  
  ## Security Context
  ## ref: https://kubernetes.io/docs/tasks/configure-pod-container/security-context/
  ##
  securityContext: {}

  ## Pod annotations
  ## ref: https://kubernetes.io/docs/concepts/overview/working-with-objects/annotations/
  ##
  podAnnotations: {}

  ## Pod affinity preset
  ## ref: https://kubernetes.io/docs/concepts/scheduling-eviction/assign-pod-node/#inter-pod-affinity-and-anti-affinity
  ## Allowed values: soft, hard
  ##
  podAffinityPreset: ""

  ## Pod anti-affinity preset
  ## ref: https://kubernetes.io/docs/concepts/scheduling-eviction/assign-pod-node/#inter-pod-affinity-and-anti-affinity
  ## Allowed values: soft, hard
  ##
  podAntiAffinityPreset: soft

  ## Node affinity preset
  ## ref: https://kubernetes.io/docs/concepts/scheduling-eviction/assign-pod-node/#node-affinity
  ## Allowed values: soft, hard
  ##
  nodeAffinityPreset:
    ## Node affinity type
    ## Allowed values: soft, hard
    ##
    type: ""
    ## Node label key to match
    ## E.g.
    ## key: "kubernetes.io/e2e-az-name"
    ##
    key: ""
    ## Node label values to match
    ## E.g.
    ## values:
    ##   - e2e-az1
    ##   - e2e-az2
    ##
    values: []

## Worker configuration
worker:

  ## Set to enable seperate deployments for background workers
  enabled: false

  ## Count of replicas to run, only 1 will be doing work at a time for tasks that rely on leader election
  replicaCount: 1

  ## Docker image configuration
  image:
    repository: "ghcr.io/epicgames/unreal-cloud-ddc"
    pullPolicy: IfNotPresent
  
  ## Disable storage by default as this is normally used for cache volumes which we do not need for the worker deployment
  persistence:
    enabled: false

  ## Liveness and readiness probes, defaults to corresponding endpoints
  ##
  livenessProbe:
    enabled: true
    initialDelaySeconds: 5
    periodSeconds: 10
    timeoutSeconds: 5
    successThreshold: 1
    failureThreshold: 5
  readinessProbe:
    enabled: true
    initialDelaySeconds: 5
    periodSeconds: 10
    timeoutSeconds: 3
    successThreshold: 1
    failureThreshold: 5
    
  ## Pod affinity preset
  ## ref: https://kubernetes.io/docs/concepts/scheduling-eviction/assign-pod-node/#inter-pod-affinity-and-anti-affinity
  ## Allowed values: soft, hard
  ##
  podAffinityPreset: ""

  ## Pod anti-affinity preset
  ## ref: https://kubernetes.io/docs/concepts/scheduling-eviction/assign-pod-node/#inter-pod-affinity-and-anti-affinity
  ## Allowed values: soft, hard
  ##
  podAntiAffinityPreset: soft

  ## Node affinity preset
  ## ref: https://kubernetes.io/docs/concepts/scheduling-eviction/assign-pod-node/#node-affinity
  ## Allowed values: soft, hard
  ##
  nodeAffinityPreset:
    ## Node affinity type
    ## Allowed values: soft, hard
    ##
    type: ""
    ## Node label key to match
    ## E.g.
    ## key: "kubernetes.io/e2e-az-name"
    ##
    key: ""
    ## Node label values to match
    ## E.g.
    ## values:
    ##   - e2e-az1
    ##   - e2e-az2
    ##
    values: []


## Prepare Nvme filesystem configuration
## Optional feature that can be enabled to run a initContainer that prepares a already attached nvme drive
## This works well with VMs with locally attached nvme drives
## If you enable it you should set the list of drives that you expect to be enabled
## if more then 1 drive is specified we raid0 these together into a single drive
prepareNvmeFilesystem:
  enabled: false
  drives: []
  ## Optional: provide a fully custom init script instead of the default mkfs/mount logic.
  ## Supports Go/Helm templates — .Values, .Release, etc. are all available.
  ## Example:
  ##   customScript: |
  ##     mkfs.ext4 -E nodiscard {{ "{{" }} first .Values.prepareNvmeFilesystem.drives {{ "}}" }} && \
  ##     mount {{ "{{" }} first .Values.prepareNvmeFilesystem.drives {{ "}}" }} /data-nvme || exit 0
  customScript: ""