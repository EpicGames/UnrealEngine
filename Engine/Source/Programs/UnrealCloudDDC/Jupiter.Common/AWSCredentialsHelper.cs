// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using Amazon.Runtime;

namespace Jupiter
{
	public static class AWSCredentialsHelper
	{
		public static AWSCredentials GetCredentials(AWSCredentialsSettings options, string sessionName)
		{
			switch (options.AWSCredentialsType)
			{
				case AWSCredentialsType.Basic:
					return new BasicAWSCredentials(options.AwsAccessKey, options.AwsSecretKey);
				case AWSCredentialsType.AssumeRole:
					return new AssumeRoleAWSCredentials(FallbackCredentialsFactory.GetCredentials(), options.AssumeRoleArn, sessionName);
				case AWSCredentialsType.AssumeRoleWebIdentity:
					return AssumeRoleWithWebIdentityCredentials.FromEnvironmentVariables();
				default:
					throw new NotImplementedException($"Unhandled enum option {options.AWSCredentialsType}");
			}
		}
	}

	public class AWSCredentialsSettings : IValidatableObject
	{
		public AWSCredentialsType AWSCredentialsType { get; set; }

		public string AssumeRoleArn { get; set; } = "";
		public string AwsSecretKey { get; set; } = "";
		public string AwsAccessKey { get; set; } = "";

		public IEnumerable<ValidationResult> Validate(ValidationContext validationContext)
		{
			List<ValidationResult> results = new();

			switch (AWSCredentialsType)
			{
				case AWSCredentialsType.Basic:
					if (string.IsNullOrEmpty(AwsSecretKey))
					{
						results.Add(new ValidationResult("AwsSecretKey is required when using basic credentials"));
					}
					if (string.IsNullOrEmpty(AwsAccessKey))
					{
						results.Add(new ValidationResult("AwsAccessKey is required when using basic credentials"));
					}

					break;
				case AWSCredentialsType.AssumeRole:
					if (string.IsNullOrEmpty(AssumeRoleArn))
					{
						results.Add(new ValidationResult("AssumeRoleArn is required when using assume role credentials"));
					}

					break;
				case AWSCredentialsType.AssumeRoleWebIdentity:
					// the environment variables will be verified when we create the credentials type, as there is no configuration in our appsettings we do not do any checks
					break;
				default:
					throw new NotImplementedException($"Unhandled enum option {AWSCredentialsType}");
			}

			return results;
		}
	}

	public enum AWSCredentialsType
	{
		Basic,
		AssumeRole,
		AssumeRoleWebIdentity
	}
}
