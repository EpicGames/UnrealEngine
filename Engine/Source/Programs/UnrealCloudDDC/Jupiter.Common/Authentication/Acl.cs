// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Security.Claims;
using System.Text.RegularExpressions;
using Microsoft.AspNetCore.Authorization;
using Microsoft.Extensions.Logging;

namespace Jupiter;

public enum JupiterAclAction
{
	/// <summary>
	/// General read access to refs / blobs and so on
	/// </summary>
	ReadObject,
	/// <summary>
	/// General write access to upload refs / blobs etc
	/// </summary>
	WriteObject,
	/// <summary>
	/// Access to delete blobs / refs etc
	/// </summary>
	DeleteObject,

	/// <summary>
	/// Access to delete a particular bucket
	/// </summary>
	DeleteBucket,
	/// <summary>
	/// Access to delete a whole namespace
	/// </summary>
	DeleteNamespace,

	/// <summary>
	/// Access to read the transaction log
	/// </summary>
	ReadTransactionLog,

	/// <summary>
	/// Access to write the transaction log
	/// </summary>
	WriteTransactionLog,

	/// <summary>
	/// Access to read build parts
	/// </summary>
	ReadBuildPart,

	/// <summary>
	/// Access to write build parts
	/// </summary>
	WriteBuildPart,

	/// <summary>
	/// Access to perform administrative task
	/// </summary>
	AdminAction,

	/// <summary>
	/// Access to enumerate all objects in a bucket
	/// </summary>
	EnumerateBucket
}

public class AclEntry
{
	/// <summary>
	/// Claims required to be present to be allowed to do the actions - if multiple claims are present *all* of them are required
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
	public List<string> Claims { get; set; } = new List<string>();

	/// <summary>
	/// The actions granted if the claims match
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
	public List<JupiterAclAction> Actions { get; set; } = new List<JupiterAclAction>();

	public IEnumerable<JupiterAclAction> Resolve(ClaimsPrincipal user)
	{
		List<JupiterAclAction> allowedActions = new List<JupiterAclAction>();

		bool allClaimsFound = true;
		// These are ANDed, e.g. all claims needs to be present
		foreach (string expectedClaim in Claims)
		{
			bool claimFound = false;
			// if expected claim is * then everyone has the associated actions
			if (expectedClaim == "*")
			{
				claimFound = true;
			}
			else if (expectedClaim.Contains('=', StringComparison.InvariantCultureIgnoreCase))
			{
				int separatorIndex = expectedClaim.IndexOf('=', StringComparison.InvariantCultureIgnoreCase);
				string claimName = expectedClaim.Substring(0, separatorIndex);
				string claimValue = expectedClaim.Substring(separatorIndex + 1);
				if (user.HasClaim(claim => string.Equals(claim.Type, claimName, StringComparison.OrdinalIgnoreCase) && string.Equals(claim.Value, claimValue, StringComparison.OrdinalIgnoreCase)))
				{
					claimFound = true;
				}
			}
			else if (user.HasClaim(claim => string.Equals(claim.Type, expectedClaim, StringComparison.OrdinalIgnoreCase)))
			{
				claimFound = true;
			}

			if (!claimFound)
			{
				allClaimsFound = false;
			}
		}

		if (allClaimsFound)
		{
			allowedActions.AddRange(Actions);
		}
		return allowedActions;
	}

	public IEnumerable<JupiterAclAction> Resolve(AuthorizationHandlerContext context)
	{
		return Resolve(context.User);
	}
}

// ReSharper disable once ClassNeverInstantiated.Global
public class AclPolicy
{
	/// <summary>
	/// Claims required to be present to be allowed to do the actions - if multiple claims are present *all* of them are required
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
	public List<string> Claims { get; set; } = new List<string>();

	/// <summary>
	/// The actions granted if the claims match
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
	public List<JupiterAclAction> Actions { get; set; } = new List<JupiterAclAction>();
	
	
	/// <summary>
	/// Debug name used to log information about the policy
	/// </summary>
	public string Name { get; set; } = "NameNotSet";

	/// <summary>
	/// Description of the scope for which this policy applies (typically some set of namespaces and or buckets)
	/// </summary>
	public AclScope? Scope { get; set; }

	public IEnumerable<JupiterAclAction> Resolve(ClaimsPrincipal user, AccessScope scope, ILogger logger, out List<string> matchedClaims)
	{
		matchedClaims = new List<string>();

		if (Scope == null)
		{
			// if no scope is set this policy is not valid so it does nothing
			logger.LogDebug("Scope was not set, policy \'{Name}\' is invalid.", Name);
			return Array.Empty<JupiterAclAction>();
		}

		bool allClaimsFound = true;
		foreach (string expectedClaim in Claims)
		{
			bool claimFound = false;
			// if expected claim is * then everyone has the associated actions
			if (expectedClaim == "*")
			{
				claimFound = true;
				logger.LogDebug("Found wildcard claim in policy \'{Name}\'.", Name);
				matchedClaims.Add("*");
			}
			else if (expectedClaim.Contains('=', StringComparison.InvariantCultureIgnoreCase))
			{
				int separatorIndex = expectedClaim.IndexOf('=', StringComparison.InvariantCultureIgnoreCase);
				string claimName = expectedClaim.Substring(0, separatorIndex);
				string claimValue = expectedClaim.Substring(separatorIndex + 1);
				if (user.HasClaim(claim => string.Equals(claim.Type.Trim(), claimName.Trim(), StringComparison.OrdinalIgnoreCase) && string.Equals(claim.Value.Trim(), claimValue.Trim(), StringComparison.OrdinalIgnoreCase)))
				{
					claimFound = true;
					logger.LogDebug("Found subset value claim {ClaimName} {ClaimValue} in policy \'{Name}\'.", claimName, claimValue, Name);
					matchedClaims.Add($"{claimName}={claimValue}");
				}
			}
			else if (user.HasClaim(claim => string.Equals(claim.Type.Trim(), expectedClaim.Trim(), StringComparison.OrdinalIgnoreCase)))
			{
				claimFound = true;
				logger.LogDebug("Found exact claim match {ExpectedClaim} in policy \'{Name}\'.", expectedClaim, Name);
				matchedClaims.Add(expectedClaim);
			}

			if (!claimFound)
			{
				allClaimsFound = false;
				break;
			}
		}

		if (!allClaimsFound)
		{
			logger.LogDebug("One or more claims was missing for policy \'{Name}\'.", Name);
			// one of the expected claims was not found, this policy does not apply
			return Array.Empty<JupiterAclAction>();
		}

		bool namespaceFound = false;
		// expected claim is found so next we check if the scope it grants access to applies
		foreach (AclScopeEntry scopeNamespace in Scope.Namespaces)
		{
			bool isMatch = scopeNamespace.Matches(scope.Namespace.ToString() ?? string.Empty, logger, Name);

			if (isMatch)
			{
				logger.LogDebug("Namespace scope matched for policy \'{Name}\'.", Name);
				namespaceFound = true;
				break;
			}
		}

		if (!namespaceFound)
		{
			// this policy did not apply for any of the namespaces of this scope
			return Array.Empty<JupiterAclAction>();
		}

		if (scope.Bucket == null)
		{
			// this is an operation that doesn't run on a bucket, if we have access to any bucket in the namespace we are allowed to do these operations as we are allowed to be aware of this namespace existing
			return Actions;
		}

		bool bucketFound = false;
		foreach (AclScopeEntry scopeBuckets in Scope.Buckets)
		{
			bool isMatch = scopeBuckets.Matches(scope.Bucket.ToString() ?? string.Empty, logger, Name);

			if (isMatch)
			{
				logger.LogDebug("Bucket scope matched for policy \'{Name}\'.", Name);
				bucketFound = true;
				break;
			}
		}

		bool requiresResourceScope = scope.ResourceType != null && scope.ResourceName != null;

		if (!requiresResourceScope && bucketFound)
		{
			// this policy match all claims and scopes, we grant the actions it contains
			return Actions;
		}

		if (requiresResourceScope)
		{
			bool resourceFound = false;
			foreach (AclResourceScopeEntry resourceScope in Scope.Resources)
			{
				if (string.Equals(resourceScope.ResourceType, scope.ResourceType, StringComparison.OrdinalIgnoreCase))
				{
					bool isMatch = resourceScope.Matches(scope.ResourceName ?? string.Empty, logger, Name);

					if (isMatch)
					{
						logger.LogDebug("Resource scope matched for policy \'{Name}\'.", Name);
						resourceFound = true;
						break;
					}
				}
			}

			if (resourceFound)
			{
				return Actions;
			}
		}

		// policy did not match
		return Array.Empty<JupiterAclAction>();
	}
}

/// <summary>
/// Description of the scope for which this policy applies (typically some set of namespaces and or buckets)
/// </summary>
// ReSharper disable once ClassNeverInstantiated.Global
public class AclScope
{
	/// <summary>
	/// List of one or more namespaces that the parent policy applies to
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
	// ReSharper disable once CollectionNeverUpdated.Global
	public List<AclScopeEntry> Namespaces { get; set; } = new List<AclScopeEntry>();

	/// <summary>
	///  List of one or more buckets that the parent policy applies to
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
	// ReSharper disable once CollectionNeverUpdated.Global
	public List<AclScopeEntry> Buckets { get; set; } = new List<AclScopeEntry>();

	/// <summary>
	///  List of one or more resources (by-name and type) that this policy applies to
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
	// ReSharper disable once CollectionNeverUpdated.Global
	public List<AclResourceScopeEntry> Resources { get; set; } = new List<AclResourceScopeEntry>();
}

/// <summary>
/// Potential mapping of a scope for a set of resources
/// </summary>
public class AclResourceScopeEntry : AclScopeEntry
{
	[Required]
	public string? ResourceType { get; set; } = null;
}

/// <summary>
/// Potential mapping of a scope
/// </summary>
// ReSharper disable once ClassNeverInstantiated.Global
public class AclScopeEntry
{
	// ReSharper disable once MemberCanBePrivate.Global
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
	public List<string> Values { get; set; } = new List<string>();
	// ReSharper disable once MemberCanBePrivate.Global
	public string? Match { get; set; } = null;
	private Regex? Regex { get; set; } = null;
	public bool Not { get; set; }

	public bool Matches(string scope, ILogger logger, string policyName)
	{
		string expectedScopeValue = scope;
		if (Match != null)
		{
			if (Regex == null)
			{
				// compile the regex
				try
				{
					Regex = new Regex(Match);
				}
				catch (ArgumentException e)
				{
					logger.LogWarning("Invalid regular expression: \"{Regex}\", ignoring. Error message: {ErrorMessage} in Policy {Name}", Match, e.Message, policyName);
					Regex = null;
					// invalid regular expression, nothing is considered matching to make sure we never grant access
					return false;
				}
			}

			Match match = Regex.Match(scope);
			if (!match.Success)
			{
				logger.LogDebug("Regular expression: \"{Regex}\", did not match scope \"{Scope}\" for policy {Name}", Match, scope, policyName);

				// regex does not match
				return false;
			}

			if (match.Groups.Count == 2)
			{
				expectedScopeValue = match.Groups[1].Value;
			}
			else
			{
				logger.LogError("Invalid regular expression: \"{Regex}\", matches more then 1 group, this is not supported. For policy {Name}", Match, policyName);

				return false;
			}
		}

		foreach (string value in Values)
		{
			bool matches;
			if (string.Equals(value, "*", StringComparison.OrdinalIgnoreCase))
			{
				logger.LogDebug("Scope value \"{Scope}\" matched wildcard value \"{Value}\" in policy {Name}.", expectedScopeValue, value, policyName);
				matches = true;
			}
			else
			{
				matches = value.Equals(expectedScopeValue, StringComparison.OrdinalIgnoreCase);
				logger.LogDebug("Compared values {Value0} and {Value1} with match result {Result} for policy {Name}. ", value, expectedScopeValue, matches, policyName);
			}

			if (Not && matches)
			{
				logger.LogDebug("Scope value \"{Scope}\" was a match for value \"{Value}\" in policy {Name}. This was a inverted operation to this is not considered a match.", expectedScopeValue, value, policyName);

				// this matched, and we have the Not flag set, meaning this should never match thus this scope is not valid
				return false;
			}

			if (matches)
			{
				logger.LogDebug("Scope value \"{Scope}\" was a match for value \"{Value}\" in policy {Name}.", expectedScopeValue, value, policyName);

				// if we find a match we can stop now, otherwise we try all other values
				return matches;
			}
		}

		if (Not)
		{
			logger.LogDebug("No values matched scope \"{Scope}\" in policy {Name}. This was a inverted operation so is considered a match.", expectedScopeValue, policyName);

			// no match found, that means this policy applies if the Not flag is set
			return true;
		}

		logger.LogDebug("No values matched scope \"{Scope}\" in policy {Name}.", expectedScopeValue, policyName);
		return false;
	}
}
