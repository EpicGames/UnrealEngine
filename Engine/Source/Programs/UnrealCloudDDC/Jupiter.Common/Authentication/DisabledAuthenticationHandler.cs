// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Security.Claims;
using System.Security.Principal;
using System.Text.Encodings.Web;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Authentication;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter
{
	// ReSharper disable once ClassNeverInstantiated.Global
	public class DisabledAuthenticationHandler : AuthenticationHandler<TestAuthenticationOptions>
	{
		public DisabledAuthenticationHandler(IOptionsMonitor<TestAuthenticationOptions> options, ILoggerFactory logger,
			UrlEncoder encoder) : base(options,
			logger, encoder)
		{
		}

		public static string AuthenticateScheme { get; } = "Test scheme";

		protected override Task<AuthenticateResult> HandleAuthenticateAsync()
		{
			AuthenticationTicket ticket =
				new AuthenticationTicket(principal: new GenericPrincipal(Options.Identity, null),
					properties: new AuthenticationProperties(), AuthenticateScheme);

			return Task.FromResult(AuthenticateResult.Success(ticket));
		}
	}

	public static class TestAuthenticationExtensions
	{
		public static AuthenticationBuilder AddTestAuth(this AuthenticationBuilder builder,
			Action<TestAuthenticationOptions> configOptions)
		{
			return builder.AddScheme<TestAuthenticationOptions, DisabledAuthenticationHandler>(
				DisabledAuthenticationHandler.AuthenticateScheme, "Test Auth",
				configOptions);
		}
	}

	public class TestAuthenticationOptions : AuthenticationSchemeOptions
	{
		public ClaimsIdentity Identity { get; } = new ClaimsIdentity(new Claim[]
		{
			new Claim(ClaimTypes.Name, "disabled-auth"),
			new Claim("sub", "disabled-auth-subject"),
			new Claim("Cache", "full"),
			new Claim("Tree", "full"),
			new Claim("TransactionLog", "full"),
			new Claim("Storage", "full"),
			new Claim("Admin", ""),
		}, "automatic");
	}
}
