// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Authorization;
using Microsoft.Extensions.Options;

namespace Jupiter
{
	public class GlobalAccessRequest
	{
		public JupiterAclAction[] Actions { get; init; } = Array.Empty<JupiterAclAction>();
	}

	/// <summary>
	/// Verifies that you have access to acl actions that are not tied to a namespace
	/// </summary>
	public class GlobalAuthorizationHandler : AuthorizationHandler<GlobalAccessRequirement, GlobalAccessRequest>
	{
		private readonly IOptionsMonitor<AuthSettings> _authSettings;

		public GlobalAuthorizationHandler(IOptionsMonitor<AuthSettings> authSettings)
		{
			_authSettings = authSettings;
		}

		protected override Task HandleRequirementAsync(AuthorizationHandlerContext context, GlobalAccessRequirement requirement,
			GlobalAccessRequest accessRequest)
		{
			if (accessRequest.Actions.Length == 0)
			{
				throw new Exception("At least 1 AclAction has to be specified for the namespace access request");
			}

			if (!_authSettings.CurrentValue.Enabled)
			{
				context.Succeed(requirement);
				return Task.CompletedTask;
			}

			List<JupiterAclAction> allowedActions = new List<JupiterAclAction>();
			foreach (AclEntry acl in _authSettings.CurrentValue.Acls)
			{
				allowedActions.AddRange(acl.Resolve(context));
			}

			bool haveAccessToActions = true;
			foreach (JupiterAclAction requiredAction in accessRequest.Actions)
			{
				if (!allowedActions.Contains(requiredAction))
				{
					haveAccessToActions = false;
				}
			}
			if (haveAccessToActions)
			{
				context.Succeed(requirement);
			}

			return Task.CompletedTask;
		}
	}

	public class GlobalAccessRequirement : IAuthorizationRequirement
	{
		public const string Name = "GlobalAccess";
	}
}
