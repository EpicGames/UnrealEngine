// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.AspNetCore.Authorization;
using Microsoft.Extensions.Options;

namespace Jupiter
{
	public class NamespaceAccessRequest
	{
		public NamespaceId Namespace { get; init; }
		public JupiterAclAction[] Actions { get; init; } = Array.Empty<JupiterAclAction>();
	}

	// verifies that you have access to a namespace by checking if you have a corresponding claim to that namespace
	public class NamespaceAuthorizationHandler : AuthorizationHandler<NamespaceAccessRequirement, NamespaceAccessRequest>
	{
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly IOptionsMonitor<AuthSettings> _authSettings;

		public NamespaceAuthorizationHandler(INamespacePolicyResolver namespacePolicyResolver, IOptionsMonitor<AuthSettings> authSettings)
		{
			_namespacePolicyResolver = namespacePolicyResolver;
			_authSettings = authSettings;
		}

		protected override Task HandleRequirementAsync(AuthorizationHandlerContext context, NamespaceAccessRequirement requirement,
			NamespaceAccessRequest accessRequest)
		{
			NamespaceId namespaceName = accessRequest.Namespace;
			if (!_authSettings.CurrentValue.Enabled && !_authSettings.CurrentValue.RequireAcls)
			{
				context.Succeed(requirement);
				return Task.CompletedTask;
			}

			try
			{
				if (accessRequest.Actions.Length == 0)
				{
					throw new Exception("At least 1 AclAction has to be specified for the namespace access request");
				}

				NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(namespaceName);

				List<JupiterAclAction> allowedActions = new List<JupiterAclAction>();
				foreach (AclEntry acl in policy.Acls)
				{
					allowedActions.AddRange(acl.Resolve(context));
				}

				// the root and namespace acls are combined, namespace acls can not override what we define in the root
				foreach (AclEntry acl in _authSettings.CurrentValue.Acls)
				{
					allowedActions.AddRange(acl.Resolve(context));
				}

				bool haveAccessToActions = true;
				foreach (JupiterAclAction requiredAction in accessRequest.Actions)
				{
					if (!allowedActions.Contains(requiredAction))
					{
						haveAccessToActions = false;
					}
				}
				if (haveAccessToActions)
				{
					context.Succeed(requirement);
				}
			}
			catch (NamespaceNotFoundException)
			{
				// if the namespace doesn't have a policy setup, e.g. we do not know which claims to require then we can just exit here as the auth will fail
			}

			return Task.CompletedTask;
		}
	}

	public class NamespaceAccessRequirement : IAuthorizationRequirement
	{
		public const string Name = "NamespaceAccess";
	}
}
