// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.AspNetCore.Authorization;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter
{
	public class ScopeAccessRequest
	{
		public JupiterAclAction[] Actions { get; init; } = Array.Empty<JupiterAclAction>();
		public AccessScope AccessScope { get; init; } = AccessScope.GlobalScope;
	}

	public class AccessScope
	{
		public AccessScope(NamespaceId? ns = null, BucketId? bucket = null)
		{
			Namespace = ns;
			Bucket = bucket;
		}

		public AccessScope(NamespaceId? ns, BucketId? bucket, string resourceType, string resourceName)
		{
			Namespace = ns;
			Bucket = bucket;
			ResourceType = resourceType;
			ResourceName = resourceName;
		}

		public static AccessScope GlobalScope { get; } = new AccessScope(null, null);
		public NamespaceId? Namespace { get; }
		public BucketId? Bucket { get; }

		public string? ResourceType { get; } = null;
		public string? ResourceName { get; } = null;
	}

	public class ScopeAuthorizationHandler : AuthorizationHandler<ScopeAccessRequirement, ScopeAccessRequest>
	{
		private readonly IOptionsMonitor<AuthSettings> _authSettings;
		private readonly ILogger<ScopeAuthorizationHandler> _logger;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;

		public ScopeAuthorizationHandler(IOptionsMonitor<AuthSettings> authSettings, ILogger<ScopeAuthorizationHandler> logger, INamespacePolicyResolver namespacePolicyResolver)
		{
			_authSettings = authSettings;
			_logger = logger;
			_namespacePolicyResolver = namespacePolicyResolver;
		}

		protected override Task HandleRequirementAsync(AuthorizationHandlerContext context, ScopeAccessRequirement requirement, ScopeAccessRequest accessRequest)
		{
			if (!_authSettings.CurrentValue.Enabled && !_authSettings.CurrentValue.RequireAcls)
			{
				context.Succeed(requirement);
				return Task.CompletedTask;
			}

			if (accessRequest.Actions.Length == 0)
			{
				throw new Exception("At least 1 AclAction has to be specified for the namespace access request");
			}

			List<JupiterAclAction> allowedActions = new List<JupiterAclAction>();
			foreach (AclPolicy acl in _authSettings.CurrentValue.Policies)
			{
				allowedActions.AddRange(acl.Resolve(context.User, accessRequest.AccessScope, _logger, out _));
			}

			if (_authSettings.CurrentValue.UseLegacyConfiguration)
			{
				// Check the legacy access configuration as well
				if (accessRequest.AccessScope.Namespace == null)
				{
					// no namespace set, is a global scope
					foreach (AclEntry acl in _authSettings.CurrentValue.Acls)
					{
						allowedActions.AddRange(acl.Resolve(context));
					}
				}
				else
				{
					NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(accessRequest.AccessScope.Namespace.Value);

					foreach (AclEntry acl in policy.Acls)
					{
						allowedActions.AddRange(acl.Resolve(context));
					}

					// the root and namespace acls are combined, namespace acls can not override what we define in the root
					foreach (AclEntry acl in _authSettings.CurrentValue.Acls)
					{
						allowedActions.AddRange(acl.Resolve(context));
					}
				}
			}
			

			bool haveAccessToActions = true;
			foreach (JupiterAclAction requiredAction in accessRequest.Actions)
			{
				if (!allowedActions.Contains(requiredAction))
				{
					haveAccessToActions = false;
				}
			}
			if (haveAccessToActions)
			{
				context.Succeed(requirement);
			}

			return Task.CompletedTask;
		}
	}

	public class ScopeAccessRequirement : IAuthorizationRequirement
	{
		public const string Name = "ScopeAccess";
	}
}
