// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Linq;
using System.Security.Claims;
using System.Text.Encodings.Web;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Authentication;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Microsoft.Extensions.Primitives;
using Microsoft.Net.Http.Headers;

namespace Jupiter
{
	class ServiceAccountAuthOptions : AuthenticationSchemeOptions
	{
		public class ServiceAccounts
		{
			[Required] public string Token { get; set; } = null!;

			public List<string> Claims { get; set; } = new List<string>();
		}

		public List<ServiceAccounts> Accounts { get; set; } = new List<ServiceAccounts>();
	}

	class ServiceAccountAuthHandler : AuthenticationHandler<ServiceAccountAuthOptions>
	{
		private readonly IOptionsMonitor<ServiceAccountAuthOptions> _options;
		public const string AuthenticationScheme = "ServiceAccount";
		public const string Prefix = "ServiceAccount";

		public ServiceAccountAuthHandler(IOptionsMonitor<ServiceAccountAuthOptions> options,
			ILoggerFactory logger, UrlEncoder encoder)
			: base(options, logger, encoder)
		{
			_options = options;
		}

		protected override async Task<AuthenticateResult> HandleAuthenticateAsync()
		{
			if (!Context.Request.Headers.TryGetValue(HeaderNames.Authorization, out StringValues headerValue))
			{
				return AuthenticateResult.NoResult();
			}

			if (headerValue.Count < 1)
			{
				return AuthenticateResult.NoResult();
			}

			string? header = headerValue[0];
			if (string.IsNullOrEmpty(header))
			{
				return AuthenticateResult.NoResult();
			}

			if (!header.StartsWith(Prefix, StringComparison.Ordinal))
			{
				return AuthenticateResult.NoResult();
			}

			await Task.CompletedTask;

			string token = header.Replace(Prefix, "", StringComparison.Ordinal).Trim();
			ServiceAccountAuthOptions.ServiceAccounts? serviceAccount = _options.CurrentValue.Accounts.FirstOrDefault(account => account.Token == token);
			if (serviceAccount == null)
			{
				return AuthenticateResult.Fail($"Service account for token {token} not found");
			}

			List<Claim> claims = new List<Claim>
			{
				new Claim(ClaimTypes.Name, AuthenticationScheme)
			};
			claims.Add(new Claim("sub", serviceAccount.Token));
			foreach (string claim in serviceAccount.Claims)
			{
				string[] tokens = claim.Split(":", 2);
				claims.Add(new Claim(tokens[0], tokens[1]));
			}

			ClaimsIdentity identity = new ClaimsIdentity(claims, Scheme.Name);
			ClaimsPrincipal principal = new ClaimsPrincipal(identity);
			AuthenticationTicket ticket = new AuthenticationTicket(principal, Scheme.Name);

			return AuthenticateResult.Success(ticket);
		}
	}
}
