// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.IO;
using EpicGames.Core;
using Microsoft.AspNetCore.Hosting;
using Microsoft.AspNetCore.Server.Kestrel.Core;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.Hosting;
using Serilog;
using Serilog.Core;
using Log = Serilog.Log;

namespace Jupiter
{
	// ReSharper disable once UnusedMember.Global
	public static class BaseProgram<T> where T : BaseStartup
	{
		private static IConfiguration Configuration { get; } = GetConfigurationBuilder();

		private static IConfiguration GetConfigurationBuilder()
		{
			string env = Environment.GetEnvironmentVariable("ASPNETCORE_ENVIRONMENT") ?? "Production";
			string mode = Environment.GetEnvironmentVariable("JUPITER_MODE") ?? "DefaultMode";
			string configRoot = "/config";
			// set the config root to config under the current directory for windows
			if (OperatingSystem.IsWindows())
			{
				configRoot = Path.Combine(Directory.GetCurrentDirectory(), "config");
			}
			return new ConfigurationBuilder()
				.SetBasePath(Directory.GetCurrentDirectory())
				.AddJsonFile("appsettings.json", false, false)
				.AddJsonFile(
					path:
					$"appsettings.{env}.json",
					true)
				.AddYamlFile(
					path:
					$"appsettings.{mode}.yaml",
					true)
				.AddYamlFile(Path.Combine(configRoot, "appsettings.Local.yaml"), optional: true, reloadOnChange: true)
				.AddEnvironmentVariables()
				.Build();

		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1000:Do not declare static members on generic types", Justification = "Keep logic for the base program together")]
		public static int BaseMain(string[] args)
		{
			Log.Logger = new LoggerConfiguration()
				.ReadFrom.Configuration(Configuration)
				.Enrich.With<DatadogLogEnricher>()
				.Enrich.WithRequestHeader("x-ue-session", "ue-session")
				.Enrich.WithRequestHeader("ue-session", "ue-session")
				.CreateLogger();

			try
			{
				JupiterSettings settings = new JupiterSettings();
				Configuration.GetSection("Jupiter").Bind(settings);
				string socketsRoot = settings.DomainSocketsRoot;

				if (settings.UseDomainSockets)
				{
					File.Delete(Path.Combine(socketsRoot, "jupiter-http.sock"));
					File.Delete(Path.Combine(socketsRoot, "jupiter-http2.sock"));
				}

				Log.Information("Creating ASPNET Host");
				IHost host = CreateHostBuilder(args).Build();
				host.Start();

				if (settings.ChmodDomainSockets)
				{
					FileUtils.SetFileMode_Linux(Path.Combine(socketsRoot, "jupiter-http.sock"), 766);
					FileUtils.SetFileMode_Linux(Path.Combine(socketsRoot, "jupiter-http2.sock"), 766);
				}

				host.WaitForShutdown();
				return 0;
			}
			catch (Exception ex)
			{
				Log.Fatal(ex, "Host terminated unexpectedly");
				return 1;
			}
			finally
			{
				Log.CloseAndFlush();
			}
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1000:Do not declare static members on generic types", Justification = "Keep logic for the base program together")]
		public static IHostBuilder CreateHostBuilder(string[] args)
		{
			return Host.CreateDefaultBuilder(args)
				.ConfigureWebHostDefaults(webBuilder =>
				{
					webBuilder.UseStartup<T>();
					webBuilder.UseConfiguration(Configuration);
					// configure microsoft.extensions.logging to configure log4net to allow us to set it in our appsettings
					// Disabled forwarding of log4net logs into serilog, as the AWS sdk is very spammy with its output producing multiple errors for a 404 (which isn't even an error in the first place)
					// This can be enabled if you need to investigate some more complicated AWS sdk issue
					/*webBuilder.ConfigureLogging((hostingContext, logging) =>
					{
						// configure log4net (used by aws sdk) to write to serilog so we get the logs in the system we want it in
						Log4net.Appender.Serilog.Configuration.Configure();
					});*/
					// remove the server header from kestrel

					JupiterSettings settings = new JupiterSettings();
					Configuration.GetSection("Jupiter").Bind(settings);

					if (settings.PendingConnectionMax.HasValue)
					{
						webBuilder.UseSockets(options =>
						{
							options.Backlog = settings.PendingConnectionMax.Value;
						});
					}

					webBuilder.ConfigureKestrel(options =>
					{
						options.AddServerHeader = false;

						// Decrease min request body rate to be more forgiving for clients that are going wide and not sending as much data per connection
						options.Limits.MinRequestBodyDataRate = new MinDataRate(10, TimeSpan.FromSeconds(60));

						// increase max request size of 1 MB to 8 MB
						options.Limits.MaxRequestBufferSize = 8 * 1024 * 1024;
						// increase header size to 1 MB
						options.Limits.MaxRequestHeadersTotalSize = 1024 * 1024;

						string socketsRoot = settings.DomainSocketsRoot;

						if (settings.UseDomainSockets)
						{
							Log.Logger.Information("Using unix domain sockets at {SocketsRoot}", socketsRoot);
							options.ListenUnixSocket(Path.Combine(socketsRoot, "jupiter-http.sock"), listenOptions =>
							{
								listenOptions.Protocols = HttpProtocols.Http1AndHttp2;
							});
							options.ListenUnixSocket(Path.Combine(socketsRoot, "jupiter-http2.sock"), listenOptions =>
							{
								listenOptions.Protocols = HttpProtocols.Http2;
							});
						}
					});
				}).UseSerilog();
		}
	}

	class DatadogLogEnricher : ILogEventEnricher
	{
		public void Enrich(Serilog.Events.LogEvent logEvent, ILogEventPropertyFactory propertyFactory)
		{
			string? ddAgentHost = System.Environment.GetEnvironmentVariable("DD_AGENT_HOST");
			if (string.IsNullOrEmpty(ddAgentHost))
			{
				return;
			}

			if (Activity.Current == null)
			{
				return;
			}

			// convert open telemetry trace ids to dd traces
			string stringTraceId = Activity.Current.TraceId.ToString();
			string stringSpanId = Activity.Current.SpanId.ToString();

			string ddTraceId = Convert.ToUInt64(stringTraceId.Substring(16), 16).ToString();
			string ddSpanId = Convert.ToUInt64(stringSpanId, 16).ToString();

			logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.trace_id", ddTraceId));
			logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.span_id", ddSpanId));
		}
	}
}
