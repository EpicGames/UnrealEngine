// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Diagnostics.Metrics;
using System.IdentityModel.Tokens.Jwt;
using System.IO;
using System.Linq;
using System.Net.Mime;
using System.Reflection;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using Amazon;
using EpicGames.AspNet;
using Jupiter.Common;
using Microsoft.AspNetCore.Authentication;
using Microsoft.AspNetCore.Authentication.JwtBearer;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Builder;
using Microsoft.AspNetCore.Diagnostics.HealthChecks;
using Microsoft.AspNetCore.Hosting;
using Microsoft.AspNetCore.HttpOverrides;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Microsoft.Extensions.Primitives;
using Microsoft.Net.Http.Headers;
using Microsoft.OpenApi.Models;
using Okta.AspNet.Abstractions;
using Okta.AspNetCore;
using OpenTelemetry.Logs;
using OpenTelemetry.Metrics;
using OpenTelemetry.Resources;
using OpenTelemetry.Trace;
using Serilog;
using OktaWebApiOptions = Okta.AspNetCore.OktaWebApiOptions;

namespace Jupiter
{
	using ILogger = Microsoft.Extensions.Logging.ILogger;

	public abstract class BaseStartup
	{
		protected ILogger Logger { get; }

		protected BaseStartup(IConfiguration configuration, ILogger logger)
		{
			Configuration = configuration;
			Auth = new AuthSettings();
			Logger = logger;
		}

		protected static ILogger CreateLogger<T>()
		{
			using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
			{
				builder.SetMinimumLevel(LogLevel.Information);
				builder.AddSerilog();
			});
			return loggerFactory.CreateLogger<T>();
		}

		protected IConfiguration Configuration { get; }

		private AuthSettings Auth { get; }

		// This method gets called by the runtime. Use this method to add services to the container.
		public void ConfigureServices(IServiceCollection services)
		{
			CbConvertersAspNet.AddAspnetConverters();

			services.AddServerTiming();
			services.AddLogging(builder => builder.AddSerilog());

			// aws specific settings
			services.AddOptions<AWSCredentialsSettings>().Bind(Configuration.GetSection("AWSCredentials")).ValidateDataAnnotations();
			services.AddDefaultAWSOptions(Configuration.GetAWSOptions());
			// send log4net logs to serilog and configure aws to log to log4net (they lack a serilog implementation)
			AWSConfigs.LoggingConfig.LogTo = LoggingOptions.Log4Net;

			services.AddOptions<AuthSettings>().Bind(Configuration.GetSection("Auth")).ValidateDataAnnotations().ValidateOnStart();
			Configuration.GetSection("Auth").Bind(Auth);

			services.AddOptions<ServiceAccountAuthOptions>().Bind(Configuration.GetSection("ServiceAccounts")).ValidateDataAnnotations();

			services.AddOptions<JupiterSettings>().Bind(Configuration.GetSection("Jupiter")).ValidateDataAnnotations();
			services.AddOptions<NamespaceSettings>().Bind(Configuration.GetSection("Namespaces")).ValidateDataAnnotations();

			services.AddSingleton(typeof(INamespacePolicyResolver), typeof(NamespacePolicyResolver));
			
			// allow CORS from any domain to call this api as you will need a valid token anyway
			services.AddCors(options =>
			{
				options.AddDefaultPolicy(policy =>
				{
					policy.AllowAnyOrigin();
					policy.AllowAnyHeader();
					policy.AllowAnyMethod();
				});
			});

			services.AddControllers()
				.AddMvcOptions(options =>
				{
					options.InputFormatters.Add(new CbInputFormatter());
					options.OutputFormatters.Add(new CbOutputFormatter());
					options.OutputFormatters.Add(new RawOutputFormatter(CreateLogger<RawOutputFormatter>()));
					options.FormatterMappings.SetMediaTypeMappingForFormat("raw", MediaTypeNames.Application.Octet);
					options.FormatterMappings.SetMediaTypeMappingForFormat("uecb", CustomMediaTypeNames.UnrealCompactBinary);
					options.FormatterMappings.SetMediaTypeMappingForFormat("uecbpkg", CustomMediaTypeNames.UnrealCompactBinaryPackage);

					OnAddControllers(options);
				}).ConfigureApiBehaviorOptions(options =>
				{
					options.InvalidModelStateResponseFactory = context =>
					{
						BadRequestObjectResult result = new BadRequestObjectResult(new ValidationProblemDetails(context.ModelState));
						// always return errors as json objects
						// we could allow more types here, but we do not want raw for instance
						result.ContentTypes.Add(MediaTypeNames.Application.Json);

						return result;
					};
				}).AddJsonOptions(jsonOptions => ConfigureJsonOptions(jsonOptions.JsonSerializerOptions));

			services.AddHttpContextAccessor();

			const string ForwardingScheme = "ForwardingScheme";
			List<string> availableSchemes = new List<string>();

			AuthenticationBuilder authenticationBuilder = services.AddAuthentication(options =>
				{
					if (Auth.Enabled)
					{
						if (Auth.Schemes.Count > 1)
						{
							// we have multiple schemes, so we set the default to the forwarding scheme which will use the jwtAuthority to pick the correct scheme for the token
							options.DefaultAuthenticateScheme = ForwardingScheme;
							options.DefaultChallengeScheme = ForwardingScheme;
						}
						else
						{
							// if we only have one scheme we set it to default
							options.DefaultAuthenticateScheme = Auth.DefaultScheme;
							options.DefaultChallengeScheme = Auth.DefaultScheme;

						}
					}
					else
					{
						options.DefaultAuthenticateScheme = DisabledAuthenticationHandler.AuthenticateScheme;
						options.DefaultChallengeScheme = DisabledAuthenticationHandler.AuthenticateScheme;
					}
				}
			);

			if (Auth.Enabled)
			{
				foreach (KeyValuePair<string, AuthSchemeEntry> schemeEntry in Auth.Schemes)
				{
					string name = schemeEntry.Key;
					AuthSchemeEntry scheme = schemeEntry.Value;

					switch (scheme.Implementation)
					{
						case SchemeImplementations.ServiceAccount:
							availableSchemes.Add(name);
							authenticationBuilder.AddScheme<ServiceAccountAuthOptions, ServiceAccountAuthHandler>(name, options => { });
							break;
						case SchemeImplementations.JWTBearer:
							availableSchemes.Add(name);
							authenticationBuilder.AddJwtBearer(name, options =>
							{
								options.Authority = scheme.JwtAuthority;
								options.Audience = scheme.JwtAudience;
							});
							break;
						case SchemeImplementations.Okta:
							JwtBearerEvents bearerEvents = new JwtBearerEvents();
							if (Auth.RemapNameClaim)
							{
								bool firstTime = true;
								bearerEvents.OnMessageReceived += context =>
								{
									if (!firstTime)
									{
										return Task.CompletedTask;
									}

									firstTime = false;
									context.Options.TokenValidationParameters.NameClaimType = "http://schemas.xmlsoap.org/ws/2005/05/identity/claims/nameidentifier";
									return Task.CompletedTask;
								};
							}
							
							availableSchemes.Add(name);
							authenticationBuilder.AddOktaWebApi(name, new OktaWebApiOptions
							{
								OktaDomain = scheme.OktaDomain,
								AuthorizationServerId = scheme.OktaAuthorizationServerId,
								Audience = scheme.JwtAudience,
								JwtBearerEvents = bearerEvents
							});
							break;
						default:
							throw new NotSupportedException($"Unknown implementation type {scheme.Implementation}");
					}
				}

				authenticationBuilder.AddPolicyScheme(ForwardingScheme, ForwardingScheme, options =>
				{
					options.ForwardDefaultSelector = context =>
					{
						string? authorization = context.Request.Headers[HeaderNames.Authorization];
						string name = "Bearer";
						string tokenName = $"{name} ";
						if (string.IsNullOrEmpty(authorization) || !authorization.StartsWith(tokenName, StringComparison.InvariantCulture))
						{
							return Auth.DefaultScheme;
						}

						string token = authorization.Substring(tokenName.Length).Trim();
						JwtSecurityTokenHandler jwtHandler = new JwtSecurityTokenHandler();

						if (!jwtHandler.CanReadToken(token))
						{
							return Auth.DefaultScheme;
						}

						JwtSecurityToken jwtToken = jwtHandler.ReadJwtToken(token);

						foreach (KeyValuePair<string, AuthSchemeEntry> entry in Auth.Schemes)
						{
							if (entry.Value.JwtAuthority == jwtToken.Issuer)
							{
								return entry.Key;
							}
						}

						return Auth.DefaultScheme;

					};
				});
			}
			else
			{
				availableSchemes.Add(DisabledAuthenticationHandler.AuthenticateScheme);
				authenticationBuilder.AddTestAuth(options => { });
			}

			services.AddAuthorization(options =>
			{
				options.AddPolicy(NamespaceAccessRequirement.Name, policy =>
				{
					policy.AuthenticationSchemes = availableSchemes;
					policy.Requirements.Add(new NamespaceAccessRequirement());
				});

				options.AddPolicy(GlobalAccessRequirement.Name, policy =>
				{
					policy.AuthenticationSchemes = availableSchemes;
					policy.Requirements.Add(new GlobalAccessRequirement());
				});

				
				options.AddPolicy(ScopeAccessRequirement.Name, policy =>
				{
					policy.AuthenticationSchemes = availableSchemes;
					policy.Requirements.Add(new ScopeAccessRequirement());
				});

				// A policy that grants any authenticated user access
				options.AddPolicy("Any", policy =>
				{
					policy.AuthenticationSchemes = availableSchemes;
					policy.RequireAuthenticatedUser();
				});

				OnAddAuthorization(options, availableSchemes);
			});
			services.AddSingleton<IAuthorizationHandler, NamespaceAuthorizationHandler>();
			services.AddSingleton<IAuthorizationHandler, GlobalAuthorizationHandler>();
			services.AddSingleton<IAuthorizationHandler, ScopeAuthorizationHandler>();

			string otelServiceName = Configuration["OTEL_SERVICE_NAME"] ?? "unreal-cloud-ddc";
			string? otelServiceVersion = Configuration["OTEL_SERVICE_VERSION"];

			string? useConsoleExporterString = Configuration["OTEL_USE_CONSOLE_EXPORTER"];
			_ = bool.TryParse(useConsoleExporterString, out bool useConsoleExporter);

			string? traceScyllaRequestsString = Configuration["OTEL_TRACE_SCYLLA_REQUESTS"];
			_ = bool.TryParse(traceScyllaRequestsString, out bool traceScyllaRequests);

			services.AddOpenTelemetry().ConfigureResource(builder =>
			{
				builder.AddService("UnrealCloudDDC", serviceNamespace: "Jupiter", serviceVersion: otelServiceVersion)
				.AddEnvironmentVariableDetector();
			}).WithTracing(builder =>
			{
				builder.AddHttpClientInstrumentation(options =>
				{
					options.EnrichWithHttpRequestMessage = (activity, message) =>
					{
						activity.AddTag("service.name", otelServiceName + "-http-client");
						activity.AddTag("operation.name", "http-request");

						string url = $"{message.Method} {message.Headers.Host}{message.RequestUri?.LocalPath}";
						activity.DisplayName = url;

						activity.AddTag("resource.name", url);
					};
				});
				builder.AddAspNetCoreInstrumentation(options =>
				{
					options.EnrichWithHttpRequest = (activity, request) =>
					{
						if (request.Headers.TryGetValue("x-ue-session", out StringValues ueSessionValues))
						{
							if (ueSessionValues.Count != 0)
							{
								activity.AddTag("ue-session", ueSessionValues.First());
							}
						}

						if (request.Headers.TryGetValue("ue-session", out StringValues ueSessionValuesOld))
						{
							if (ueSessionValuesOld.Count != 0)
							{
								activity.AddTag("ue-session", ueSessionValuesOld.First());
							}
						}

						if (request.Headers.TryGetValue("x-ue-request", out StringValues ueRequestValues))
						{
							if (ueRequestValues.Count != 0)
							{
								activity.AddTag("ue-request", ueRequestValues.First());
							}
						}

						if (request.Headers.TryGetValue("ue-request", out StringValues ueRequestValuesOld))
						{
							if (ueRequestValuesOld.Count != 0)
							{
								activity.AddTag("ue-request", ueRequestValuesOld.First());
							}
						}

						if (request.Headers.TryGetValue("ue-IsBuildMachine", out StringValues ueIsBuildMachine))
						{
							if (ueIsBuildMachine.Count != 0)
							{
								activity.AddTag("ue-IsBuildMachine", ueIsBuildMachine.First());
							}
						}
					};
				});
				builder.SetSampler(new AlwaysOnSampler());
				builder.AddOtlpExporter();
				if (useConsoleExporter)
				{
					builder.AddConsoleExporter();
				}
				builder.AddSource("UnrealCloudDDC");

				if (traceScyllaRequests)
				{
					builder.AddSource("ScyllaDB");
				}
			}).WithMetrics(builder =>
			{
				builder
					.AddMeter("UnrealCloudDDC")
					.AddOtlpExporter()
					.AddAspNetCoreInstrumentation()
					.AddHttpClientInstrumentation()
					.AddRuntimeInstrumentation();

				if (traceScyllaRequests)
				{
					builder.AddMeter("ScyllaDB");
				}
				if (useConsoleExporter)
				{
					builder.AddConsoleExporter();
				}
			});

			services.Configure<OpenTelemetryLoggerOptions>(opt =>
			{
				opt.IncludeScopes = true;
				opt.ParseStateValues = true;
				opt.IncludeFormattedMessage = true;
			});

			services.Configure<ForwardedHeadersOptions>(options =>
			{
				// include enough ips to allow for multiple proxies in front
				options.ForwardLimit = 3;

				// we do not know the address of load balancers in front of us but want to accept the forwarding headers from them so we reset these lists
				// we generally assume that traffic happens over https and that the network we are on is very limited in scope (only ingress via load balancers)
				// TODO: If we wanted to we could introduce a setting to specify the ip range of the load balancers here, but would be kind of annoying to have to specify
				// , but it could be useful if this is run in less secure networks
				options.KnownIPNetworks.Clear();
				options.KnownProxies.Clear();
			});

			services.AddSingleton<Tracer>(CreateTracer);

			services.AddSingleton<Meter>(CreateMeter);

			services.Configure<ForwardedHeadersOptions>(options =>
			{
				options.ForwardedHeaders = ForwardedHeaders.XForwardedFor | ForwardedHeaders.XForwardedProto;
			});

			services.AddSwaggerGen(settings =>
			{
				string? assemblyName = Assembly.GetEntryAssembly()?.GetName().Name;
				settings.SwaggerDoc("v1", info: new OpenApiInfo
				{
					Title = "Unreal Cloud DDC",
					Contact = new OpenApiContact
					{
						Name = "Joakim Lindqvist",
						Email = "joakim.lindqvist@epicgames.com",
					}
				});

				// Set the comments path for the Swagger JSON and UI.
				string xmlFile = $"{assemblyName}.xml";
				string xmlPath = Path.Combine(AppContext.BaseDirectory, xmlFile);
				if (File.Exists(xmlPath))
				{
					settings.IncludeXmlComments(xmlPath);
				}
			});

			OnAddService(services);

			OnAddHealthChecks(services);
		}

		public static void ConfigureJsonOptions(JsonSerializerOptions options)
		{
			options.AllowTrailingCommas = true;
			options.DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull;
			options.PropertyNameCaseInsensitive = true;

			options.Converters.Add(new JsonStringEnumConverter());
		}

		private Meter CreateMeter(IServiceProvider provider)
		{
			return new Meter("UnrealCloudDDC");
		}

		private Tracer CreateTracer(IServiceProvider provider)
		{
			Tracer tracer = TracerProvider.Default.GetTracer("UnrealCloudDDC");

			return tracer;
		}

		private void OnAddHealthChecks(IServiceCollection services)
		{
			IHealthChecksBuilder healthChecks = services.AddHealthChecks()
				.AddCheck("self", () => HealthCheckResult.Healthy(), tags: new[] { "self" });
			OnAddHealthChecks(services, healthChecks);
		}

		/// <summary>
		/// Register health checks for individual services
		/// </summary>
		/// <remarks>Use the self tag for checks if the service is running while the services tag can be used for any dependencies which needs to work</remarks>
		/// <param name="services">DI service injector</param>
		/// <param name="healthChecks">A already configured builder that you can add more checks to</param>
		protected abstract void OnAddHealthChecks(IServiceCollection services, IHealthChecksBuilder healthChecks);

		protected abstract void OnAddAuthorization(AuthorizationOptions authorizationOptions, List<string> defaultSchemes);

		protected virtual void OnAddControllers(MvcOptions options)
		{
		}

		protected abstract void OnAddService(IServiceCollection services);

		// This method gets called by the runtime. Use this method to configure the HTTP request pipeline.
		public void Configure(IApplicationBuilder app, IWebHostEnvironment env)
		{
			JupiterSettings jupiterSettings = app.ApplicationServices.GetService<IOptionsMonitor<JupiterSettings>>()!.CurrentValue;

			if (jupiterSettings.ShowPII)
			{
				Logger.LogError("Personally Identifiable information being shown. This should not be generally enabled in prod.");

				// do not hide personal information during development
				Microsoft.IdentityModel.Logging.IdentityModelEventSource.ShowPII = true;
			}

			ConfigureMiddlewares(jupiterSettings, app, env);
		}

		private void ConfigureMiddlewares(JupiterSettings jupiterSettings, IApplicationBuilder app, IWebHostEnvironment env)
		{
			// enable use of forwarding headers as we expect a reverse proxy to be running in front of us
			app.UseForwardedHeaders();

			if (jupiterSettings.UseRequestLogging)
			{
				app.UseSerilogRequestLogging();
			}

			OnConfigureAppEarly(app, env);

			if (env.IsDevelopment() && UseDeveloperExceptionPage)
			{
				app.UseDeveloperExceptionPage();
			}
			else
			{
				app.UseExceptionHandler("/error");
			}

			app.UseRouting();

			app.UseCors();

			app.UseAuthentication();
			app.UseAuthorization();

			app.UseMiddleware<SuppressExceptionMiddleware>();
			app.UseMiddleware<ServerTimingMiddleware>();

			app.UseEndpoints(endpoints =>
			{
				static bool PassAllChecks(HealthCheckRegistration check) => true;

				// Ready checks in Kubernetes is to verify that the service is working, if this returns false the app will not get any traffic (load balancer ignores it)
				endpoints.MapHealthChecks("/health/readiness", options: new HealthCheckOptions()
				{
					Predicate = jupiterSettings.DisableHealthChecks ? PassAllChecks : (check) => check.Tags.Contains("self"),
				});

				// Live checks in Kubernetes to see if the pod is working as it should, if this returns false the entire pod is killed
				endpoints.MapHealthChecks("/health/liveness", options: new HealthCheckOptions()
				{
					Predicate = jupiterSettings.DisableHealthChecks ? PassAllChecks : (check) => check.Tags.Contains("services"),
				});

				endpoints.MapGet("/health/ready", async context =>
				{
					context.Response.StatusCode = 200;
					context.Response.Headers.ContentType = "text/plain";
					await context.Response.Body.WriteAsync(Encoding.ASCII.GetBytes("Healthy"));
				});

				endpoints.MapGet("/health/live", async context =>
				{
					context.Response.StatusCode = 200;
					context.Response.Headers.ContentType = "text/plain";
					await context.Response.Body.WriteAsync(Encoding.ASCII.GetBytes("Healthy"));
				});

				endpoints.MapControllers();
			});

			if (jupiterSettings.HostSwaggerDocumentation)
			{
				app.UseSwagger();
				app.UseReDoc(options => { options.SpecUrl = "/swagger/v1/swagger.json"; });
			}

			OnConfigureApp(app, env);
		}

		public virtual bool UseDeveloperExceptionPage { get; } = false;

		protected virtual void OnConfigureAppEarly(IApplicationBuilder app, IWebHostEnvironment env)
		{
		}

		protected virtual void OnConfigureApp(IApplicationBuilder app, IWebHostEnvironment env)
		{

		}
	}

	/*public class MvcJsonOptionsWrapper : IConfigureOptions<MvcNewtonsoftJsonOptions>
	{
		readonly IServiceProvider ServiceProvider;

		public MvcJsonOptionsWrapper(IServiceProvider serviceProvider)
		{
			ServiceProvider = serviceProvider;
		}
		public void Configure(MvcNewtonsoftJsonOptions options)
		{
			options.SerializerSettings.ContractResolver = new FieldFilteringResolver(ServiceProvider);
		}
	}*/

	/*public class FieldFilteringResolver : DefaultContractResolver
	{
		private readonly IHttpContextAccessor _httpContextAccessor;

		public FieldFilteringResolver(IServiceProvider sp)
		{
			_httpContextAccessor = sp.GetRequiredService<IHttpContextAccessor>();

			NamingStrategy = new CamelCaseNamingStrategy(false, true);
		}

		protected override JsonProperty CreateProperty(MemberInfo member, MemberSerialization memberSerialization)
		{

			JsonProperty property = base.CreateProperty(member, memberSerialization);

			property.ShouldSerialize = o =>
			{
				HttpContext? httpContext = _httpContextAccessor.HttpContext;

				if (httpContext == null)
				{
					return true;
				}

				// if no fields are being filtered we should serialize the property
				if (!httpContext.Request.Query.ContainsKey("fields"))
				{
					return true;
				}

				StringValues fields = httpContext.Request.Query["fields"];
				bool ignore = true;
				foreach (string field in fields)
				{
					// a empty field to filter for is considered a match for everything as fields= should be the same as just omitting fields
					if (string.IsNullOrEmpty(field))
					{
						return true;
					}
					if (string.Equals(field, property.PropertyName, StringComparison.OrdinalIgnoreCase))
					{
						ignore = false;
					}
				}

				return !ignore;
			};

			return property;
		}
	}*/

	public enum SchemeImplementations
	{
		JWTBearer,
		Okta,
		ServiceAccount
	};

	public class AuthSchemeEntry : IValidatableObject
	{
		/// <summary>
		/// The implementation to use, this controls which other configuration values needs to be set. For most servers JWTBearer should work fine.
		/// </summary>
		[Required]
		public SchemeImplementations Implementation { get; set; } = SchemeImplementations.JWTBearer;

		/// <summary>
		/// The Okta domain (url to your okta server - do not include the authorization server id)
		/// </summary>
		public string OktaDomain { get; set; } = "";
		/// <summary>
		/// The Okta AuthorizationServerId, this is used if you have more then one authorization server within your Okta server. We recommend using a separate authorization server for each major set of systems to reduce blast radius of security issues.
		/// </summary>
		public string OktaAuthorizationServerId { get; set; } = OktaWebOptions.DefaultAuthorizationServerId;

		/// <summary>
		/// The JWT Authority (url to your IdP)
		/// </summary>
		public string JwtAuthority { get; set; } = "";

		/// <summary>
		/// The audience for the token, this is usually defined by your IdP 
		/// </summary>
		[Required]
		public string JwtAudience { get; set; } = "";

		/// <summary>
		/// The namespaces which these scheme is allowed to grant access to, all if this is omitted or empty
		/// </summary>
		public string[] AllowedNamespaces { get; set; } = Array.Empty<string>();

		public IEnumerable<ValidationResult> Validate(ValidationContext validationContext)
		{
			List<ValidationResult> validationResults = new List<ValidationResult>();
			if (Implementation == SchemeImplementations.JWTBearer)
			{
				if (string.IsNullOrEmpty(JwtAuthority))
				{
					validationResults.Add(new ValidationResult("JWT Authority must be specified when using JWTBearer implementation"));
				}
				if (string.IsNullOrEmpty(JwtAudience))
				{
					validationResults.Add(new ValidationResult("JWT Audience must be specified when using JWTBearer implementation"));
				}
			}
			else if (Implementation == SchemeImplementations.Okta)
			{
				if (string.IsNullOrEmpty(OktaDomain))
				{
					validationResults.Add(new ValidationResult("Okta Domain must be specified when using Okta implementation"));
				}
				if (string.IsNullOrEmpty(JwtAudience))
				{
					validationResults.Add(new ValidationResult("JWT Audience must be specified when using Okta implementation"));
				}
				if (string.IsNullOrEmpty(JwtAuthority))
				{
					validationResults.Add(new ValidationResult("JWT Authority must be specified when using Okta implementation"));
				}
			}
			else
			{
				throw new NotSupportedException($"Unknown auth implementation {Implementation}");
			}

			return validationResults;
		}
	}

	/// <summary>
	/// The definition of how a specific oidc client works, should be the same as ProviderInfo in EpicGames.Okta. This will be read by EpicGames.Okta so its definition is authoritative. 
	/// </summary>
	public class ProviderInfo
	{
		/// <summary>
		/// The server uri to the IdP
		/// </summary>
		[Required] public Uri ServerUri { get; set; } = null!;

		/// <summary>
		/// The client id as set by the IdP
		/// </summary>

		[Required] public string ClientId { get; set; } = null!;

		/// <summary>
		/// The display name to use for this provider, usually the same as the provider id
		/// </summary>

		[Required] public string DisplayName { get; set; } = null!;

		/// <summary>
		/// The redirect uri - deprecated - use PossibleRedirectUri instead as supporting multiple ports is very useful when running on localhost were ports maybe taken
		/// </summary>
		public Uri? RedirectUri { get; set; } = null;
		
		/// <summary>
		/// List of redirect uris that can be used (are allow listed in the IdP). In priority order.
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
		public List<Uri>? PossibleRedirectUri { get; set; } = null!;

		/// <summary>
		/// Enable to load claims from the user profile after the token has been read. Converted it from a thin token to a think token.
		/// </summary>
		[Required] public bool LoadClaimsFromUserProfile { get; set; } = false;

		/// <summary>
		/// The scopes to request
		/// </summary>
		public string Scopes { get; set; } = "openid profile offline_access email";

		/// <summary>
		/// A string added to the local error page when failing to login, usually a good place to tell users who and how to get help.
		/// </summary>
		public string? GenericErrorInformation { get; set; }

		/// <summary>
		/// Can be disabled to not use the discovery documents that are part of the oidc standard to find endpoints.
		/// </summary>
		public bool UseDiscoveryDocument { get; set; } = true;

		// We should absolutely not include client secrets here as this information is public on internet, thus you want public clients used
		//public string? ClientSecret { get; set; } = null;
	}

	public class ClientOidcConfiguration
	{
		public string? DefaultProvider { get; set; } = null;

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
		public Dictionary<string, ProviderInfo> Providers { get; set; } = new Dictionary<string, ProviderInfo>();
	}

	public class AuthSettings : IValidatableObject
	{
		/// <summary>
		/// The name of the scheme to use by default
		/// </summary>
		public string DefaultScheme { get; set; } = "Bearer";

		/// <summary>
		/// Used to disable authentication, not recommended to set for anything other then local use cases
		/// </summary>
		public bool Enabled { get; set; } = true;

		/// <summary>
		/// Can be set to require acls to be used even if auth is disabled, mostly used for testing
		/// </summary>
		public bool RequireAcls { get; set; } = false;

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
		public Dictionary<string, AuthSchemeEntry> Schemes { get; set; } = new Dictionary<string, AuthSchemeEntry>();

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
		public List<AclEntry> Acls { get; set; } = new List<AclEntry>();

		/// <summary>
		/// Remaps name claim from http://schemas.xmlsoap.org/ws/2005/05/identity/claims/name to http://schemas.xmlsoap.org/ws/2005/05/identity/claims/nameidentifier which is required for Okta
		/// </summary>
		public bool RemapNameClaim { get; set; } = true;

		/// <summary>
		/// Setup policies for how access control is managed
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
		public List<AclPolicy> Policies { get; set; } = new List<AclPolicy>();

		/// <summary>
		/// Set to true to also consider the legacy configuration in the acl fields of this object as well as within the namespace policy for endpoints that support the acl policies
		/// </summary>
		public bool UseLegacyConfiguration { get; set; } = false;

		/// <summary>
		/// Configuration read by clients on how to interactively login locally
		/// </summary>
		public ClientOidcConfiguration? ClientOidcConfiguration { get; set; } = null;

		/// <summary>
		/// The encryption key used to encrypt the configuration - usually not modified as you then need to also distribute this key. This needs to be exactly 16 bytes
		/// </summary>
		public string ClientOidcEncryptionKey { get; set; } = "892a27ef5cbf4894af2e6bd53a54aa48";

		public IEnumerable<ValidationResult> Validate(ValidationContext validationContext)
		{
			List<ValidationResult> validationResults = new List<ValidationResult>();
			if (!Enabled)
			{
				return validationResults;
			}

			if (Schemes.Count == 0)
			{
				validationResults.Add(new ValidationResult("You must have at least one scheme when authentication is enabled"));
			}

			if (!Schemes.ContainsKey(DefaultScheme))
			{
				validationResults.Add(new ValidationResult($"Expected to find a scheme with the name {DefaultScheme} as its set as the default scheme"));
			}

			return validationResults;
		}
	}

	public class JupiterSettings
	{
		// enable to unhide potentially personal information, see https://aka.ms/IdentityModel/PII
		public bool ShowPII { get; set; } = false;
		public bool DisableHealthChecks { get; set; } = false;
		public bool HostSwaggerDocumentation { get; set; } = true;

		/// <summary>
		/// Port used to host the internally accessible api (as well as the public api).
		/// This hosts both public and private namespaces
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]

		public List<int> InternalApiPorts { get; set; } = new List<int>() { 8080 };

		/// <summary>
		/// Port that hosts public and private namespaces
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]

		public List<int> CorpApiPorts { get; set; } = new List<int>() { 8008 };

		/// <summary>
		/// Port that only hosts the public namespaces
		/// </summary>
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]

		public List<int> PublicApiPorts { get; set; } = new List<int>() { 80 };

		// Enable to echo every request to the log file, usually this is more efficiently done on the load balancer
		public bool UseRequestLogging { get; set; } = false;

		/// <summary>
		///  Name of the current site, has to be globally unique across all deployments
		/// </summary>
		[Required]
		[Key]
		public string CurrentSite { get; set; } = "";

		/// <summary>
		/// Used to move where domain sockets are allocated
		/// </summary>
		public string DomainSocketsRoot { get; set; } = "/tmp/sockets";

		/// <summary>
		/// Enable to create unix domain sockets for inter process communication
		/// </summary>
		public bool UseDomainSockets { get; set; } = false;

		/// <summary>
		/// Enable to change access (chmod) the sockets to allow for anyone to access them
		/// </summary>
		public bool ChmodDomainSockets { get; set; } = false;

		/// <summary>
		/// Assumes that any local connection should have full access, this is used only for tests
		/// </summary>
		public bool AssumeLocalConnectionsHasFullAccess { get; set; }

		/// <summary>
		/// Set this to increase the max number of TCP connections that are allowed to be queued.
		/// </summary>
		public int? PendingConnectionMax { get; set; }
	}
}
