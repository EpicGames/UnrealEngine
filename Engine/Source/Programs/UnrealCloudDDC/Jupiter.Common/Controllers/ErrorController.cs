// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.AspNetCore.Diagnostics;
using Microsoft.AspNetCore.Mvc;
using OpenTelemetry.Trace;

namespace Jupiter.Controllers
{
	[ApiController]
	[ApiExplorerSettings(IgnoreApi = true)]
	public class ErrorController : ControllerBase
	{
		[Route("/error")]
		public IActionResult Error()
		{
			IExceptionHandlerFeature? context = HttpContext.Features.Get<IExceptionHandlerFeature>();

			if (context == null)
			{
				return NoContent();
			}

			// ignore cancelled exceptions from the health checks, that will happen if a health check is started while another is running
			// the later will contain a valid result
			if (context.Error is OperationCanceledException)
			{
				if (context.Path.StartsWith("/health", StringComparison.Ordinal))
				{
					return Ok();
				}
			}

			// add the error to our trace
			Tracer.CurrentSpan.SetStatus(Status.Error);
			Tracer.CurrentSpan.RecordException(context.Error);

			return Problem(
				detail: context.Error.StackTrace,
				title: context.Error.Message);
		}
	}
}
