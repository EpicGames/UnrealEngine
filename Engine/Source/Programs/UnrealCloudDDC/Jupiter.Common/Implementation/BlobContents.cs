// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading.Tasks;
using Jupiter.Common.Implementation;

namespace Jupiter.Implementation
{
	public sealed class BlobContents : IDisposable, IAsyncDisposable
	{
		private Stream? _stream;
		private readonly IBufferedPayload? _bufferedPayload;

		public BlobContents(Stream stream, long length, string? localPath = null, string? absolutePath = null)
		{
			_stream = stream;

			Length = length;
			LocalPath = localPath;
			AbsolutePath = absolutePath;
		}

		public BlobContents(byte[] payload)
		{
			_stream = new MemoryStream(payload);
			Length = payload.LongLength;
			LocalPath = null;
		}

		public BlobContents(IBufferedPayload payload)
		{
			_bufferedPayload = payload;
			Length = payload.Length;
			LocalPath = null;
		}

		public BlobContents(Uri redirectUri)
		{
			RedirectUri = redirectUri;
		}

		public Uri? RedirectUri { get; }

		public Stream Stream
		{
			get
			{
				if (_bufferedPayload != null && _stream == null)
				{
					_stream = _bufferedPayload.GetStream();
				}

				if (_stream == null)
				{
					throw new Exception("Stream not set in blob contents, did you specify support for redirect uris but forgot to check result?");
				}
				return _stream;
			}
		}
		public long Length { get; }

		public string? LocalPath { get; }
		public string? AbsolutePath { get; }
		
		public IBufferedPayload? BufferedPayload => _bufferedPayload;

		public void Dispose()
		{
			_stream?.Dispose();
			_bufferedPayload?.Dispose();
		}

		public ValueTask DisposeAsync()
		{
			_stream?.Dispose();
			return ValueTask.CompletedTask;
		}
	}
}
