// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.ComponentModel;
using System.Globalization;
using System.IO;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using Blake3;
using EpicGames.Core;
using EpicGames.Serialization;

namespace Jupiter.Implementation
{
	[TypeConverter(typeof(BlobIdentifierTypeConverter))]
	[JsonConverter(typeof(BlobIdentifierJsonConverter))]
	[CbConverter(typeof(BlobIdentifierCbConverter))]
	public class BlobId : ContentHash, IEquatable<BlobId>
	{
		// multi thread the hashing for blobs larger then this size
		private const int MultiThreadedSize = 1_000_000;
		private string? _stringIdentifier;

		public BlobId(byte[] identifier) : base(identifier)
		{
		}

		[JsonConstructor]
		public BlobId(string identifier) : base(identifier)
		{

		}

		public override int GetHashCode()
		{
			return Comparer.GetHashCode(Identifier);
		}

		public bool Equals(BlobId? other)
		{
			if (other == null)
			{
				return false;
			}

			return Comparer.Equals(Identifier, other.Identifier);
		}

		public override bool Equals(object? obj)
		{
			if (ReferenceEquals(null, obj))
			{
				return false;
			}

			if (ReferenceEquals(this, obj))
			{
				return true;
			}

			if (obj.GetType() != GetType())
			{
				return false;
			}

			return Equals((BlobId)obj);
		}

		public override string ToString()
		{
			if (_stringIdentifier == null)
			{
				_stringIdentifier = StringUtils.FormatAsHexString(Identifier);
			}

			return _stringIdentifier;
		}

		public static new BlobId FromBlob(byte[] blobMemory)
		{
			Hash blake3Hash;
			if (blobMemory.Length < MultiThreadedSize)
			{
				using Hasher hasher = Hasher.New();
				hasher.Update(blobMemory);
				blake3Hash = hasher.Finalize();
			}
			else
			{
				using Hasher hasher = Hasher.New();
				hasher.UpdateWithJoin(blobMemory);
				blake3Hash = hasher.Finalize();
			}

			// we only keep the first 20 bytes of the Blake3 hash
			Span<byte> hash = blake3Hash.AsSpan().Slice(0, 20);
			return new BlobId(hash.ToArray());
		}

		public static BlobId FromBlob(in Memory<byte> blobMemory)
		{
			Hash blake3Hash;
			if (blobMemory.Length < MultiThreadedSize)
			{
				using Hasher hasher = Hasher.New();
				hasher.Update(blobMemory.Span);
				blake3Hash = hasher.Finalize();
			}
			else
			{
				using Hasher hasher = Hasher.New();
				hasher.UpdateWithJoin(blobMemory.Span);
				blake3Hash = hasher.Finalize();
			}

			// we only keep the first 20 bytes of the Blake3 hash
			Span<byte> hash = blake3Hash.AsSpan().Slice(0, 20);
			return new BlobId(hash.ToArray());
		}

		public static BlobId FromContentHash(ContentHash testObjectHash)
		{
			return new BlobId(testObjectHash.HashData);
		}

		public static async Task<BlobId> FromStreamAsync(Stream stream, CancellationToken cancellationToken = default)
		{
			using Hasher hasher = Hasher.New();

			const int BufferSize = 1024 * 1024 * 5;
			byte[] buffer = ArrayPool<byte>.Shared.Rent(BufferSize);

			try
			{
				int read = await stream.ReadAsync(buffer, 0, buffer.Length, cancellationToken);
				while (read > 0)
				{
					hasher.UpdateWithJoin(new ReadOnlySpan<byte>(buffer, 0, read));
					read = await stream.ReadAsync(buffer, 0, buffer.Length, cancellationToken);
				}
				Hash blake3Hash = hasher.Finalize();

				// we only keep the first 20 bytes of the Blake3 hash
				byte[] hash = blake3Hash.AsSpan().Slice(0, 20).ToArray();
				return new BlobId(hash);
			}
			finally
			{
				ArrayPool<byte>.Shared.Return(buffer);
			}
		}

		public static BlobId FromIoHash(IoHash blobIdentifier)
		{
			return new BlobId(blobIdentifier.ToByteArray());
		}

		public IoHash AsIoHash()
		{
			return new IoHash(HashData);
		}
	}

	public class BlobIdentifierTypeConverter : TypeConverter
	{
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			if (sourceType == typeof(string))
			{
				return true;
			}
			return base.CanConvertFrom(context, sourceType);
		}

		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string s)
			{
				return new BlobId(s);
			}

			return base.ConvertFrom(context, culture, value);
		}

		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
		{
			if (destinationType == typeof(string))
			{
				return true;
			}
			return base.CanConvertTo(context, destinationType);
		}

		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				BlobId? identifier = (BlobId?)value;
				return identifier?.ToString();
			}
			return base.ConvertTo(context, culture, value, destinationType);
		}
	}

	public class BlobIdentifierJsonConverter : JsonConverter<BlobId>
	{
		public override BlobId? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? str = reader.GetString();
			if (str == null)
			{
				throw new InvalidDataException("Unable to parse blob identifier");
			}

			return new BlobId(str);
		}

		public override void Write(Utf8JsonWriter writer, BlobId value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.ToString());
		}
	}

	public class BlobIdentifierCbConverter : CbConverter<BlobId>
	{
		public override BlobId Read(CbField field) => new BlobId(field.AsHash().ToByteArray());

		/// <inheritdoc/>
		public override void Write(CbWriter writer, BlobId value) => writer.WriteBinaryAttachmentValue(new IoHash(value.HashData));

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, BlobId value) => writer.WriteBinaryAttachment(name, new IoHash(value.HashData));
	}
}
