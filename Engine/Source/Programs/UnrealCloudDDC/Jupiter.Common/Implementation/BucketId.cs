// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Serialization;

namespace Jupiter.Common
{
	/// <summary>
	/// Identifier for a storage bucket
	/// </summary>
	[JsonConverter(typeof(BucketIdJsonConverter))]
	[TypeConverter(typeof(BucketIdTypeConverter))]
	[CbConverter(typeof(BucketIdCbConverter))]
	public struct BucketId : IEquatable<BucketId>
	{
		/// <summary>
		/// The text representing this id
		/// </summary>
		readonly StringId _inner;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="input">Unique id for the string</param>
		public BucketId(string input)
		{
			StringId id;
			if (!StringId.TryParse(new Utf8String(input), MaxLength, out id, out string? errorMessage))
			{
				throw new ArgumentException(errorMessage, nameof(input));
			}

			_inner = id;
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is BucketId id && _inner.Equals(id._inner);

		/// <inheritdoc/>
		public override int GetHashCode() => _inner.GetHashCode();

		/// <inheritdoc/>
		public bool Equals(BucketId other) => _inner.Equals(other._inner);

		/// <inheritdoc/>
		public override string ToString() => _inner.ToString();

		/// <inheritdoc cref="StringId.op_Equality"/>
		public static bool operator ==(BucketId left, BucketId right) => left._inner == right._inner;

		/// <inheritdoc cref="StringId.op_Inequality"/>
		public static bool operator !=(BucketId left, BucketId right) => left._inner != right._inner;

		// Increase the max length compared to normal string ids
		private const int MaxLength = 256;
	}

	/// <summary>
	/// Type converter for BucketId to and from JSON
	/// </summary>
	sealed class BucketIdJsonConverter : JsonConverter<BucketId>
	{
		/// <inheritdoc/>
		public override BucketId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new BucketId(reader.GetString() ?? string.Empty);

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, BucketId value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToString());
	}

	/// <summary>
	/// Type converter from strings to BucketId objects
	/// </summary>
	sealed class BucketIdTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type? sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object? value) => new BucketId((string)value!);
	}

	sealed class BucketIdCbConverter : CbConverter<BucketId>
	{
		public override BucketId Read(CbField field) => new BucketId(field.AsString());

		/// <inheritdoc/>
		public override void Write(CbWriter writer, BucketId value) => writer.WriteStringValue(value.ToString());

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, BucketId value) => writer.WriteString(name, value.ToString());
	}
}
