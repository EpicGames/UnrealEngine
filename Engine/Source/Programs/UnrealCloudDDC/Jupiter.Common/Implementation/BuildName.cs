// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;
using Jupiter.Common;
using EpicGames.Serialization;

namespace Jupiter.Implementation;

/// <summary>
/// Identifier for a build
/// </summary>
[JsonConverter(typeof(BuildNameJsonConverter))]
[TypeConverter(typeof(BuildNameTypeConverter))]
[CbConverter(typeof(BuildNameCbConverter))]
public readonly struct BuildName : IEquatable<BuildName>
{
	/// <summary>
	/// The text representing this id
	/// </summary>
	readonly StringId _inner;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="input">Unique id for the string</param>
	public BuildName(string input)
	{
		_inner = new StringId(input);
	}

	/// <inheritdoc/>
	public override bool Equals(object? obj) => obj is BuildName id && _inner.Equals(id._inner);

	/// <inheritdoc/>
	public override int GetHashCode() => _inner.GetHashCode();

	/// <inheritdoc/>
	public bool Equals(BuildName other) => _inner.Equals(other._inner);

	/// <inheritdoc/>
	public override string ToString() => _inner.ToString();

	/// <inheritdoc cref="StringId.op_Equality"/>
	public static bool operator ==(BuildName left, BuildName right) => left._inner == right._inner;

	/// <inheritdoc cref="StringId.op_Inequality"/>
	public static bool operator !=(BuildName left, BuildName right) => left._inner != right._inner;
}

/// <summary>
/// Type converter for BucketId to and from JSON
/// </summary>
sealed class BuildNameJsonConverter : JsonConverter<BuildName>
{
	/// <inheritdoc/>
	public override BuildName Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new BuildName(reader.GetString() ?? string.Empty);

	/// <inheritdoc/>
	public override void Write(Utf8JsonWriter writer, BuildName value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToString());
}

/// <summary>
/// Type converter from strings to BucketId objects
/// </summary>
sealed class BuildNameTypeConverter : TypeConverter
{
	/// <inheritdoc/>
	public override bool CanConvertFrom(ITypeDescriptorContext? context, Type? sourceType) => sourceType == typeof(string);

	/// <inheritdoc/>
	public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object? value) => new BuildName((string)value!);
}

sealed class BuildNameCbConverter : CbConverter<BuildName>
{
	public override BuildName Read(CbField field) => new BuildName(field.AsString());

	/// <inheritdoc/>
	public override void Write(CbWriter writer, BuildName value) => writer.WriteStringValue(value.ToString());

	/// <inheritdoc/>
	public override void WriteNamed(CbWriter writer, CbFieldName name, BuildName value) => writer.WriteString(name, value.ToString());
}
