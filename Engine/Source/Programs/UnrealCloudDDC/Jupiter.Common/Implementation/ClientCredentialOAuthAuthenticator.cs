// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net.Http;
using System.Net.Http.Json;
using System.Threading;
using System.Threading.Tasks;

namespace Jupiter.Implementation
{
	public interface IAuthenticator
	{
		public Task<string?> AuthenticateAsync(CancellationToken cancellationToken);
	}

	public class ClientCredentialOAuthAuthenticator : IAuthenticator
	{
		public ClientCredentialOAuthAuthenticator(IHttpClientFactory httpClientFactory, Uri authUrl, string clientId, string clientSecret, string scope)
		{
			_httpClientFactory = httpClientFactory;
			_authUrl = authUrl;
			_clientId = clientId;
			_clientSecret = clientSecret;
			_scope = scope;
		}

		private string? _accessToken;

		private readonly IHttpClientFactory _httpClientFactory;
		private readonly Uri _authUrl;
		private readonly string _clientId;
		private readonly string _clientSecret;
		private readonly string _scope;
		private DateTime _expiresAt;

		public async Task<string?> AuthenticateAsync(CancellationToken cancellationToken = default)
		{
			if (string.IsNullOrEmpty(_accessToken) || DateTime.Now > _expiresAt)
			{
				await PreAuthenticateAsync(cancellationToken);
			}

			return _accessToken;
		}

		private async Task PreAuthenticateAsync(CancellationToken cancellationToken = default)
		{
			(ClientCredentialsResponse result, string body) = await DoAuthenticationRequestAsync(cancellationToken);
			string? accessToken = result.access_token;
			if (string.IsNullOrEmpty(accessToken))
			{
				throw new InvalidOperationException("The authentication token received by the server is null or empty. Body received was: " + body);
			}
			_accessToken = accessToken;
			// renew after half the renewal time
			_expiresAt = DateTime.Now + TimeSpan.FromSeconds((result?.expires_in ?? 3200) / 2.0);

		}

		private async Task<(ClientCredentialsResponse, string)> DoAuthenticationRequestAsync(CancellationToken cancellationToken = default)
		{
			using HttpClient client = _httpClientFactory.CreateClient();
			using HttpRequestMessage request = new HttpRequestMessage(HttpMethod.Post, _authUrl);

			request.Content = new FormUrlEncodedContent(new[]
				{
					new KeyValuePair<string, string>("grant_type", "client_credentials"),
					new KeyValuePair<string, string>("client_id", _clientId),
					new KeyValuePair<string, string>("client_secret", _clientSecret),
					new KeyValuePair<string, string>("scope", _scope)
				});
			HttpResponseMessage response = await client.SendAsync(request, cancellationToken);

			string s = await response.Content.ReadAsStringAsync(cancellationToken);
			if (!response.IsSuccessStatusCode)
			{
				throw new AuthenticationFailedException(s);
			}

			ClientCredentialsResponse? responseBody = await response.Content.ReadFromJsonAsync<ClientCredentialsResponse>(cancellationToken);
			if (responseBody == null)
			{
				throw new Exception("Unable to deserialize client credential response");
			}
			return (responseBody, s);
		}
	}

	// ReSharper disable once ClassNeverInstantiated.Global
	internal class ClientCredentialsResponse
	{
		public string? access_token { get; set; }

		public string? token_type { get; set; }

		public int? expires_in { get; set; }

		public string? scope { get; set; }
	}

	public class AuthenticationFailedException : Exception
	{
		public AuthenticationFailedException(string errorResult) : base(errorResult) { }
	}
}
