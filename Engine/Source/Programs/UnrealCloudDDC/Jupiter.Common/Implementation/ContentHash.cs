// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.IO;
using System.Text.Json;
using System.Text.Json.Serialization;
using Blake3;
using EpicGames.Core;
using EpicGames.Serialization;

namespace Jupiter.Implementation
{
	[TypeConverter(typeof(ContentHashTypeConverter))]
	[JsonConverter(typeof(ContentHashJsonConverter))]
	[CbConverter(typeof(ContentHashCbConverter))]
	public class ContentHash : IEquatable<ContentHash>, IEquatable<byte[]>
	{
		protected ByteArrayComparer Comparer { get; } = new ByteArrayComparer();
		protected byte[] Identifier { get; init; }
		public const int HashLength = 20;

		public byte[] HashData => Identifier;
		public ContentHash(byte[] identifier)
		{
			Identifier = identifier;
			/*if (identifier.Length != HashLength)
			{
				throw new ArgumentException("Supplied identifier was not 20 bytes, this is not a valid identifier", nameof(identifier));
			}*/
		}

		[JsonConstructor]
		public ContentHash(string identifier)
		{
			if (identifier == null)
			{
				throw new ArgumentNullException(nameof(identifier));
			}

			byte[] byteIdentifier = StringUtils.ToHashFromHexString(identifier);
			Identifier = byteIdentifier;
			/*if (byteIdentifier.Length != HashLength)
			{
				throw new ArgumentException("Supplied identifier was not 20 bytes, this is not a valid identifier", nameof(identifier));
			}*/
		}

		public override int GetHashCode()
		{
			return Comparer.GetHashCode(Identifier);
		}

		public bool Equals(ContentHash? other)
		{
			if (other == null)
			{
				return false;
			}

			return Comparer.Equals(Identifier, other.Identifier);
		}

		public bool Equals(byte[]? other)
		{
			if (other == null)
			{
				return false;
			}

			return Comparer.Equals(Identifier, other);
		}

		public override bool Equals(object? obj)
		{
			if (ReferenceEquals(null, obj))
			{
				return false;
			}

			if (ReferenceEquals(this, obj))
			{
				return true;
			}

			if (obj.GetType() != GetType())
			{
				return false;
			}

			return Equals((ContentHash)obj);
		}

		public override string ToString()
		{
			return StringUtils.FormatAsHexString(Identifier);
		}

		public static ContentHash FromBlob(byte[] blobMemory)
		{
			using Hasher hasher = Hasher.New();
			hasher.UpdateWithJoin(blobMemory);
			Hash blake3Hash = hasher.Finalize();

			// we only keep the first 20 bytes of the Blake3 hash
			Span<byte> hash = blake3Hash.AsSpan().Slice(0, HashLength);

			return new ContentHash(hash.ToArray());
		}
	}

	public class ContentHashCbConverter : CbConverter<ContentHash>
	{
		public override ContentHash Read(CbField field) => new ContentHash(field.AsHash().ToByteArray());

		/// <inheritdoc/>
		public override void Write(CbWriter writer, ContentHash value) => writer.WriteHashValue(new IoHash(value.HashData));

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, ContentHash value) => writer.WriteHash(name, new IoHash(value.HashData));
	}

	public class ContentHashJsonConverter : JsonConverter<ContentHash>
	{
		public override ContentHash Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? str = reader.GetString();
			if (str == null)
			{
				throw new InvalidDataException("Unable to parse content hash");
			}

			return new ContentHash(str);
		}

		public override void Write(Utf8JsonWriter writer, ContentHash value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.ToString());
		}
	}

	public class ContentHashTypeConverter : TypeConverter
	{
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			if (sourceType == typeof(string))
			{
				return true;
			}
			return base.CanConvertFrom(context, sourceType);
		}

		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string s)
			{
				return new ContentHash(s);
			}

			return base.ConvertFrom(context, culture, value);
		}

		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType)
		{
			if (destinationType == typeof(string))
			{
				return true;
			}
			return base.CanConvertTo(context, destinationType);
		}

		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				ContentHash? identifier = (ContentHash?)value;
				return identifier?.ToString();
			}
			return base.ConvertTo(context, culture, value, destinationType);
		}
	}
}
