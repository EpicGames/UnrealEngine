// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.IO;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Serialization;

namespace Jupiter.Implementation
{
	[TypeConverter(typeof(ContentIdTypeConverter))]
	[JsonConverter(typeof(ContentIdJsonConverter))]
	[CbConverter(typeof(ContentIdCbConverter))]
	public class ContentId : ContentHash, IEquatable<ContentId>
	{
		public ContentId(byte[] identifier) : base(identifier)
		{
		}

		[JsonConstructor]
		public ContentId(string identifier) : base(identifier)
		{

		}

		public override int GetHashCode()
		{
			return Comparer.GetHashCode(Identifier);
		}

		public bool Equals(ContentId? other)
		{
			if (other == null)
			{
				return false;
			}

			return Comparer.Equals(Identifier, other.Identifier);
		}

		public override bool Equals(object? obj)
		{
			if (ReferenceEquals(null, obj))
			{
				return false;
			}

			if (ReferenceEquals(this, obj))
			{
				return true;
			}

			if (obj.GetType() != GetType())
			{
				return false;
			}

			return Equals((ContentId)obj);
		}

		public static ContentId FromContentHash(ContentHash contentHash)
		{
			return new ContentId(contentHash.HashData);
		}

		public static ContentId FromBlobIdentifier(BlobId blobIdentifier)
		{
			return new ContentId(blobIdentifier.HashData);
		}

		public BlobId AsBlobIdentifier()
		{
			return new BlobId(HashData);
		}

		public static ContentId FromIoHash(IoHash ioHash)
		{
			return new ContentId(ioHash.ToByteArray());
		}

		public IoHash AsIoHash()
		{
			return new IoHash(HashData);
		}
	}

	public class ContentIdTypeConverter : TypeConverter
	{
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{  
			if (sourceType == typeof(string))  
			{  
				return true;
			}  
			if (sourceType == typeof(ContentHash))  
			{  
				return true;
			}  
			return base.CanConvertFrom(context, sourceType);
		}  
  
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)  
		{
			if (value is string s)
			{
				return new ContentId(s);
			}

			if (value is ContentHash contentHash)
			{
				return ContentId.FromContentHash(contentHash);
			}

			return base.ConvertFrom(context, culture, value);  
		}  
	}

	public class ContentIdJsonConverter : JsonConverter<ContentId>
	{
		public override ContentId? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? str = reader.GetString();
			if (str == null)
			{
				throw new InvalidDataException("Unable to parse content id");
			}

			return new ContentId(str);
		}

		public override void Write(Utf8JsonWriter writer, ContentId value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.ToString());
		}
	}

	public class ContentIdCbConverter : CbConverter<ContentId>
	{
		public override ContentId Read(CbField field) => new ContentId(field.AsHash().ToByteArray());

		/// <inheritdoc/>
		public override void Write(CbWriter writer, ContentId value) => writer.WriteHashValue(new IoHash(value.HashData));

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, ContentId value) => writer.WriteHash(name, new IoHash(value.HashData));
	}
}
