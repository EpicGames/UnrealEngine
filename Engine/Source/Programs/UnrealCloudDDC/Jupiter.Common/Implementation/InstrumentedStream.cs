// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.Metrics;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

namespace Jupiter.Common.Implementation
{
	/// <summary>
	/// Wraps a stream and adds instrumentation for throughput which is written to a counter
	/// </summary>
	public class InstrumentedStream : Stream
	{
		private readonly Stream _streamImplementation;
		private readonly string _sourceIdentifier;
		private bool _started = false;
		private long _totalBytesWritten = 0;
		private DateTime _consumeStartedAt;

		private static Histogram<double>? s_streamThroughputCounter;

		public InstrumentedStream(Stream s, Meter meter, string sourceIdentifier)
		{
			_streamImplementation = s;
			_sourceIdentifier = sourceIdentifier;

			if (s_streamThroughputCounter == null)
			{
				s_streamThroughputCounter = meter.CreateHistogram<double>("jupiter.stream_throughput");
			}
		}

		public override async ValueTask DisposeAsync()
		{
			await _streamImplementation.DisposeAsync();
			await base.DisposeAsync();
			GC.SuppressFinalize(this);
		}

		public override void Close()
		{
			TimeSpan duration = DateTime.Now - _consumeStartedAt;
			double rate = _totalBytesWritten / duration.TotalSeconds;

			s_streamThroughputCounter?.Record(rate, new KeyValuePair<string, object?>("sourceIdentifier", _sourceIdentifier));
			base.Close();
		}

		public override void Flush()
		{
			_streamImplementation.Flush();
		}

		public override int Read(byte[] buffer, int offset, int count)
		{
			if (!_started)
			{
				_consumeStartedAt = DateTime.Now;
				_started = true;
			}

			Interlocked.Add(ref _totalBytesWritten, count);
			return _streamImplementation.Read(buffer, offset, count);
		}

		public override long Seek(long offset, SeekOrigin origin)
		{
			return _streamImplementation.Seek(offset, origin);
		}

		public override void SetLength(long value)
		{
			_streamImplementation.SetLength(value);
		}

		public override void Write(byte[] buffer, int offset, int count)
		{
			if (!_started)
			{
				_consumeStartedAt = DateTime.Now;
				_started = true;
			}

			Interlocked.Add(ref _totalBytesWritten, count);
			_streamImplementation.Write(buffer, offset, count);
		}

		public override bool CanRead => _streamImplementation.CanRead;

		public override bool CanSeek => _streamImplementation.CanSeek;

		public override bool CanWrite => _streamImplementation.CanWrite;

		public override long Length => _streamImplementation.Length;

		public override long Position
		{
			get => _streamImplementation.Position;
			set => _streamImplementation.Position = value;
		}
	}
}
