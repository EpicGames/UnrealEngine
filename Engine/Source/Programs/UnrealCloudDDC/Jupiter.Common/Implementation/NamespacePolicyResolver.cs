// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using Microsoft.Extensions.Options;

namespace Jupiter.Common
{
	public interface INamespacePolicyResolver
	{
		public NamespacePolicy GetPoliciesForNs(NamespaceId ns);

		static NamespaceId JupiterInternalNamespace => new NamespaceId("jupiter-internal");
	}

	public class NamespaceSettings
	{
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
		public Dictionary<string, NamespacePolicy> Policies { get; set; } = new Dictionary<string, NamespacePolicy>();
	}

	public class NamespacePolicy
	{
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by the configuration system")]
		public List<AclEntry> Acls { get; set; } = new List<AclEntry>();
		public string StoragePool { get; set; } = "";

		public bool LastAccessTracking { get; set; } = true;
		public bool OnDemandReplication { get; set; } = false;
		public bool UseBlobIndexForExists { get; set; } = false;
		public bool UseBlobIndexForSlowExists { get; set; } = false;
		public bool IsPublicNamespace { get; set; } = true;
		public NamespaceId? FallbackNamespace { get; set; } = null;
		public bool PopulateFallbackNamespaceOnUpload { get; set; } = true;

		public bool UseContentAddressedStorage { get; set; } = true;

		/// <summary>
		/// Set to limit the amount of bytes used for filesystem storage - PerNamespaceGc needs to be set in the Filesystem settings for this to be applicable
		/// </summary>
		public ulong? MaxFilesystemStorageBytes { get; set; } = null;

		/// <summary>
		/// When enabled blobs that are written are not saved in the cache, only read blobs would be saved. Useful for write heavy operations were we frequently only read parts of the data.
		/// </summary>
		public bool? BypassCacheOnWrite { get; set; } = null;

		public enum StoragePoolGCMethod
		{
			/// <summary>
			/// Never run GC on this namespace
			/// </summary>
			None,
			/// <summary>
			/// Apply last access deletion, objects not used for a duration set in GCSettings will be removed
			/// </summary>
			LastAccess,
			/// <summary>
			/// Objects are removed after DefaultTTL time has passed, no matter if they are used or not
			/// </summary>
			TTL,
			/// <summary>
			/// Always GC references to this namespace, used to opt in to cleaning out old data
			/// </summary>
			Always
		};

		public StoragePoolGCMethod? GcMethod { get; set; } = null;

		public TimeSpan DefaultTTL { get; set; } = TimeSpan.FromDays(14);
		public bool AllowRedirectUris { get; set; } = false;
		public bool AllowOverwritesOfRefs { get; set; } = true;
	}

	public class NamespacePolicyResolver : INamespacePolicyResolver
	{
		private readonly IOptionsMonitor<NamespaceSettings> _namespaceSettings;
		private readonly NamespacePolicy _internalNamespacePolicy;

		public NamespacePolicyResolver(IOptionsMonitor<NamespaceSettings> namespaceSettings, IOptionsMonitor<AuthSettings> authSettings)
		{
			_namespaceSettings = namespaceSettings;
			_internalNamespacePolicy = new NamespacePolicy()
			{
				StoragePool = "persistent",
				GcMethod = NamespacePolicy.StoragePoolGCMethod.None
			};

			// if auth is disabled add a default namespace policy that allows access to everything unless something else exists to override it
			if (!authSettings.CurrentValue.Enabled)
			{
				_namespaceSettings.CurrentValue.Policies.TryAdd("*", new NamespacePolicy() { Acls = new List<AclEntry> { new() { Claims = new List<string> { "*" } } } });
			}
		}

		public NamespacePolicy GetPoliciesForNs(NamespaceId ns)
		{
			if (_namespaceSettings.CurrentValue.Policies.TryGetValue(ns.ToString(), out NamespacePolicy? settings))
			{
				return settings;
			}

			// attempt to find the default mapping
			if (_namespaceSettings.CurrentValue.Policies.TryGetValue("*", out NamespacePolicy? defaultSettings))
			{
				return defaultSettings;
			}

			// if no override has been specified for the internal jupiter namespace then we default it
			if (ns == INamespacePolicyResolver.JupiterInternalNamespace)
			{
				return _internalNamespacePolicy;
			}

			throw new NamespaceNotFoundException(ns, $"Unable to find a valid policy for namespace {ns}");
		}
	}
}
