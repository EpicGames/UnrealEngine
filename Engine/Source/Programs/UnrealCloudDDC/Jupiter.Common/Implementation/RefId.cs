// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.IO;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Serialization;

namespace Jupiter.Implementation
{
	[TypeConverter(typeof(IoHashKeyTypeConverter))]
	[JsonConverter(typeof(IoHashKeyJsonConverter))]
	[CbConverter(typeof(IoHashKeyCbConverter))]

	public readonly struct RefId : IEquatable<RefId>
	{
		public RefId(string key)
		{
			_text = key.ToLower();

			if (_text.Length != 40)
			{
				throw new ArgumentException("IoHashKeys must be exactly 40 bytes.");
			}

			for (int idx = 0; idx < _text.Length; idx++)
			{
				if (!IsValidCharacter(_text[idx]))
				{
					throw new ArgumentException($"{_text} is not a valid namespace id");
				}
			}
		}

		static bool IsValidCharacter(char c)
		{
			if (c >= 'a' && c <= 'z')
			{
				return true;
			}
			if (c >= '0' && c <= '9')
			{
				return true;
			}
			return false;
		}
		readonly string _text;

		public RefId(byte[] hash) : this(StringUtils.FormatAsHexLowerString(hash))
		{
		}

		public bool Equals(RefId other)
		{
			return string.Equals(_text, other._text, StringComparison.Ordinal);
		}

		public override bool Equals(object? obj)
		{
			return obj is RefId other && Equals(other);
		}

		public override int GetHashCode()
		{
			return _text.GetHashCode(StringComparison.Ordinal);
		}

		public override string ToString()
		{
			return _text;
		}

		public static bool operator ==(RefId left, RefId right)
		{
			return left.Equals(right);
		}

		public static bool operator !=(RefId left, RefId right)
		{
			return !left.Equals(right);
		}

		public static RefId FromName(string s)
		{
			return new RefId(ContentHash.FromBlob(Encoding.UTF8.GetBytes(s)).ToString());
		}

		public IoHash AsIoHash()
		{
			return new IoHash(StringUtils.ToHashFromHexString(_text));
		}
	}

	public sealed class IoHashKeyTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			if (sourceType == typeof(string))
			{
				return true;
			}

			return base.CanConvertFrom(context, sourceType);
		}

		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string s)
			{
				return new RefId(s);
			}

			return base.ConvertFrom(context, culture, value);
		}
	}

	public class IoHashKeyJsonConverter : JsonConverter<RefId>
	{
		public override RefId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? str = reader.GetString();
			if (str == null)
			{
				throw new InvalidDataException("Unable to parse io hash key");
			}

			return new RefId(str);
		}

		public override void Write(Utf8JsonWriter writer, RefId value, JsonSerializerOptions options)
		{
			writer.WriteStringValue(value.ToString());
		}
	}

	sealed class IoHashKeyCbConverter : CbConverter<RefId>
	{
		public override RefId Read(CbField field) => new RefId(field.AsString());

		/// <inheritdoc/>
		public override void Write(CbWriter writer, RefId value) => writer.WriteStringValue(value.ToString());

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, RefId value) => writer.WriteString(name, value.ToString());
	}
}

