// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text.Json;
using System.Threading.Tasks;
using Amazon.SecretsManager;
using Amazon.SecretsManager.Model;
using Azure;
using Azure.Identity;
using Azure.Security.KeyVault.Secrets;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace Jupiter.Common.Implementation
{
	public interface ISecretResolver
	{
		string Resolve(string value);
	}

	public class SecretResolver : ISecretResolver
	{
		private readonly IServiceProvider _serviceProvider;
		private readonly ILogger _logger;

		public SecretResolver(IServiceProvider serviceProvider, ILogger<SecretResolver> logger)
		{
			_serviceProvider = serviceProvider;
			_logger = logger;
		}

		public string Resolve(string value)
		{
			int providerSeparator = value.IndexOf("!", StringComparison.Ordinal);
			if (providerSeparator != -1)
			{
				string providerId = value.Substring(0, providerSeparator);
				string providerPath = value.Substring(providerSeparator + 1);
				return ResolveUsingProvider(providerId, providerPath, value);
			}

			return value;
		}

		private string ResolveUsingProvider(string providerId, string providerPath, string originalValue)
		{
			switch (providerId.ToUpperInvariant())
			{
				case "AWS":
					return ResolveAWSSecret(providerPath);
				case "AKV":
					return ResolveAKVSecret(providerPath);
				default:
					// no provider matches so just return the original value
					return originalValue;
			}
		}

		private string ResolveAWSSecret(string providerPath)
		{
			SplitByFirstSeparator(providerPath, '|', out string arn, out string? key);

			IAmazonSecretsManager? secretsManager = _serviceProvider.GetService<IAmazonSecretsManager>();
			if (secretsManager == null)
			{
				throw new Exception($"Unable to get AWSSecretsManager when resolving aws secret resource: {providerPath}");
			}

			string secretValue;
			try
			{
				Task<GetSecretValueResponse>? response = secretsManager.GetSecretValueAsync(new GetSecretValueRequest
				{
					SecretId = arn,
				});
				secretValue = response.Result.SecretString;
			}
			catch (ResourceNotFoundException e)
			{
				_logger.LogError(e, "Failed to find AWS secret: {Arn}", arn);
				throw;
			}

			if (key == null)
			{
				return secretValue;
			}

			Dictionary<string, string>? keyCollection = JsonSerializer.Deserialize<Dictionary<string, string>>(secretValue);

			if (keyCollection == null)
			{
				throw new Exception($"Unable to deserialize secret to a json payload for path: {providerPath}");
			}

			if (keyCollection.TryGetValue(key, out string? s))
			{
				return s;
			}

			throw new Exception($"Unable to find key {key} in blob returned for secret {arn}");
		}

		/// <summary>
		/// AKV = Azure Key Vault.
		/// Uses <see cref="DefaultAzureCredential"/> to get a secret from Azure Key Vault.
		/// Using <see cref="DefaultAzureCredential"/> lets us debug using <see cref="VisualStudioCredential"/>
		/// and deploy to an app service that uses <see cref="ManagedIdentityCredential"/>.
		/// </summary>
		/// <param name="providerPath"></param>
		/// <returns></returns>
		/// <exception cref="Exception"></exception>
		private static string ResolveAKVSecret(string providerPath)
		{
			if (!SplitByFirstSeparator(providerPath, '|', out string vaultName, out string? secretName))
			{
				throw new InvalidDataException("Azure Key Vault secret path must be of the form vaultName|secretName");
			}

			string vaultUrl = $"https://{vaultName}.vault.azure.net/";

			SecretClient client = new SecretClient(new Uri(vaultUrl), new DefaultAzureCredential());
			Response<KeyVaultSecret> response = client.GetSecret(secretName);
			return response.Value.Value;
		}

		private static bool SplitByFirstSeparator(string fullPath, char sep, out string left, out string? right)
		{
			int keySeparator = fullPath.IndexOf(sep, StringComparison.OrdinalIgnoreCase);
			left = fullPath;
			right = null;
			if (keySeparator != -1)
			{
				left = fullPath.Substring(0, keySeparator);
				right = fullPath.Substring(keySeparator + 1);
				return true;
			}
			return false;
		}
	}
}
