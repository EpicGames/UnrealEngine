// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Utils;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Common.Implementation
{
	public interface IBufferedPayload : IDisposable
	{
		Stream GetStream();
		long Length { get; }
	}

	/// <summary>
	/// Streaming request that is streamed into memory
	/// </summary>
	public sealed class MemoryBufferedPayload : IBufferedPayload
	{
		private readonly byte[] _buffer;

		public MemoryBufferedPayload(byte[] source)
		{
			_buffer = source;
		}

		public static async Task<MemoryBufferedPayload> CreateAsync(Tracer tracer, Stream s, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = tracer.StartActiveSpan("payload.buffer")
				.SetAttribute("operation.name", "payload.buffer")
				.SetAttribute("bufferType", "Memory");
			MemoryBufferedPayload payload = new MemoryBufferedPayload(await s.ToByteArrayAsync(cancellationToken));

			return payload;
		}

		public void Dispose()
		{

		}

		public Stream GetStream()
		{
			return new MemoryStream(_buffer);
		}

		public long Length => _buffer.LongLength;
	}

	/// <summary>
	/// Helper to generate a filesystem buffered payload from a stream 
	/// </summary>
	public sealed class FilesystemBufferedPayloadWriter : IDisposable
	{
		private FileInfo? _tempFile;

		private FilesystemBufferedPayloadWriter(string filesystemRoot, string debugPrefix)
		{
			_tempFile = new FileInfo(Path.Combine(filesystemRoot,$"{debugPrefix}-{Path.GetRandomFileName()}"));
		}

		public void Dispose()
		{
			if (_tempFile is { Exists: true })
			{
				_tempFile.Delete();
			}
		}

		public FilesystemBufferedPayload Done()
		{
			if (_tempFile == null)
			{
				throw new Exception("Writable buffer already closed once");
			}

			FilesystemBufferedPayload payload = new FilesystemBufferedPayload(_tempFile);
			// transfer ownership of the temp file to the filesystem buffered payload
			_tempFile = null;
			return payload;
		}

		public Stream GetWritableStream()
		{
			if (_tempFile == null)
			{
				throw new Exception("Writable buffer was closed when fetching writable stream");
			}

			return _tempFile.OpenWrite();
		}

		public static FilesystemBufferedPayloadWriter Create(string filesystemTempPayloadRoot, string debugPrefix)
		{
			return new FilesystemBufferedPayloadWriter(filesystemTempPayloadRoot, debugPrefix);
		}
	}

	/// <summary>
	/// A streaming request backed by a temporary file on disk
	/// </summary>
	public sealed class FilesystemBufferedPayload : IBufferedPayload
	{
		private readonly FileInfo _tempFile;
		private long _length;

		public FileInfo TempFile => _tempFile;

		private FilesystemBufferedPayload(string filesystemRoot, string debugPrefix)
		{
			_tempFile = new FileInfo(Path.Combine(filesystemRoot, $"{debugPrefix}-{Path.GetRandomFileName()}"));
		}

		internal FilesystemBufferedPayload(FileInfo bufferFile)
		{
			_tempFile = bufferFile;
			_tempFile.Refresh();
			_length = _tempFile.Length;
		}

		public static async Task<FilesystemBufferedPayload> CreateAsync(Tracer tracer, Stream s, string filesystemRoot, string debugPrefix, CancellationToken cancellationToken)
		{
			FilesystemBufferedPayload payload = new FilesystemBufferedPayload(filesystemRoot, debugPrefix);

			{
				using TelemetrySpan? scope = tracer.StartActiveSpan("payload.buffer")
					.SetAttribute("operation.name", "payload.buffer")
					.SetAttribute("bufferType", "Filesystem");
				await using FileStream fs = payload._tempFile.OpenWrite();
				await s.CopyToAsync(fs, cancellationToken);
			}

			payload._tempFile.Refresh();
			payload._length = payload._tempFile.Length;

			return payload;
		}

		public void Dispose()
		{
			if (_tempFile.Exists)
			{
				_tempFile.Delete();
			}
		}

		public Stream GetStream()
		{
			return _tempFile.OpenRead();
		}

		public long Length => _length;
	}

	public class BufferedPayloadOptions
	{
		/// <summary>
		/// If the request is smaller then MemoryBufferSize we buffer it in memory rather then as a file
		/// </summary>
		public long MemoryBufferSize { get; set; } = int.MaxValue;

		/// <summary>
		/// The default root to create temporary buffered files under, defaults to %TEMP%/temp-payloads or /tmp/temp-payloads
		/// </summary>
		public string FilesystemTempPayloadRoot { get; set; } = Path.Combine(Path.GetTempPath(), "temp-payloads");
	}

	public class BufferedPayloadFactory
	{
		private readonly IOptionsMonitor<BufferedPayloadOptions> _options;
		private readonly Tracer _tracer;

		public BufferedPayloadFactory(IOptionsMonitor<BufferedPayloadOptions> options, Tracer tracer)
		{
			_options = options;
			_tracer = tracer;

			Directory.CreateDirectory(options.CurrentValue.FilesystemTempPayloadRoot);
		}

		public Task<IBufferedPayload> CreateFromRequestAsync(HttpRequest request, string debugPrefix, CancellationToken cancellationToken)
		{
			long? contentLength = request.ContentLength;

			if (contentLength == null)
			{
				throw new Exception("Expected content-length on all requests");
			}

			return CreateFromStreamAsync(request.Body, contentLength.Value, debugPrefix, cancellationToken);
		}

		public async Task<IBufferedPayload> CreateFromStreamAsync(Stream s, long contentLength, string debugPrefix, CancellationToken cancellationToken)
		{
			// blob is small enough to fit into memory we just read it as is
			try
			{
				if (contentLength < _options.CurrentValue.MemoryBufferSize)
				{
					return await MemoryBufferedPayload.CreateAsync(_tracer, s, cancellationToken);
				}
			}
			catch (OutOfMemoryException)
			{
				// if we fail to allocate the memory buffered payload we fallback to backing it on the filesystem
			}

			return await FilesystemBufferedPayload.CreateAsync(_tracer, s, _options.CurrentValue.FilesystemTempPayloadRoot, debugPrefix, cancellationToken);
		}

		public async Task<IBufferedPayload> CreateFilesystemBufferedPayloadAsync(Stream s, string debugPrefix, CancellationToken cancellationToken)
		{
			return await FilesystemBufferedPayload.CreateAsync(_tracer, s, _options.CurrentValue.FilesystemTempPayloadRoot, debugPrefix, cancellationToken);
		}

		public FilesystemBufferedPayloadWriter CreateFilesystemBufferedPayloadWriter(string debugPrefix)
		{
			return FilesystemBufferedPayloadWriter.Create(_options.CurrentValue.FilesystemTempPayloadRoot, debugPrefix);
		}
	}
}
