// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using Microsoft.AspNetCore.Mvc;
using Microsoft.AspNetCore.Mvc.Filters;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	[AttributeUsage(AttributeTargets.Method | AttributeTargets.Class)]
	public sealed class InternalApiFilterAttribute : Attribute, IResourceFilter
	{
		public void OnResourceExecuting(ResourceExecutingContext context)
		{
			IOptionsMonitor<JupiterSettings>? settings = context.HttpContext.RequestServices.GetService<IOptionsMonitor<JupiterSettings>>();

			// if internal port is set to 0 we just assume all traffic is the internal port, used for tests
			bool isInternalPort = settings!.CurrentValue.InternalApiPorts.Contains(context.HttpContext.Connection.LocalPort) ||
			                      settings.CurrentValue.InternalApiPorts.Contains(0);
			if (!isInternalPort)
			{
				// this endpoint should only be exposed on the internal port, so we return a 404 as this is not on the internal port
				context.Result = new NotFoundResult();
			}
		}

		public void OnResourceExecuted(ResourceExecutedContext context)
		{
		}
	}
}
