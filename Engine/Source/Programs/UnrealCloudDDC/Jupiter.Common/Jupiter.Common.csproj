<Project Sdk="Microsoft.NET.Sdk.Web">

  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <ApplicationIcon />
    <OutputType>Library</OutputType>
    <StartupObject />
    <Nullable>enable</Nullable>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <WarningsAsErrors />
    <OpenApiGenerateDocuments>false</OpenApiGenerateDocuments>
    <LangVersion>10</LangVersion>
	
    <NoWarn>$(NoWarn);1591</NoWarn>
	<Configurations>Debug;Release;Analyze</Configurations>
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)|$(Platform)'=='Analyze|AnyCPU'">
    <RunAnalyzersDuringBuild>True</RunAnalyzersDuringBuild>
    <TreatWarningsAsErrors>False</TreatWarningsAsErrors>
    <EnforceCodeStyleInBuild>True</EnforceCodeStyleInBuild>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Blake3" Version="1.1.0" />

    <PackageReference Include="AWSSDK.Extensions.NETCore.Setup" Version="3.7.300" />
    <PackageReference Include="AWSSDK.SecretsManager" Version="3.7.302.8" />
    <PackageReference Include="AWSSDK.SecurityToken" Version="3.7.300.34" />
    <PackageReference Include="Azure.Identity" Version="1.11.4" />
    <PackageReference Include="Azure.Security.KeyVault.Secrets" Version="4.5.0" />
    <PackageReference Include="Newtonsoft.Json" Version="13.0.4" />
    <PackageReference Include="OpenTelemetry" Version="1.14.0" />
    <PackageReference Include="OpenTelemetry.Exporter.Console" Version="1.14.0" />
    <PackageReference Include="OpenTelemetry.Exporter.OpenTelemetryProtocol" Version="1.14.0" />
    <PackageReference Include="OpenTelemetry.Instrumentation.AspNetCore" Version="1.14.0" />
    <PackageReference Include="OpenTelemetry.Instrumentation.Http" Version="1.14.0" />
    <PackageReference Include="OpenTelemetry.Instrumentation.Runtime" Version="1.14.0" />
    <PackageReference Include="OpenTelemetry.Extensions.Hosting" Version="1.14.0" />
    <PackageReference Include="IdentityServer4.AccessTokenValidation" Version="3.0.1" />

    <PackageReference Include="Microsoft.AspNet.WebApi.Client" Version="6.0.0" />
    <PackageReference Include="Microsoft.AspNetCore.Authentication.JwtBearer" Version="10.0.6" />

    <PackageReference Include="Okta.AspNetCore" Version="4.6.1" />
    <!-- Adding explicit version of identity model to workaround deletion of a internal symbol used by Okta.AspNetCore -->
    <PackageReference Include="Microsoft.IdentityModel.Tokens" Version="8.17.0" />
    <PackageReference Include="Microsoft.IdentityModel.Logging" Version="8.17.0" />
    <PackageReference Include="Microsoft.IdentityModel.Protocols" Version="8.17.0" />
    <PackageReference Include="Microsoft.IdentityModel.JsonWebTokens" Version="8.17.0" />

    <!--Support reading yaml config files -->
    <PackageReference Include="NetEscapades.Configuration.Yaml" Version="3.1.0" />
    <PackageReference Include="Serilog.Enrichers.ClientInfo" Version="2.1.1" />
    <PackageReference Include="Serilog.Extensions.Hosting" Version="8.0.0" />
    <PackageReference Include="SharpYaml" Version="2.1.0" />

    <!--Serilog logging -->
    <PackageReference Include="Serilog" Version="3.1.1" />
    <PackageReference Include="Serilog.AspNetCore" Version="8.0.0" />
    <PackageReference Include="Serilog.Formatting.Compact" Version="2.0.0" />
    <PackageReference Include="Serilog.Settings.Configuration" Version="8.0.0" />
    <PackageReference Include="Serilog.Sinks.File" Version="5.0.0" />
    <PackageReference Include="SerilogAnalyzer" Version="0.15.0" />

    <!-- Swagger generation -->
    <PackageReference Include="Swashbuckle.AspNetCore" Version="6.5.0" />
    <PackageReference Include="Swashbuckle.AspNetCore.ReDoc" Version="6.5.0" />

    <PackageReference Include="WebApiContrib.Core.Formatter.Bson" Version="2.1.0" />
    <PackageReference Include="WebApiContrib.Core.Formatter.Yaml" Version="2.1.0" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.AspNet\EpicGames.AspNet.csproj" />
  </ItemGroup>

</Project>
 