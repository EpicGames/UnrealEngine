// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace Jupiter.Common
{
	/// <summary>
	/// Identifier for a storage namespace
	/// </summary>
	[LogValueType]
	[JsonSchemaString]
	[JsonConverter(typeof(NamespaceIdJsonConverter))]
	[TypeConverter(typeof(NamespaceIdTypeConverter))]
	public readonly struct NamespaceId : IEquatable<NamespaceId>
	{
		/// <summary>
		/// The text representing this id
		/// </summary>
		public StringId Text { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Unique id for the namespace</param>
		public NamespaceId(string text)
			: this(new Utf8String(text))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Unique id for the namespace</param>
		public NamespaceId(Utf8String text)
		{
			Text = new StringId(text);
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is NamespaceId id && Text.Equals(id.Text);

		/// <inheritdoc/>
		public override int GetHashCode() => Text.GetHashCode();

		/// <inheritdoc/>
		public bool Equals(NamespaceId other) => Text.Equals(other.Text);

		/// <inheritdoc/>
		public override string ToString() => Text.ToString();

		/// <inheritdoc cref="StringId.op_Equality"/>
		public static bool operator ==(NamespaceId left, NamespaceId right) => left.Text == right.Text;

		/// <inheritdoc cref="StringId.op_Inequality"/>
		public static bool operator !=(NamespaceId left, NamespaceId right) => left.Text != right.Text;
	}

	/// <summary>
	/// Type converter for NamespaceId to and from JSON
	/// </summary>
	sealed class NamespaceIdJsonConverter : JsonConverter<NamespaceId>
	{
		/// <inheritdoc/>
		public override NamespaceId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new NamespaceId(new Utf8String(reader.GetUtf8String().ToArray()));

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, NamespaceId value, JsonSerializerOptions options) => writer.WriteStringValue(value.Text.Span);
	}

	/// <summary>
	/// Type converter from strings to NamespaceId objects
	/// </summary>
	sealed class NamespaceIdTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => new NamespaceId(new Utf8String((string)value));
	}
}
