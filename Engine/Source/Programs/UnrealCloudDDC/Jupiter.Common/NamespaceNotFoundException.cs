// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace Jupiter.Common
{
	public class NamespaceNotFoundException : Exception
	{
		public NamespaceId Namespace { get; }

		public NamespaceNotFoundException(NamespaceId @namespace) : base($"Could not find namespace {@namespace}")
		{
			Namespace = @namespace;
		}

		public NamespaceNotFoundException(NamespaceId @namespace, string message) : base(message)
		{
			Namespace = @namespace;
		}
	}
}
