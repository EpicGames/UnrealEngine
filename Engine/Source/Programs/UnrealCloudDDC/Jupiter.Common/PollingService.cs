// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace Jupiter
{
	public abstract class PollingService<T> : IHostedService, IAsyncDisposable
	{
		private struct ThreadState
		{
			public string ServiceName { get; set; }
			public TimeSpan PollFrequency { get; set; }

			public CancellationToken StopPollingToken { get; set; }
			public T ServiceState { get; set; }
			public PollingService<T> Instance { get; set; }
		}

		private readonly ILogger _logger;
		private readonly string _serviceName;
		private readonly TimeSpan _pollFrequency;
		private readonly T _state;
		private readonly bool _startAtRandomTime;
		private readonly CancellationTokenSource _stopPolling = new CancellationTokenSource();
		private readonly ManualResetEvent _hasFinishedRunning = new ManualResetEvent(true);
		private volatile bool _alreadyRunning = false;
		private Timer? _timer;
		private bool _disposed = false;

		protected PollingService(string serviceName, TimeSpan pollFrequency, T state, ILogger logger, bool startAtRandomTime = false)
		{
			_serviceName = serviceName;
			_pollFrequency = pollFrequency;
			_state = state;
			_logger = logger;
			_startAtRandomTime = startAtRandomTime;

			_timer = new Timer(x => OnUpdate(x, _logger), new ThreadState
			{
				ServiceName = _serviceName,
				PollFrequency = _pollFrequency,
				ServiceState = _state,
				StopPollingToken = _stopPolling.Token,
				Instance = this,
			}, -1, -1);
		}

		public bool Running => _timer != null;

		public T State => _state;

		protected virtual bool ShouldStartPolling()
		{
			return true;
		}

		public async Task StartAsync(CancellationToken cancellationToken)
		{
			bool shouldPoll = ShouldStartPolling();
			_logger.LogInformation("Polling service {Service} initialized, will poll: {WillPoll}.", _serviceName, shouldPoll);

			if (shouldPoll)
			{
				int startOffset = 0;
				if (_startAtRandomTime)
				{
					// start at a random time between now and the poll frequency
					startOffset = Random.Shared.Next(0, (int)_pollFrequency.TotalSeconds);
				}
				_timer?.Change(TimeSpan.FromSeconds(startOffset), _pollFrequency);
			}

			await OnStartingAsync(_state);
		}

		private static void OnUpdate(object? state, ILogger logger)
		{
			ThreadState? ts = (ThreadState?)state;
			if (ts == null)
			{
				throw new ArgumentNullException(nameof(state), "Null thread state passed to polling service");
			}
			ThreadState threadState = ts.Value;

			PollingService<T> instance = threadState.Instance;
			string serviceName = threadState.ServiceName;
			CancellationToken stopPollingToken = threadState.StopPollingToken;

			if (instance._alreadyRunning)
			{
				return;
			}

			try
			{
				instance._alreadyRunning = true;
				instance._hasFinishedRunning.Reset();

				if (stopPollingToken.IsCancellationRequested)
				{
					return;
				}

				bool _ = instance.OnPollAsync(threadState.ServiceState, stopPollingToken).Result;
			}
			catch (AggregateException e)
			{
				bool taskCancelled =
					e.InnerExceptions.Any(exception => exception.GetType() == typeof(TaskCanceledException));
				if (!taskCancelled)
				{
					logger.LogError(e, "{Service} Aggregate exception in polling service", serviceName);
					foreach (Exception inner in e.InnerExceptions)
					{
						logger.LogError(inner, "{Service} inner exception in polling service. Trace: {StackTrace}",
							serviceName, inner.StackTrace);

					}
				}
				else
				{
					logger.LogWarning("{Service} poll cancelled in polling service", serviceName);
				}
			}
			catch (TaskCanceledException)
			{
				logger.LogWarning("{Service} poll cancelled in polling service", serviceName);
			}
			catch (Exception e)
			{
				logger.LogError(e, "{Service} Exception in polling service", serviceName);
				Tracer.CurrentSpan.SetStatus(Status.Error);
				Tracer.CurrentSpan.RecordException(e);
			}
			finally
			{
				instance._alreadyRunning = false;
				instance._hasFinishedRunning.Set();
			}
		}

		public abstract Task<bool> OnPollAsync(T state, CancellationToken cancellationToken);

		protected virtual Task OnStopping(T state)
		{
			return Task.CompletedTask;
		}

		protected virtual Task OnStartingAsync(T state)
		{
			return Task.CompletedTask;
		}

		public async Task StopAsync(CancellationToken cancellationToken)
		{
			_logger.LogInformation("{Service} poll service stopping.", _serviceName);

			if (_timer != null)
			{
				await _timer.DisposeAsync();
			}

			_timer = null;

			await OnStopping(_state);

			await _stopPolling.CancelAsync();
			_hasFinishedRunning.WaitOne();
		}

		public async ValueTask DisposeAsync()
		{
			if (_disposed)
			{
				return;
			}

			_disposed = true;

			await StopAsync(CancellationToken.None);

			_stopPolling.Dispose();
			_hasFinishedRunning.Dispose();

			GC.SuppressFinalize(this);
		}
	}
}
