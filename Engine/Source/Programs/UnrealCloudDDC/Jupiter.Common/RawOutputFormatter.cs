// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;
using System.Text;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.AspNetCore.Mvc.Formatters;
using Microsoft.Extensions.Logging;
using Microsoft.Net.Http.Headers;

namespace Jupiter
{
	public class RawOutputFormatter : OutputFormatter
	{
		private readonly ILogger _logger;

		public RawOutputFormatter(ILogger logger)
		{
			_logger = logger;
			SupportedMediaTypes.Add(MediaTypeHeaderValue.Parse("application/octet-stream"));
		}

		public override async Task WriteResponseBodyAsync(OutputFormatterWriteContext context)
		{
			if (context.Object == null)
			{
				return;
			}

			HttpResponse response = context.HttpContext.Response;

			object o = context.Object;
			PropertyInfo? outputProperty = null;
			foreach (PropertyInfo propertyInfo in o.GetType().GetProperties())
			{
				object[] attr =
					propertyInfo.GetCustomAttributes(attributeType: typeof(RawOutputPropertyAttribute), true);
				if (attr.Length != 0)
				{
					outputProperty = propertyInfo;
				}
			}

			if (o is ProblemDetails p)
			{
				context.ContentType = "text/plain";
				await response.Body.WriteAsync(Encoding.UTF8.GetBytes(p.Title!));
				return;
			}

			if (outputProperty == null)
			{
				context.ContentType = "text/plain";

				_logger.LogDebug("No RawOutputProperty detected, unable to use raw formatter on this object of type {Type}", o.GetType().FullName);
				return;
			}

			if (outputProperty.PropertyType != typeof(byte[]))
			{
				throw new Exception("RawOutputProperty can only be set on a byte[]");
			}

			object? nullableValue = outputProperty.GetValue(o);
			if (nullableValue == null)
			{
				throw new Exception("RawOutputProperty field can not be null");
			}

			byte[] value = (byte[])nullableValue;
			await response.Body.WriteAsync(value);
		}
	}

	/// <summary>
	///  Used to indicate that the field in question is what we want to return as a raw set of bytes
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class RawOutputPropertyAttribute : Attribute
	{
	}
}
