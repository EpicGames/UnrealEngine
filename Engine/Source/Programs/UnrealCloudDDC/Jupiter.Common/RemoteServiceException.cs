// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace Jupiter.Common
{
	public class RemoteServiceException : Exception
	{
		public RemoteServiceException(string message, Exception cause) : base(message, cause)
		{
		}
	}
}
