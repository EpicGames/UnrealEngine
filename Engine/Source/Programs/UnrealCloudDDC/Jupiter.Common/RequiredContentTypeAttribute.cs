// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using Microsoft.AspNetCore.Mvc.ActionConstraints;
using Microsoft.Extensions.Primitives;

namespace Jupiter
{
	[AttributeUsage(AttributeTargets.Method)]
	public sealed class RequiredContentTypeAttribute : Attribute, IActionConstraint
	{
		public string[] MediaTypeNames { get; }

		public RequiredContentTypeAttribute(params string[] mediaTypeNames)
		{
			MediaTypeNames = mediaTypeNames;
		}

		public int Order => 0;

		public bool Accept(ActionConstraintContext context)
		{
			StringValues contentTypeHeader = context.RouteContext.HttpContext.Request.Headers["Content-Type"];

			bool valid = contentTypeHeader.Any(s => MediaTypeNames.Contains(s, StringComparer.InvariantCultureIgnoreCase));
			return valid;
		}
	}
}
