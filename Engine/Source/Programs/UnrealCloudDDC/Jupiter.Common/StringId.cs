// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace Jupiter.Common
{
	/// <summary>
	/// Normalized string identifier for a resource
	/// </summary>
	[JsonSchemaString]
	[JsonConverter(typeof(StringIdJsonConverter))]
	[TypeConverter(typeof(StringIdTypeConverter))]
	public readonly struct StringId : IEquatable<StringId>, IEquatable<string>, IEquatable<ReadOnlyMemory<char>>
	{
		/// <summary>
		/// Enum used to disable validation on string arguments
		/// </summary>
		public enum Validate
		{
			/// <summary>
			/// No validation required
			/// </summary>
			None,
		};

		/// <summary>
		/// Maximum length for a string id
		/// </summary>
		public const int MaxLength = 64;

		/// <summary>
		/// The text representing this id
		/// </summary>
		public Utf8String Text { get; }

		/// <summary>
		/// Accessor for the string bytes
		/// </summary>
		public ReadOnlySpan<byte> Span => Text.Span;

		/// <summary>
		/// Accessor for the string bytes
		/// </summary>
		public ReadOnlyMemory<byte> Memory => Text.Memory;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Unique id for the string</param>
		public StringId(string text)
			: this(new Utf8String(text))
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Unique id for the string</param>
		public StringId(Utf8String text)
		{
			StringId id;
			if (!TryParse(text, out id, out string? errorMessage))
			{
				throw new ArgumentException(errorMessage, nameof(text));
			}
			Text = id.Text;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">Unique id for the string</param>
		/// <param name="validate">Argument used for overload resolution for pre-validated strings</param>
		[SuppressMessage("Style", "IDE0060:Remove unused parameter")]
		public StringId(Utf8String text, Validate validate)
		{
			Text = text;
		}

		/// <summary>
		/// Checks whether this StringId is set
		/// </summary>
		public bool IsEmpty => Text.IsEmpty;

		/// <summary>
		/// Generates a new string id from the given text
		/// </summary>
		/// <param name="text">Text to generate from</param>
		/// <returns>New string id</returns>
		public static StringId Sanitize(string text)
		{
			StringBuilder result = new StringBuilder();
			for (int idx = 0; idx < text.Length && result.Length < MaxLength; idx++)
			{
				char character = (char)text[idx];
				if (character >= 'A' && character <= 'Z')
				{
					result.Append((char)('a' + (character - 'A')));
				}
				else if (character == '.')
				{
					if (result.Length > 0)
					{
						if (result[^1] == '-')
						{
							result[^1] = character;
						}
						else
						{
							result.Append(character);
						}
					}
				}
				else if (IsValidCharacter(character))
				{
					result.Append(character);
				}
				else if (result.Length > 0 && result[^1] != '-' && result[^1] != '.')
				{
					result.Append('-');
				}
			}
			while (result.Length > 0 && (result[^1] == '-' || result[^1] == '.'))
			{
				result.Remove(result.Length - 1, 1);
			}
			return new StringId(new Utf8String(result.ToString()), Validate.None);
		}

		/// <summary>
		/// Validates the given string as a StringId, normalizing it if necessary.
		/// </summary>
		/// <param name="text">Text to validate as a StringId</param>
		/// <param name="id">Parsed identifier</param>
		/// <returns>True if the identifier was parsed</returns>
		public static bool TryParse(string text, out StringId id)
			=> TryParse(new Utf8String(text), out id, out _);

		/// <summary>
		/// Validates the given string as a StringId, normalizing it if necessary.
		/// </summary>
		/// <param name="text">Text to validate as a StringId</param>
		/// <param name="id">Parsed identifier</param>
		/// <param name="errorMessage">Error message if the id cannot be parsed</param>
		/// <returns>True if the identifier was parsed</returns>
		public static bool TryParse(string text, out StringId id, [NotNullWhen(false)] out string? errorMessage)
			=> TryParse(new Utf8String(text), out id, out errorMessage);

		/// <summary>
		/// Validates the given string as a StringId, normalizing it if necessary.
		/// </summary>
		/// <param name="text">Text to validate as a StringId</param>
		/// <param name="id">Parsed identifier</param>
		/// <returns>True if the identifier was parsed</returns>
		public static bool TryParse(Utf8String text, out StringId id)
			=> TryParse(text, out id, out _);

		/// <summary>
		/// Validates the given string as a StringId, normalizing it if necessary.
		/// </summary>
		/// <param name="text">Text to validate as a StringId</param>
		/// <param name="id">Parsed identifier</param>
		/// <param name="errorMessage">Error message if the id cannot be parsed</param>
		/// <returns>True if the identifier was parsed</returns>
		public static bool TryParse(Utf8String text, out StringId id, [NotNullWhen(false)] out string? errorMessage)
		{
			return TryParse(text, MaxLength, out id, out errorMessage);
		}

		/// <summary>
		/// Validates the given string as a StringId, normalizing it if necessary.
		/// </summary>
		/// <param name="text">Text to validate as a StringId</param>
		/// <param name="maxLength">Max length allowed of the string, usually just the constant MaxLength</param>
		/// <param name="id">Parsed identifier</param>
		/// <param name="errorMessage">Error message if the id cannot be parsed</param>
		/// <returns>True if the identifier was parsed</returns>
		public static bool TryParse(Utf8String text, int maxLength, out StringId id, [NotNullWhen(false)] out string? errorMessage)
		{
			if (text.Length > maxLength)
			{
				id = default;
				errorMessage = $"Id '{text}' is longer than {maxLength} characters";
				return false;
			}

			if (text.Length > 0 && (text[0] == '.' || text[^1] == '.'))
			{
				id = default;
				errorMessage = $"'{text}' is not a valid id (cannot start or end with a period)";
				return false;
			}

			for (int idx = 0; idx < text.Length; idx++)
			{
				byte character = text[idx];
				if (!IsValidCharacter(character))
				{
					if (character >= 'A' && character <= 'Z')
					{
						text = ToLower(text);
					}
					else
					{
						id = default;
						errorMessage = $"'{text}' is not a valid id (character '{(char)character}' is not allowed)";
						return false;

					}
				}
			}

			id = new StringId(text, Validate.None);
			errorMessage = null;
			return true;
		}

		/// <summary>
		/// Converts an utf8 string to lowercase
		/// </summary>
		/// <param name="text"></param>
		/// <returns></returns>
		static Utf8String ToLower(Utf8String text)
		{
			byte[] output = new byte[text.Length];
			for (int idx = 0; idx < text.Length; idx++)
			{
				byte character = text[idx];
				if (character >= 'A' && character <= 'Z')
				{
					character = (byte)((character - 'A') + 'a');
				}
				output[idx] = character;
			}
			return new Utf8String(output);
		}

		/// <summary>
		/// Checks whether the given character is valid within a string id
		/// </summary>
		/// <param name="character">The character to check</param>
		/// <returns>True if the character is valid</returns>
		public static bool IsValidCharacter(char character) => character <= 0x7f && IsValidCharacter((byte)character);

		/// <summary>
		/// Checks whether the given character is valid within a string id
		/// </summary>
		/// <param name="character">The character to check</param>
		/// <returns>True if the character is valid</returns>
		public static bool IsValidCharacter(byte character)
		{
			if (character >= 'a' && character <= 'z')
			{
				return true;
			}
			if (character >= '0' && character <= '9')
			{
				return true;
			}
			if (character == '-' || character == '_' || character == '.')
			{
				return true;
			}
			return false;
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is StringId id && Equals(id);

		/// <inheritdoc/>
		public override int GetHashCode() => Text.GetHashCode();

		/// <inheritdoc/>
		public bool Equals(StringId other) => Text.Equals(other.Text);

		/// <inheritdoc/>
		public bool Equals(string? other) => other != null && Equals(other.AsMemory());

		/// <inheritdoc/>
		public bool Equals(ReadOnlyMemory<char> other)
		{
			ReadOnlySpan<char> span = other.Span;
			if (span.Length != Text.Length)
			{
				return false;
			}
			for (int idx = 0; idx < Text.Length; idx++)
			{
				if (span[idx] != Text[idx])
				{
					return false;
				}
			}
			return true;
		}

		/// <inheritdoc/>
		public override string ToString() => Text.ToString();

		/// <summary>
		/// Compares two string ids for equality
		/// </summary>
		/// <param name="left">The first string id</param>
		/// <param name="right">Second string id</param>
		/// <returns>True if the two string ids are equal</returns>
		public static bool operator ==(StringId left, StringId right) => left.Equals(right);

		/// <summary>
		/// Compares two string ids for inequality
		/// </summary>
		/// <param name="left">The first string id</param>
		/// <param name="right">Second string id</param>
		/// <returns>True if the two string ids are not equal</returns>
		public static bool operator !=(StringId left, StringId right) => !left.Equals(right);
	}

	/// <summary>
	/// Class which serializes <see cref="StringId"/> types
	/// </summary>
	sealed class StringIdJsonConverter : JsonConverter<StringId>
	{
		/// <inheritdoc/>
		public override StringId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => new StringId(new Utf8String(reader.GetUtf8String().ToArray()));

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, StringId value, JsonSerializerOptions options) => writer.WriteStringValue(value.Span);
	}

	/// <summary>
	/// Class which serializes <see cref="StringId"/> types
	/// </summary>
	sealed class StringIdTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => new StringId((string)value);

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType) => ((StringId)value!).Text.ToString();
	}
}
