// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Globalization;
using System.Reflection;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Serialization;

namespace Jupiter.Common
{
	/// <summary>
	/// Base class for converting to and from types containing a <see cref="StringId"/>. Useful pattern for reducing boilerplate with strongly typed records.
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public abstract class StringIdConverter<T> where T : struct
	{
		/// <summary>
		/// Converts a type to a <see cref="StringId"/>
		/// </summary>
		public abstract StringId ToStringId(T value);

		/// <summary>
		/// Constructs a type from a <see cref="StringId"/>
		/// </summary>
		public abstract T FromStringId(StringId id);
	}

	/// <summary>
	/// Attribute declaring a <see cref="StringIdConverter{T}"/> for a particular type
	/// </summary>
	[AttributeUsage(AttributeTargets.Struct)]
	public sealed class StringIdConverterAttribute : Attribute
	{
		/// <summary>
		/// The converter type
		/// </summary>
		public Type ConverterType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public StringIdConverterAttribute(Type converterType) => ConverterType = converterType;
	}

	/// <summary>
	/// Converter to compact binary objects
	/// </summary>
	public sealed class StringIdCbConverter<TValue, TConverter> : CbConverter<TValue> where TValue : struct where TConverter : StringIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new TConverter();

		/// <inheritdoc/>
		public override TValue Read(CbField field)
		{
			return _converter.FromStringId(new StringId(new Utf8String(field.AsString())));
		}

		/// <inheritdoc/>
		public override void Write(CbWriter writer, TValue value)
		{
			writer.WriteStringValue(_converter.ToStringId(value).ToString());
		}

		/// <inheritdoc/>
		public override void WriteNamed(CbWriter writer, CbFieldName name, TValue value)
		{
			writer.WriteString(name, _converter.ToStringId(value).ToString());
		}
	}

	/// <summary>
	/// Class which serializes types with a <see cref="StringIdConverter{T}"/> to Json
	/// </summary>
	public sealed class StringIdTypeConverter<TValue, TConverter> : TypeConverter where TValue : struct where TConverter : StringIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new TConverter();

		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string) || sourceType == typeof(StringId);
		}

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string str)
			{
				return _converter.FromStringId(new StringId(new Utf8String(str)));
			}
			if (value is StringId stringId)
			{
				return _converter.FromStringId(stringId);
			}
			return null;
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(string) || destinationType == typeof(StringId);

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return _converter.ToStringId((TValue)value!).ToString();
			}
			if (destinationType == typeof(StringId))
			{
				return _converter.ToStringId((TValue)value!);
			}
			return null;
		}
	}

	/// <summary>
	/// Class which serializes types with a <see cref="StringIdConverter{T}"/> to Json
	/// </summary>
	public sealed class StringIdJsonConverter<TValue, TConverter> : JsonConverter<TValue> where TValue : struct where TConverter : StringIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new TConverter();

		/// <inheritdoc/>
		public override TValue Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
			=> _converter.FromStringId(new StringId(new Utf8String(reader.GetUtf8String().ToArray())));

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, TValue value, JsonSerializerOptions options)
			=> writer.WriteStringValue(_converter.ToStringId(value).Span);

		/// <inheritdoc/>
		public override TValue ReadAsPropertyName(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
			=> Read(ref reader, typeToConvert, options);

		/// <inheritdoc/>
		public override void WriteAsPropertyName(Utf8JsonWriter writer, TValue value, JsonSerializerOptions options)
			=> writer.WritePropertyName(_converter.ToStringId(value).Span);
	}

	/// <summary>
	/// Creates constructors for types with a <see cref="StringIdConverter{T}"/> to Json
	/// </summary>
	public sealed class StringIdJsonConverterFactory : JsonConverterFactory
	{
		/// <inheritdoc/>
		public override bool CanConvert(Type typeToConvert) => typeToConvert.GetCustomAttribute<StringIdConverterAttribute>() != null;

		/// <inheritdoc/>
		public override JsonConverter? CreateConverter(Type type, JsonSerializerOptions options)
		{
			StringIdConverterAttribute? attribute = type.GetCustomAttribute<StringIdConverterAttribute>();
			if (attribute == null)
			{
				return null;
			}
			return (JsonConverter?)Activator.CreateInstance(typeof(StringIdJsonConverter<,>).MakeGenericType(type, attribute.ConverterType));
		}
	}
}
