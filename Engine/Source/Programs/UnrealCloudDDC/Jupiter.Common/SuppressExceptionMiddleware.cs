// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;
using EpicGames.AspNet;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.Logging;

namespace Jupiter.Common
{
	public class SuppressExceptionMiddleware
	{
		private readonly ILogger _logger;
		private readonly RequestDelegate _next;

		public SuppressExceptionMiddleware(RequestDelegate next, ILogger<SuppressExceptionMiddleware> logger)
		{
			_next = next ?? throw new ArgumentNullException(nameof(next));
			_logger = logger;
		}

		public async Task InvokeAsync(HttpContext context)
		{
			try
			{
				await _next(context);
			}
			// suppress certain known exceptions that are not considered errors but that we can not catch in our code
			catch (ClientSendSlowException e)
			{
				// always suppress slow sends
				_logger.LogWarning(e, "Client was sending data slowly");
			}
			catch (TaskCanceledException e)
			{
				// if health checks are cancelled its fine, it means it was called while it was running
				string pathString = context.Request.Path.Value ?? "";

				if (!pathString.StartsWith("/health", StringComparison.OrdinalIgnoreCase))
				{
					// rethrow
					throw;
				}
				else
				{
					_logger.LogWarning(e, "Health check was cancelled. This can happen if multiple health checks happen at the same time");
				}
			}
		}
	}
}
