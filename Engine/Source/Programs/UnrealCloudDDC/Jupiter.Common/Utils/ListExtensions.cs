// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace Jupiter.Common.Utils
{
	public static class ListExtensions
	{
		public static void Shuffle<T>(this IList<T> list)
		{
			// Randomly shuffle the order of a list by swapping members
			// based on Fisher Yates shuffle - https://en.wikipedia.org/wiki/Fisher%E2%80%93Yates_shuffle
			int count = list.Count;
			while (count > 1)
			{
				count--;
				int k = Random.Shared.Next(count + 1);
				T value = list[k];
				list[k] = list[count];
				list[count] = value;
			}
		}
	}
}
