// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.Metrics;
using System.IO;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.DependencyInjection;
using OpenTelemetry.Trace;

namespace Jupiter
{
	public static class RequestUtil
	{
		private static Histogram<double>? s_streamThroughputCounter;
		
		public static async Task<byte[]> ReadRawBodyAsync(HttpRequest request)
		{
			long? contentLength = request.GetTypedHeaders().ContentLength;

			if (contentLength == null)
			{
				throw new Exception("Expected content-length on all raw body requests");
			}

			Tracer? tracer = request.HttpContext.RequestServices.GetService<Tracer>();
			using TelemetrySpan? scope = tracer?.StartActiveSpan("readbody")
				.SetAttribute("operation.name", "readbody")
				.SetAttribute("Content-Length", contentLength.Value);

			await using MemoryStream ms = new MemoryStream((int)contentLength);
			DateTime readStart = DateTime.Now;
			await request.Body.CopyToAsync(ms);
			TimeSpan duration = DateTime.Now - readStart;

			double rate = contentLength.Value / duration.TotalSeconds;

			if (s_streamThroughputCounter == null)
			{
				Meter meter = request.HttpContext.RequestServices.GetService<Meter>()!;
				s_streamThroughputCounter = meter.CreateHistogram<double>("jupiter.stream_throughput");
			}

			s_streamThroughputCounter.Record(rate, new KeyValuePair<string, object?>("sourceIdentifier", "readbody"));

			return ms.GetBuffer();
		}
	}
}
