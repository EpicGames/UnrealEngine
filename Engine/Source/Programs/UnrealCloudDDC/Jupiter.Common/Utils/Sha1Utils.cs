// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Security.Cryptography;
using System.Threading.Tasks;

namespace Jupiter
{
	public static class Sha1Utils
	{
		public static async Task<string> GetSHA1AsStringAsync(byte[] payload)
		{
			return await Task.Run(() => GetSHA1AsString(payload));
		}

		public static byte[] GetSHA1(byte[] payload)
		{
			using SHA1 sha1 = SHA1.Create();
			return sha1.ComputeHash(payload);
		}

		public static byte[] GetSHA1(byte[] payload, int offset, int count)
		{
			using SHA1 sha1 = SHA1.Create();
			return sha1.ComputeHash(payload, offset, count);
		}

		public static string GetSHA1AsString(byte[] payload)
		{
			byte[] sha1 = GetSHA1(payload);

			return StringUtils.FormatAsHexString(sha1);
		}
	}

	public class ByteArrayComparer : EqualityComparer<byte[]>, IComparer<byte[]>
	{
		public override bool Equals(byte[]? a, byte[]? b)
		{
			if (a == null)
			{
				return b == null;
			}

			if (b == null)
			{
				return false;
			}

			if (a.Length != b.Length)
			{
				return false;
			}

			for (int i = 0; i < a.Length; i++)
			{
				if (a[i] != b[i])
				{
					return false;
				}
			}
			return true;
		}
		public override int GetHashCode(byte[] a)
		{
			uint b = 0;
			foreach (byte t in a)
			{
				b = ((b << 23) | (b >> 9)) ^ t;
			}

			return unchecked((int)b);
		}

		public int Compare(byte[]? x, byte[]? y)
		{
			if (x == null)
			{
				return -1;
			}

			if (y == null)
			{
				return 1;
			}

			for (int i = 0; i < x.Length; i++)
			{
				byte a = x[i];
				byte b = y[i];

				int comp = a.CompareTo(b);
				if (comp != 0)
				{
					return comp;
				}
			}

			return 0;
		}
	}
}
