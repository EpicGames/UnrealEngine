// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.AspNet;
using Microsoft.AspNetCore.Http;

namespace Jupiter.Utils
{
	public static class StreamUtils
	{
		public static async Task<byte[]> ToByteArrayAsync(this Stream s, CancellationToken cancellationToken)
		{
			try
			{
				await using MemoryStream ms = new MemoryStream();
				await s.CopyToAsync(ms, cancellationToken);
				return ms.ToArray();
			}
			catch (BadHttpRequestException e)
			{
				ClientSendSlowExceptionUtil.MaybeThrowSlowSendException(e);
				throw;
			}
		}
	}
}
