// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;

namespace Jupiter
{
	public static class StringUtils
	{
		public static int GetHexVal(char hex)
		{
			return hex - (hex < 58 ? 48 : hex < 97 ? 55 : 87);
		}

		public static string FormatAsHexString(byte[] data)
		{
			StringBuilder sb = new StringBuilder(data.Length * 2);
			foreach (byte b in data)
			{
				string hex = b.ToString("X2");
				sb.Append(hex);
			}

			return sb.ToString();
		}

		public static string FormatAsHexLowerString(Span<byte> data)
		{
			StringBuilder sb = new StringBuilder(data.Length * 2);
			foreach (byte b in data)
			{
				string hex = b.ToString("x2");
				sb.Append(hex);
			}

			return sb.ToString();
		}

		public static byte[] ToHashFromHexString(string hex)
		{
			if (hex.Length % 2 == 1)
			{
				throw new Exception("The binary key cannot have an odd number of digits");
			}

			byte[] arr = new byte[hex.Length >> 1];

			for (int i = 0; i < hex.Length >> 1; ++i)
			{
				arr[i] = (byte)((StringUtils.GetHexVal(hex[i << 1]) << 4) + StringUtils.GetHexVal(hex[(i << 1) + 1]));
			}

			return arr;
		}
	}
}
