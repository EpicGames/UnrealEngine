// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;

namespace Jupiter
{
	public static class TaskUtils
	{
		public static byte[] CombineIntoSingleBuffer(List<byte[]> tasks)
		{
			if (tasks.Count == 1)
			{
				return tasks[0];
			}

			int totalLength = tasks.Sum(task => task.Length);
			byte[] buffer = new byte[totalLength];
			int index = 0;
			foreach (byte[] partialObject in tasks)
			{
				Array.Copy(partialObject, 0, buffer, index, partialObject.Length);
				index += partialObject.Length;
			}

			return buffer;
		}
	}
}
