// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.CompilerServices;

[assembly: InternalsVisibleTo("Jupiter.Tests.Functional")]
[assembly: InternalsVisibleTo("Jupiter.Tests.Unit")]
