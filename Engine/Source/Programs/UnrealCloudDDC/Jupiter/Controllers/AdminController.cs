// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Implementation;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Configuration;

namespace Jupiter.Controllers
{
	[ApiController]
	[Route("api/v1/admin")]
	[Authorize]
	public class AdminController : Controller
	{
		private readonly IRefCleanup _refCleanup;
		private readonly IConfiguration _configuration;
		private readonly IRequestHelper _requestHelper;

		public AdminController(IRefCleanup refCleanup, IConfiguration configuration, IRequestHelper requestHelper)
		{
			_refCleanup = refCleanup;
			_configuration = configuration;
			_requestHelper = requestHelper;
		}
		
		/// <summary>
		/// Manually run the refs cleanup
		/// </summary>
		/// <remarks>
		/// Manually triggers a cleanup of the refs keys based on last access time. This is done automatically so the only reason to use this endpoint is for debugging purposes.
		/// </remarks>
		/// <returns></returns>
		[HttpPost("refCleanup")]
		public async Task<IActionResult> RefCleanupAsync()
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, AccessScope.GlobalScope, new[] { JupiterAclAction.AdminAction });
			if (result != null)
			{	
				return result;
			}

			int countOfDeletedRecords = await _refCleanup.Cleanup(CancellationToken.None);
			return Ok(new RemovedRefRecordsResponse(countOfDeletedRecords));
		}

		/// <summary>
		/// Dumps all settings currently in use
		/// </summary>
		/// <returns></returns>
		[HttpGet("settings")]
		public async Task<IActionResult> SettingsAsync()
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, AccessScope.GlobalScope, new[] { JupiterAclAction.AdminAction });
			if (result != null)
			{
				return result;
			}

			Dictionary<string, Dictionary<string, object>> settings = new Dictionary<string, Dictionary<string, object>>();

			Dictionary<string, object> ResolveSection(IConfigurationSection section)
			{
				Dictionary<string, object> values = new Dictionary<string, object>();
				foreach (IConfigurationSection childSection in section.GetChildren())
				{
					if (childSection.Value == null)
					{
						values.Add(childSection.Key, ResolveSection(childSection));
					}
					else
					{
						values.Add(childSection.Key, childSection.Value);
					}
				}

				return values;
			}

			foreach (IConfigurationSection section in _configuration.GetChildren())
			{
				Dictionary<string, object> values = ResolveSection(section);
				if (values.Count != 0)
				{
					settings.Add(section.Key, values);
				}
			}

			return new JsonResult(new
			{
				Settings = settings
			}, new System.Text.Json.JsonSerializerOptions
			{
				WriteIndented = true
			})
			{
				StatusCode = (int)HttpStatusCode.OK
			};
		}
	}

	public class RemovedBlobRecords
	{
		public BlobId[] Blobs { get; }

		public RemovedBlobRecords(IEnumerable<BlobId> blobs)
		{
			Blobs = blobs.ToArray();
		}
	}

	public class RemovedRefRecordsResponse
	{
		public RemovedRefRecordsResponse(int countOfRemovedRecords)
		{
			CountOfRemovedRecords = countOfRemovedRecords;
		}

		public int CountOfRemovedRecords { get; }
	}

	public class UpdatedRecordsResponse
	{
		public UpdatedRecordsResponse(List<UpdatedRecord> updatedRecords)
		{
			UpdatedRecords = updatedRecords;
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1034:Nested types should not be visible", Justification = "Only used by serialization")]
		public class UpdatedRecord
		{
			public UpdatedRecord(LastAccessRecord record, DateTime time)
			{
				Record = record;
				Time = time;
			}

			public LastAccessRecord Record { get; }
			public DateTime Time { get; }
		}

		public List<UpdatedRecord> UpdatedRecords { get; }
	}
}
