// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.IO;
using System.Linq;
using System.Net.Mime;
using System.Security.Claims;
using System.Security.Cryptography;
using System.Text;
using System.Text.Json;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Serialization;
using Jupiter.Common;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Controllers
{
	[ApiController]
	[Route("api/v1/auth")]
	[Authorize]
	public class AuthController : ControllerBase
	{
		private readonly IRequestHelper _requestHelper;
		private readonly IOptionsMonitor<AuthSettings> _authSettings;

		public AuthController(IRequestHelper requestHelper, IOptionsMonitor<AuthSettings> authSettings)
		{
			_requestHelper = requestHelper;
			_authSettings = authSettings;
		}

		[HttpGet("oidc-configuration")]
		// disable authentication on this endpoint
		[AllowAnonymous]
		// this endpoint always produces encrypted json
		[Produces(MediaTypeNames.Application.Octet)]
		public async Task<ActionResult> GetConfigAsync()
		{
			if (_authSettings.CurrentValue.ClientOidcConfiguration == null)
			{
				return BadRequest();
			}

			byte[] b = JsonSerializer.SerializeToUtf8Bytes(_authSettings.CurrentValue.ClientOidcConfiguration);

			using Aes aes = Aes.Create();
			byte[] key = Convert.FromHexString(_authSettings.CurrentValue.ClientOidcEncryptionKey);
			aes.Key = key;
			aes.GenerateIV();
			// write the IV into the stream before the encrypted content
			Stream responseStream = new MemoryStream();
			await responseStream.WriteAsync(aes.IV);
			await using CryptoStream cryptoStream = new(responseStream, aes.CreateEncryptor(), CryptoStreamMode.Write);
			await cryptoStream.WriteAsync(b, 0, b.Length);
			await cryptoStream.FlushFinalBlockAsync();

			Response.ContentType = MediaTypeNames.Application.Octet;
			Response.StatusCode = 200;
			Response.ContentLength = responseStream.Length;
			responseStream.Position = 0;
			await responseStream.CopyToAsync(Response.Body);
			return new EmptyResult();
		}

		[HttpGet("{ns}")]
		public async Task<IActionResult> VerifyAsync([FromRoute][Required] NamespaceId ns)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			return Ok();
		}

		[HttpGet("{ns}/actions")]
		public IActionResult Actions([FromRoute][Required] NamespaceId ns)
		{
			List<JupiterAclAction> allowedActions = new List<JupiterAclAction>();
			
			CaptureLogger captureLogger = new CaptureLogger();
			bool policiesFound = false;
			List<MatchedPolicy> matchedPolicies = new();
			foreach (AclPolicy acl in _authSettings.CurrentValue.Policies)
			{
				policiesFound = true;
				List<JupiterAclAction> grantedActions = acl.Resolve(User, new AccessScope(ns), captureLogger, out List<string> matchedClaims).ToList();
				allowedActions.AddRange(grantedActions);

				MatchedPolicy matchedPolicy = new MatchedPolicy { Actions = grantedActions, Claims = matchedClaims };
				matchedPolicies.Add(matchedPolicy);
			}

			if (!policiesFound)
			{
				captureLogger.LogWarning("No policies set so no actions generated");
				return BadRequest(new ActionsResult {Actions = allowedActions, LogOutput = captureLogger.RenderLines(), MatchedPolicies = matchedPolicies });
			}

			return Ok(new ActionsResult { Actions = allowedActions , LogOutput = captureLogger.RenderLines(), MatchedPolicies = matchedPolicies});
		}
		
		[HttpGet("{ns}/{bucket}/actions")]
		public IActionResult Actions([FromRoute][Required] NamespaceId ns, [FromRoute][Required] BucketId bucket, [FromQuery] string? claims = null)
		{
			List<JupiterAclAction> allowedActions = new List<JupiterAclAction>();

			ClaimsPrincipal principal = User;
			if (claims != null)
			{
				string s = Encoding.ASCII.GetString(Convert.FromBase64String(claims));
				JsonDocument doc = JsonDocument.Parse(s);
				List<Claim> claimList = new();
				foreach(JsonProperty prop in doc.RootElement.EnumerateObject())
				{
					if (prop.Value.ValueKind == JsonValueKind.String)
					{
						claimList.Add(new Claim(prop.Name, prop.Value.GetString()!));
					} 
					else if (prop.Value.ValueKind == JsonValueKind.Array)
					{
						foreach (JsonElement element in prop.Value.EnumerateArray())
						{
							claimList.Add(new Claim(prop.Name, element.GetString()!));
						}
					}
					else
					{
						throw new NotImplementedException("Unknown json property type when attempting to convert it into a claim");
					}
				}
				principal = new ClaimsPrincipal(new ClaimsIdentity(claimList));
			}
			List<MatchedPolicy> matchedPolicies = new();
			CaptureLogger captureLogger = new CaptureLogger();
			bool policiesFound = false;
			foreach (AclPolicy acl in _authSettings.CurrentValue.Policies)
			{
				policiesFound = true;
				List<JupiterAclAction> grantedActions = acl.Resolve(principal, new AccessScope(ns, bucket), captureLogger, out List<string> matchedClaims).ToList();
				allowedActions.AddRange(grantedActions);

				MatchedPolicy matchedPolicy = new MatchedPolicy { Name = acl.Name, Actions = grantedActions, Claims = matchedClaims };
				matchedPolicies.Add(matchedPolicy);
			}

			if (!policiesFound)
			{
				captureLogger.LogWarning("No policies set so no actions generated");
				return BadRequest(new ActionsResult { Actions = allowedActions, LogOutput = captureLogger.RenderLines(), MatchedPolicies = matchedPolicies });
			}

			return Ok(new ActionsResult { Actions = allowedActions , LogOutput = captureLogger.RenderLines(), MatchedPolicies = matchedPolicies});
		}
	}

	public class ActionsResult
	{
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		[CbField("actions")]
		public List<JupiterAclAction> Actions { get; set; } = new List<JupiterAclAction>();

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		[CbField("logOutput")]
		public List<string> LogOutput { get; set; } = new List<string>();

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		[CbField("matchedPolicies")]
		public List<MatchedPolicy> MatchedPolicies { get; set; } = new List<MatchedPolicy>();
	}

	public class MatchedPolicy
	{
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		[CbField("actions")] 
		public List<JupiterAclAction> Actions { get; set; } = new List<JupiterAclAction>();

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		[CbField("claims")]
		public List<string> Claims { get; set; } = new List<string>();

		[CbField("name")]
		public string Name { get; set; } = null!;
	}
}
