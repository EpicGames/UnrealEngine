// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Mime;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Serialization;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Implementation;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;

namespace Jupiter.Controllers
{
	using BlobNotFoundException = Jupiter.Implementation.BlobNotFoundException;
	using IDiagnosticContext = Serilog.IDiagnosticContext;

	[ApiController]
	[Route("api/v1/s", Order = 1)]
	[Route("api/v1/blobs", Order = 0)]
	[Authorize]
	public class BlobsController : ControllerBase
	{
		private readonly IBlobService _storage;
		private readonly IDiagnosticContext _diagnosticContext;
		private readonly IRequestHelper _requestHelper;
		private readonly BufferedPayloadFactory _bufferedPayloadFactory;
		private readonly NginxRedirectHelper _nginxRedirectHelper;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;

		public BlobsController(IBlobService storage, IDiagnosticContext diagnosticContext, IRequestHelper requestHelper, BufferedPayloadFactory bufferedPayloadFactory, NginxRedirectHelper nginxRedirectHelper, INamespacePolicyResolver namespacePolicyResolver)
		{
			_storage = storage;
			_diagnosticContext = diagnosticContext;
			_requestHelper = requestHelper;
			_bufferedPayloadFactory = bufferedPayloadFactory;
			_nginxRedirectHelper = nginxRedirectHelper;
			_namespacePolicyResolver = namespacePolicyResolver;
		}

		[HttpGet("{ns}/{id}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetAsync(
			[Required] NamespaceId ns,
			[Required] BlobId id,
			[FromQuery] List<string>? storageLayers = null,
			[FromQuery] bool supportsRedirect = false,
			[FromQuery] bool allowOndemandReplication = true)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				BlobContents blobContents = await GetImplAsync(ns, id, storageLayers, supportsRedirectUri: supportsRedirect, allowOndemandReplication: allowOndemandReplication);

				if (blobContents.RedirectUri != null)
				{
					return Redirect(blobContents.RedirectUri.ToString());
				}
				if (_nginxRedirectHelper.CanRedirect(Request, blobContents))
				{
					return _nginxRedirectHelper.CreateActionResult(blobContents, MediaTypeNames.Application.Octet);
				}
				return File(blobContents.Stream, MediaTypeNames.Application.Octet, enableRangeProcessing: true);
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Blob {e.Blob} not found" });
			}
			catch (AuthorizationException e)
			{
				return e.Result;
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpHead("{ns}/{id}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> HeadAsync(
			[Required] NamespaceId ns,
			[Required] BlobId id,
			[FromQuery] List<string>? storageLayers = null)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				bool exists = await _storage.ExistsAsync(ns, id, storageLayers);

				if (!exists)
				{
					return NotFound(new ValidationProblemDetails { Title = $"Blob {id} not found" });
				}
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return Ok();
		}

		[HttpPost("{ns}/exists")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> ExistsMultipleAsync(
			[Required] NamespaceId ns,
			[Required][FromQuery] List<BlobId> id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			ConcurrentBag<BlobId> missingBlobs = new ConcurrentBag<BlobId>();

			IEnumerable<Task> tasks = id.Select(async blob =>
			{
				if (!await _storage.ExistsAsync(ns, blob))
				{
					missingBlobs.Add(blob);
				}
			});
			await Task.WhenAll(tasks);

			return Ok(new HeadMultipleResponse { Needs = missingBlobs.ToArray() });
		}

		[HttpPost("{ns}/exist")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> ExistsBodyAsync(
			[Required] NamespaceId ns,
			[FromBody] BlobId[] bodyIds)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			ConcurrentBag<BlobId> missingBlobs = new ConcurrentBag<BlobId>();

			IEnumerable<Task> tasks = bodyIds.Select(async blob =>
			{
				if (!await _storage.ExistsAsync(ns, blob))
				{
					missingBlobs.Add(blob);
				}
			});
			await Task.WhenAll(tasks);

			return Ok(new HeadMultipleResponse { Needs = missingBlobs.ToArray() });
		}

		private async Task<BlobContents> GetImplAsync(NamespaceId ns, BlobId blob, List<string>? storageLayers = null, bool supportsRedirectUri = false, bool allowOndemandReplication = true)
		{
			try
			{
				return await _storage.GetObjectAsync(ns, blob, storageLayers, supportsRedirectUri: supportsRedirectUri, allowOndemandReplication: allowOndemandReplication, bucketHint: null);
			}
			catch (BlobNotFoundException)
			{
				if (!_storage.ShouldFetchBlobOnDemand(ns) || !allowOndemandReplication)
				{
					throw;
				}

				return await _storage.ReplicateObjectAsync(ns, blob);
			}
		}

		[HttpPut("{ns}/{id}")]
		[RequiredContentType(MediaTypeNames.Application.Octet)]
		[DisableRequestSizeLimit]
		public async Task<IActionResult> PutAsync(
			[Required] NamespaceId ns,
			[Required] BlobId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);

			try
			{
				bool? bypassCache = _namespacePolicyResolver.GetPoliciesForNs(ns).BypassCacheOnWrite;

				Uri? uri = await _storage.MaybePutObjectWithRedirectAsync(ns, id, bucketHint: null, cancellationToken: HttpContext.RequestAborted);
				if (uri != null)
				{
					return Ok(new BlobUploadUriResponse(id, uri));
				}
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "blob-put", HttpContext.RequestAborted);

				BlobId identifier = await _storage.PutObjectAsync(ns, payload, id, bucketHint: null, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);
				return Ok(new BlobUploadResponse(identifier));
			}
			catch (ResourceHasToManyRequestsException)
			{
				return StatusCode(StatusCodes.Status429TooManyRequests);
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPost("{ns}")]
		[RequiredContentType(MediaTypeNames.Application.Octet)]
		[DisableRequestSizeLimit]
		public async Task<IActionResult> PostAsync(
			[Required] NamespaceId ns)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);
			try
			{
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "blob-post", HttpContext.RequestAborted);
				bool? bypassCache = _namespacePolicyResolver.GetPoliciesForNs(ns).BypassCacheOnWrite;

				await using Stream stream = payload.GetStream();

				BlobId id = await BlobId.FromStreamAsync(stream, HttpContext.RequestAborted);
				await _storage.PutObjectKnownHashAsync(ns, payload, id, bucketHint: null, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);

				return Ok(new BlobUploadResponse(id));
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpDelete("{ns}/{id}")]
		public async Task<IActionResult> DeleteAsync(
			[Required] NamespaceId ns,
			[Required] BlobId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.DeleteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				await DeleteImplAsync(ns, id);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return NoContent();
		}

		[HttpDelete("{ns}")]
		public async Task<IActionResult> DeleteNamespaceAsync(
			[Required] NamespaceId ns)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.DeleteNamespace });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				await _storage.DeleteNamespaceAsync(ns, HttpContext.RequestAborted);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return NoContent();
		}

		private async Task DeleteImplAsync(NamespaceId ns, BlobId id)
		{
			await _storage.DeleteObjectAsync(ns, id, HttpContext.RequestAborted);
		}

		// ReSharper disable UnusedAutoPropertyAccessor.Global
		// ReSharper disable once ClassNeverInstantiated.Global
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1034:Nested types should not be visible", Justification = "Used by serialization")]
		public class BatchOp
		{
			// ReSharper disable once InconsistentNaming
			public enum Operation
			{
				INVALID,
				GET,
				PUT,
				DELETE,
				HEAD
			}

			[Required] public NamespaceId? Namespace { get; set; }

			public BlobId? Id { get; set; }

			[Required] public Operation Op { get; set; }

			public byte[]? Content { get; set; }
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1034:Nested types should not be visible", Justification = "Used by serialization")]
		public class BatchCall
		{
			public BatchOp[]? Operations { get; set; }
		}
		// ReSharper restore UnusedAutoPropertyAccessor.Global

		[HttpPost("")]
		public async Task<IActionResult> PostAsync([FromBody] BatchCall batch)
		{
			JupiterAclAction MapToAclAction(BatchOp.Operation op)
			{
				switch (op)
				{
					case BatchOp.Operation.GET:
					case BatchOp.Operation.HEAD:
						return JupiterAclAction.ReadObject;
					case BatchOp.Operation.PUT:
						return JupiterAclAction.WriteObject;
					case BatchOp.Operation.DELETE:
						return JupiterAclAction.DeleteObject;
					default:
						throw new ArgumentOutOfRangeException(nameof(op), op, null);
				}
			}

			if (batch?.Operations == null)
			{
				throw new();
			}

			Task<object?>[] tasks = new Task<object?>[batch.Operations.Length];
			for (int index = 0; index < batch.Operations.Length; index++)
			{
				BatchOp op = batch.Operations[index];
				if (op.Namespace == null)
				{
					throw new Exception(nameof(op.Namespace));
				}

				ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(op.Namespace.Value), new[] { MapToAclAction(op.Op) });

				if (result != null)
				{
					return result;
				}

				switch (op.Op)
				{
					case BatchOp.Operation.INVALID:
						throw new InvalidOperationException($"Op type {BatchOp.Operation.INVALID} is not a valid op type for operation id {op.Id}");
					case BatchOp.Operation.GET:
						if (op.Id == null)
						{
							return BadRequest();
						}

						tasks[index] = GetImplAsync(op.Namespace.Value, op.Id).ContinueWith((t, _) =>
						{
							// TODO: This is very allocation heavy but given that the end result is a json object we can not really stream this anyway
							using BlobContents blobContents = t.Result;

							using MemoryStream ms = new MemoryStream();
							blobContents.Stream.CopyTo(ms);
							ms.Seek(0, SeekOrigin.Begin);
							string str = Convert.ToBase64String(ms.ToArray());
							return (object?)str;
						}, null, TaskScheduler.Current);
						break;
					case BatchOp.Operation.HEAD:
						if (op.Id == null)
						{
							return BadRequest();
						}

						tasks[index] = _storage.ExistsAsync(op.Namespace.Value, op.Id)
							.ContinueWith((t, _) => t.Result ? (object?)null : op.Id, null, TaskScheduler.Current);
						break;
					case BatchOp.Operation.PUT:
						{
							if (op.Content == null)
							{
								return BadRequest();
							}

							if (op.Id == null)
							{
								return BadRequest();
							}

							using MemoryBufferedPayload payload = new MemoryBufferedPayload(op.Content);
							tasks[index] = _storage.PutObjectAsync(op.Namespace.Value, payload, op.Id, bucketHint: null, cancellationToken: HttpContext.RequestAborted).ContinueWith((t, _) => (object?)t.Result, null, TaskScheduler.Current);
							break;
						}
					case BatchOp.Operation.DELETE:
						if (op.Id == null)
						{
							return BadRequest();
						}

						tasks[index] = DeleteImplAsync(op.Namespace.Value, op.Id).ContinueWith((t, _) => (object?)null, null, TaskScheduler.Current);
						break;
					default:
						throw new NotImplementedException($"{op.Op} is not a support op type");
				}
			}

			await Task.WhenAll(tasks);

			object?[] results = tasks.Select(t => t.Result).ToArray();

			return Ok(results);
		}
	}

	public class BlobUploadResponse
	{
		public BlobUploadResponse()
		{
			Identifier = null!;
		}

		public BlobUploadResponse(BlobId identifier)
		{
			Identifier = identifier;
		}

		[CbField("identifier")]
		public BlobId Identifier { get; set; }
	}

	public class BlobUploadUriResponse
	{
		public BlobUploadUriResponse()
		{
			Identifier = null!;
		}

		public BlobUploadUriResponse(BlobId identifier, Uri redirectUri)
		{
			Identifier = identifier;
			RedirectUri = redirectUri;
		}

		[CbField("identifier")]
		public BlobId Identifier { get; set; }

		[CbField("redirectUri")]
		public Uri? RedirectUri { get; set; } = null;
	}

	public class HeadMultipleResponse
	{
		[CbField("needs")]
		public BlobId[] Needs { get; set; } = null!;
	}
}
