// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Diagnostics;
using System.Diagnostics.Metrics;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Mime;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Core;
using EpicGames.Serialization;
using EpicGames.Serialization.Converters;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Implementation;
using Jupiter.Implementation.Blob;
using Jupiter.Implementation.Builds;
using Jupiter.Utils;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Http.Extensions;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Microsoft.Extensions.Primitives;
using OpenTelemetry.Trace;
using Serilog;
using CbField = EpicGames.Serialization.CbField;
using ContentHash = Jupiter.Implementation.ContentHash;
using ILogger = Serilog.ILogger;
using NotImplementedException = System.NotImplementedException;

namespace Jupiter.Controllers
{
	
	[ApiController]
	[Route("api/v2/builds", Order = 0)]
	[Authorize]
	[Produces(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
	[FormatFilter]
	public class BuildsController : ControllerBase
	{
		private readonly IRefService _refService;
		private readonly IBlobService _blobStore;
		private readonly IBlockStore _blockStore;
		private readonly IDiagnosticContext _diagnosticContext;
		private readonly FormatResolver _formatResolver;
		private readonly BufferedPayloadFactory _bufferedPayloadFactory;
		private readonly NginxRedirectHelper _nginxRedirectHelper;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly IRequestHelper _requestHelper;
		private readonly Tracer _tracer;
		private readonly ILogger<BuildsController> _logger;
		private readonly IBuildStore _buildStore;
		private readonly IBlobIndex _blobIndex;
		private readonly IReferenceResolver _referenceResolver;
		private readonly ILogger? _auditLogger;

		/// <summary>
		/// Constant used for access control of parts
		/// </summary>
		private const string BuildPartType = "parts";

		public BuildsController(IRefService refService, IBlobService blobStore, IBlockStore blockStore, IBuildStore buildStore, IBlobIndex blobIndex, IReferenceResolver referenceResolver, IDiagnosticContext diagnosticContext, FormatResolver formatResolver, BufferedPayloadFactory bufferedPayloadFactory, NginxRedirectHelper nginxRedirectHelper, INamespacePolicyResolver namespacePolicyResolver, IRequestHelper requestHelper, Tracer tracer, ILogger<BuildsController> logger, IOptionsMonitor<BuildsSettings> settings)
		{
			_refService = refService;
			_blobStore = blobStore;
			_blockStore = blockStore;
			_buildStore = buildStore;
			_blobIndex = blobIndex;
			_referenceResolver = referenceResolver;
			_diagnosticContext = diagnosticContext;
			_formatResolver = formatResolver;
			_bufferedPayloadFactory = bufferedPayloadFactory;
			_nginxRedirectHelper = nginxRedirectHelper;
			_namespacePolicyResolver = namespacePolicyResolver;
			_requestHelper = requestHelper;
			_tracer = tracer;
			_logger = logger;
			_auditLogger = settings.CurrentValue.EnableAuditLog ? Serilog.Log.ForContext("LogType", "Audit") : null;
		}

		#region Build endpoints
		[HttpPut("{ns}/{bucket}/{buildId}.{format?}")]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> PutBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			LogAuditEntry(HttpMethod.Put, nameof(PutBuildAsync), ns, bucket, buildId, null);

			CbObject buildObject;
			try
			{
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-build", HttpContext.RequestAborted);

				switch (Request.ContentType)
				{
					 case MediaTypeNames.Application.Json:
						{
							await using MemoryStream ms = new MemoryStream();
							await using Stream payloadStream = payload.GetStream();
							await payloadStream.CopyToAsync(ms);
							byte[] b = ms.ToArray();
							string s = Encoding.UTF8.GetString(b);
							buildObject = CbObject.FromJson(s);
							break;
						}
					case CustomMediaTypeNames.UnrealCompactBinary:
						{
							await using MemoryStream ms = new MemoryStream();
							await using Stream payloadStream = payload.GetStream();
							await payloadStream.CopyToAsync(ms);
							buildObject = new CbObject(ms.ToArray());
							break;
						}
					default:
						throw new Exception($"Unknown request type {Request.ContentType}");
				}
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}

			try
			{
				// verify that all the required fields needed to construct a build context are present
				BlockContext _ = BlockContext.FromObject(buildObject);
			}
			catch (RequiredFieldMissingException e)
			{
				return BadRequest(e.Message);
			}

			// copy the cb object to a writer so that we can append useful metadata fields
			CbWriter newObjectWriter = new CbWriter();
			newObjectWriter.BeginObject();
			foreach (CbField field in buildObject)
			{
				newObjectWriter.WriteField(field);
			}
			uint idealChunkSize = _blobStore.GetMultipartLimits(ns)?.IdealChunkSize ?? DefaultChunkSize;
			newObjectWriter.WriteInteger("chunkSize", idealChunkSize);
			newObjectWriter.EndObject();
			CbObject mutatedObject = newObjectWriter.ToObject();

			uint ttl = (uint)_namespacePolicyResolver.GetPoliciesForNs(ns).DefaultTTL.TotalSeconds;

			try
			{
				await _buildStore.PutBuildAsync(ns, bucket, buildId, mutatedObject, ttl);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
			
			return Ok(new PutBuildResponse {ChunkSize = idealChunkSize});
		}

		private const int DefaultChunkSize = 32 * 1024 * 1024; // Default to 32 MB chunks

		[HttpPost("{ns}/{bucket}/{buildId}/finalize")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> FinalizeBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			uint ttl = (uint)_namespacePolicyResolver.GetPoliciesForNs(ns).DefaultTTL.TotalSeconds;

			try
			{
				await foreach ((string partName, CbObjectId partId) in _buildStore.GetBuildPartsAsync(ns, bucket, buildId))
				{
					ActionResult? accessResultPart = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket, BuildPartType, partName), new[] { JupiterAclAction.WriteBuildPart });
					if (accessResultPart != null)
					{
						return accessResultPart;
					}

					RefId refId = RefId.FromName($"{buildId}/{partId}");
					try
					{
						(RefRecord refRecord, BlobContents? _)  = await _refService.GetAsync(ns, bucket, refId);
						if (!refRecord.IsFinalized)
						{
							return BadRequest($"Build part {partId} ({refId}) for build {buildId} in bucket {bucket} was not finalized. Build can not be finalized.");
						}
					}
					catch (RefNotFoundException)
					{
						return BadRequest($"Build part {partId} ({refId}) for build {buildId} in bucket {bucket} did not exist. Build can not be finalized.");
					}
				}
				
				await _buildStore.FinalizeBuildAsync(ns, bucket, buildId, ttl);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
			
			return Ok();
		}

		[HttpGet("{ns}/{bucket}/{buildId}/ttl")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetTTLAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			try
			{
				uint? ttl = await _buildStore.GetTTL(ns, bucket, buildId);

				if (!ttl.HasValue)
				{
					return NotFound();
				}
				return Ok(new GetTTLResponse { TTL = ttl.Value});
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPost("{ns}/{bucket}/{buildId}/updateTTL")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> UpdateTTLAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, [FromBody] UpdateTTLRequest request)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.AdminAction });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			LogAuditEntry(HttpMethod.Post, nameof(UpdateTTLAsync), ns, bucket, buildId, null);

			try
			{
				await foreach ((string _, CbObjectId partId) in _buildStore.GetBuildPartsAsync(ns, bucket, buildId))
				{
					// we do not access control updating the individual part here as it would cause inconsistencies if some parts were updated but others weren't

					RefId refId = RefId.FromName($"{buildId}/{partId}");
					await _refService.UpdateTTL(ns, bucket, refId, request.TTL);
				}

				await _buildStore.UpdateTTL(ns, bucket, buildId, request.TTL);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return Ok();
		}

		[HttpDelete("{ns}/{bucket}/{buildId}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> DeleteBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.DeleteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			LogAuditEntry(HttpMethod.Delete, nameof(DeleteBuildAsync), ns, bucket, buildId, null);

			try
			{
				await foreach ((string _, CbObjectId partId) in _buildStore.GetBuildPartsAsync(ns, bucket, buildId))
				{
					// we do not explicitly access control the deletion of the build part, makes no sense as you can not delete just individual parts instead you delete the whole build

					RefId refId = RefId.FromName($"{buildId}/{partId}");
					await _refService.DeleteAsync(ns, bucket, refId);
				}

				await foreach (string appendixName in _buildStore.GetBuildAppendixesAsync(ns, bucket, buildId))
				{
					RefId refId = RefId.FromName($"{buildId}/appendix/{appendixName}");
					await _refService.DeleteAsync(ns, bucket, refId);
				}

				await _buildStore.DeleteBuild(ns, bucket, buildId);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return Ok();
		}

		[HttpGet("{ns}/{bucket}/{buildId}.{format?}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string? format = null)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			LogAuditEntry(HttpMethod.Get, nameof(GetBuildAsync), ns, bucket, buildId, null);

			try
			{
				BuildRecord? buildRecord = await GetBuildWithRetriesAsync(ns, bucket, buildId);

				if (buildRecord == null)
				{
					return NotFound($"Build \"{buildId}\" in bucket {bucket} and namespace {ns}");
				}

				// write a new cb object because we want to append the parts field to it for the build parts
				CbWriter newObjectWriter = new CbWriter();
				newObjectWriter.BeginObject();
				foreach (CbField field in buildRecord.BuildObject)
				{
					newObjectWriter.WriteField(field);
				}

				// inject the build parts into the object
				// we know the object fields will all have the same type so make it a uniform object to please compact binary validation
				newObjectWriter.BeginUniformObject("parts", CbFieldType.ObjectId | CbFieldType.HasFieldName);
				bool hadAccessToPart = false;
				await foreach ((string partName, CbObjectId partId) in _buildStore.GetBuildPartsAsync(ns, bucket, buildId))
				{
					ActionResult? accessResultPart = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket, BuildPartType, partName), new[] { JupiterAclAction.ReadBuildPart });
					if (accessResultPart == null)
					{
						hadAccessToPart = true;
						// only include the build part if this request has access to it
						newObjectWriter.WriteObjectId(partName, partId);
					}
				}
				newObjectWriter.EndObject(); // parts
				newObjectWriter.EndObject(); // root
				CbObject newObject = newObjectWriter.ToObject();

				if (!hadAccessToPart)
				{
					// user did not have access to any part, that means they had no access to this at all
					return new ForbidResult();
				}

				string responseType = _formatResolver.GetResponseType(Request, format, CustomMediaTypeNames.UnrealCompactBinary);
				Tracer.CurrentSpan.SetAttribute("response-type", responseType);

				switch (responseType)
				{
					case CustomMediaTypeNames.UnrealCompactBinary:
					{
						Response.ContentType = CustomMediaTypeNames.UnrealCompactBinary;
						Response.StatusCode = 200;
						await Response.Body.WriteAsync(newObject.GetView());
						return new EmptyResult();
					}
					case MediaTypeNames.Application.Json:
					{
						string s = newObject.ToJson();
						Response.ContentType = MediaTypeNames.Application.Json;
						return Ok(s);
					}
					default:
						throw new NotImplementedException($"Unknown response type: {responseType}");
				}
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPost("{ns}/{bucket}/search")]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> SearchBuildAsync(NamespaceId ns, BucketId bucket, [FromBody] SearchRequest searchRequest)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.EnumerateBucket });
			if (accessResult != null)
			{
				return accessResult;
			}
			
			List<ISearchOp> searchOps = searchRequest.ToSearchOps();

			List<BuildMetadata> foundResults = new List<BuildMetadata>();
			int count = 0;
			bool checkForMaxTime = searchRequest.Options.MaxTimeMs != 0u;
			DateTime startTime = DateTime.Now;
			TimeSpan maxTimeToSpend = TimeSpan.FromMilliseconds(searchRequest.Options.MaxTimeMs);
			bool includeTTL = searchRequest.Options.IncludeTTL;
			bool includePartial = searchRequest.Options.IncludePartial;
			bool partialResult = false;
			TimeSpan timeSpent = TimeSpan.Zero;

			LogAuditEntry(HttpMethod.Post, nameof(SearchBuildAsync), ns, bucket, null, null);

			try
			{
				await foreach (BuildMetadata build in _buildStore.ListBuildsAsync(ns, bucket, includeTTL))
				{
					if (!build.IsFinalized && !includePartial)
					{
						// we ignore non-finalized builds unless explicitly asked to include them
						continue;
					}

					count++;
					if (count < searchRequest.Options.Skip)
					{
						continue;
					}

					if (count >= searchRequest.Options.Max)
					{
						// we have considered all the builds we are allowed to
						partialResult = true;
						break;
					}

					timeSpent = DateTime.Now - startTime;
					if (checkForMaxTime && timeSpent > maxTimeToSpend)
					{
						// spent all the time we were given for this query
						OkObjectResult result = Ok(new SearchResult(foundResults, true));
						result.StatusCode = (int)HttpStatusCode.RequestTimeout;
						return result;
					}

					bool match = true;
					foreach (ISearchOp op in searchOps)
					{
						CbObject o = build.Metadata;
						if (!op.Matches(build.BuildId, o, _logger))
						{
							match = false;
							break;
						}
					}

					if (match)
					{
						foundResults.Add(build);
					}

					if (foundResults.Count >= searchRequest.Options.Limit)
					{
						partialResult = true;
						break;
					}
				}
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			_logger.LogDebug("Search completed in {Namespace} {Bucket} request: {@SearchRequest} Results: {CountOfResults} Considered: {CountOfBuildsConsidered} took {Duration}", ns, bucket, searchRequest, foundResults.Count, count, timeSpent);
			Tracer.CurrentSpan.SetAttribute("countOfMatches", foundResults.Count);
			_diagnosticContext.Set("MatchCount", foundResults.Count);

			return Ok(new SearchResult(foundResults, partialResult));
		}

		
		[HttpPost("{ns}/search")]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> SearchBucketsAsync(NamespaceId ns, [FromBody] SearchBucketsRequest searchBucketRequest)
		{
			List<BucketId> bucketsToSearch = new List<BucketId>();

			Regex regex;
			try
			{
#pragma warning disable CA3012 // regex has a timeout so is okay to read from the body
				regex = new Regex(searchBucketRequest.BucketRegex, RegexOptions.IgnoreCase, TimeSpan.FromSeconds(5));
#pragma warning restore CA3012
			}
			catch (RegexParseException e)
			{
				return BadRequest($"Invalid regular expression for bucket filtering: {searchBucketRequest.BucketRegex}. Parse error was: {e.Error}");
			}

			try
			{
				bool lackedAccessToAtLeastOneBucket = false;
				foreach (BucketId bucket in await _refService.GetBucketsAsync(ns, HttpContext.RequestAborted)
					         .ToArrayAsync(HttpContext.RequestAborted))
				{
					if (!regex.IsMatch(bucket.ToString()))
					{
						// does not match the filter
						continue;
					}

					// verify we have access to the bucket
					ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket),
						new[] { JupiterAclAction.EnumerateBucket });
					if (accessResult == null)
					{
						bucketsToSearch.Add(bucket);
					}
					else
					{
						lackedAccessToAtLeastOneBucket = true;
					}
				}

				// if we found no matching buckets that we had access to we raise a 403
				if (bucketsToSearch.Count == 0 && lackedAccessToAtLeastOneBucket)
				{
					return new ForbidResult();
				}

				// no buckets matched
				if (bucketsToSearch.Count == 0)
				{
					// it's not an error to search were nothing matches, but it could be useful to know that it was due to the regular expression
					return Ok(new SearchMultipleBucketsResult());
				}
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			SearchRequest searchRequest = searchBucketRequest;
			List<ISearchOp> searchOps = searchRequest.ToSearchOps();

			int countConsidered = 0;
			bool checkForMaxTime = searchRequest.Options.MaxTimeMs != 0u;
			DateTime startTime = DateTime.Now;
			TimeSpan maxTimeToSpend = TimeSpan.FromMilliseconds(searchRequest.Options.MaxTimeMs);
			bool includeTTL = searchRequest.Options.IncludeTTL;
			bool includePartial = searchRequest.Options.IncludePartial;
			bool partialResult = false;
			TimeSpan timeSpent = TimeSpan.Zero;
			bool timedOut = false;

			SortedList<CbObjectId, BuildMetadata> foundResults = new (new CbObjectIdReverseComparer());

			using CancellationTokenSource cancellationTokenSource = new CancellationTokenSource();
			try
			{
				string sessionHeader = "None";

				if (Request!.Headers.TryGetValue("ue-session", out StringValues sessionHeaderValues))
				{
					sessionHeader = sessionHeaderValues.FirstOrDefault() ?? "None";
				}

				string jti = User.Claims.FirstOrDefault(claim => claim.Type == "jti")?.Value ?? "<unknown>";
				string userAgentString = string.Join(' ', Request.Headers.UserAgent.ToArray());
				string username = User.Identity?.Name ?? "Unknown-user";
				string ip = Request.HttpContext.Connection.RemoteIpAddress?.ToString() ?? "Unknown";

				await Parallel.ForEachAsync(bucketsToSearch, cancellationTokenSource.Token, async (bucket, token) =>
				{
					LogAuditEntry(HttpMethod.Post, nameof(SearchBucketsAsync), sessionHeader, username, userAgentString, ip, jti, ns, bucket, null, null);
				
					await foreach (BuildMetadata build in _buildStore.ListBuildsAsync(ns, bucket, includeTTL).WithCancellation(token))
					{
						if (!build.IsFinalized && !includePartial)
						{
							// we ignore non-finalized builds unless explicitly asked to include them
							continue;
						}

						if (token.IsCancellationRequested)
						{
							partialResult = true;
							break;
						}
						int count = Interlocked.Increment(ref countConsidered);

						if (count >= searchRequest.Options.Max)
						{
							// we have considered all the builds we are allowed to consider
							partialResult = true;
							await cancellationTokenSource.CancelAsync();
							break;
						}

						timeSpent = DateTime.Now - startTime;
						if (checkForMaxTime && timeSpent > maxTimeToSpend)
						{
							// spent all the time we were given for this query
							timedOut = true;
							partialResult = true;
							await cancellationTokenSource.CancelAsync();
							break;
						}

						bool match = true;
						foreach (ISearchOp op in searchOps)
						{
							CbObject o = build.Metadata;
							if (!op.Matches(build.BuildId, o, _logger))
							{
								match = false;
								break;
							}
						}

						if (match)
						{
							lock (foundResults)
							{
								foundResults.Add(build.BuildId, build);
							}
						}
					}
				});
			}
			catch (TaskCanceledException)
			{
				// cancellation is not an issue so we can suppress this
				partialResult = true;
			}
			catch (AggregateException e)
			{
				if (e.InnerExceptions.Any(exception => exception.GetType() == typeof(RemoteServiceException)))
				{
					return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
				}

				throw;
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
			// Collect the results that was newest that match our limits and search
			List<BuildMetadata> foundResultsInBuckets = foundResults.Values.Skip(searchRequest.Options.Skip).Take(searchRequest.Options.Limit).ToList();

			int countOfMatches = foundResultsInBuckets.Count;
			Tracer.CurrentSpan.SetAttribute("countOfMatches", countOfMatches);
			_diagnosticContext.Set("MatchCount", countOfMatches);

			if (timedOut)
			{
				OkObjectResult result = Ok(new SearchMultipleBucketsResult(foundResultsInBuckets, true));
				result.StatusCode = (int)HttpStatusCode.RequestTimeout;
				return result;
			}

			_logger.LogDebug("Search completed in {Namespace} for Buckets {@Buckets} request: {@SearchRequest} Results: {CountOfResults} Considered: {CountOfBuildsConsidered} took {Duration}", ns, bucketsToSearch, searchRequest, countOfMatches, countConsidered, timeSpent);

			return Ok(new SearchMultipleBucketsResult(foundResultsInBuckets, partialResult));
		}
		
		[HttpPost("{ns}/{bucket}/{buildId}/copyBuild")]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> CopyBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, [FromBody] CopyBuildRequest request)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			NamespaceId targetNamespace = request.NewNamespace ?? ns;
			BucketId targetBucket = request.NewBucket;
			ActionResult? accessResultTarget = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(targetNamespace, targetBucket), new[] { JupiterAclAction.WriteObject });
			if (accessResultTarget != null)
			{
				return accessResultTarget;
			}

			try
			{
				BuildRecord? build = await GetBuildWithRetriesAsync(ns, bucket, buildId);
				if (build == null)
				{
					return NotFound($"Build \"{build}\" in bucket {bucket} and namespace {ns}");
				}

				CbObject originalBuildObject = build.BuildObject;

				CbWriter cbWriter = new CbWriter();
				cbWriter.BeginObject();
				List<string> fieldsToNotCopy = new List<string>();
				if (!string.IsNullOrEmpty(request.NewBranch))
				{
					cbWriter.WriteString("branch", request.NewBranch);
					fieldsToNotCopy.Add("branch");
				}

				if (!string.IsNullOrEmpty(request.NewProject))
				{
					cbWriter.WriteString("project", request.NewProject);
					fieldsToNotCopy.Add("project");
				}

				if (!string.IsNullOrEmpty(request.NewPlatform))
				{
					cbWriter.WriteString("platform", request.NewPlatform);
					fieldsToNotCopy.Add("platform");
				}

				foreach (CbField field in originalBuildObject)
				{
					string fieldName = field.Name.ToString();
					if (fieldsToNotCopy.Contains(fieldName))
					{
						continue;
					}

					cbWriter.WriteField(field);
				}

				cbWriter.EndObject();

				CbObject newBuildObject = cbWriter.ToObject();
				uint ttl = (uint)_namespacePolicyResolver.GetPoliciesForNs(ns).DefaultTTL.TotalSeconds;
				CbObjectId newBuildId = CbObjectId.NewObjectId();
				await _buildStore.PutBuildAsync(targetNamespace, targetBucket, newBuildId, newBuildObject, ttl);

				BlockContext newBlockContext = BlockContext.FromObject(newBuildObject);

				async Task CopyRefAsync(NamespaceId ns, BucketId bucket, RefId oldRefId, NamespaceId targetNamespace, BucketId targetBucket, RefId newRefId, CbObjectId? buildPart = null)
				{
					// we do not explicitly access control the creation of the build part, this part already existed so we should create a full copy of it

					(RefRecord refRecord, BlobContents? contents) = await _refService.GetAsync(ns, bucket, oldRefId);
					if (contents == null)
					{
						// large blob that is not inlined, needs to be copied
						await _blobStore.CopyBlobAsync(ns, targetNamespace, refRecord.BlobIdentifier, bucket);
						contents = await _blobStore.GetObjectAsync(targetNamespace, refRecord.BlobIdentifier);
					}

					CbObject payloadObject = new CbObject(await contents.Stream.ReadAllBytesAsync());
					BlobId payloadHash = refRecord.BlobIdentifier;

					ConcurrentDictionary<ContentHash, List<BlobId>> contentIdToBlobs = new ConcurrentDictionary<ContentHash, List<BlobId>>();

					{
						// check all attachments for content ids and blocks and copy them
						IContentIdStore contentIdStore = HttpContext.RequestServices.GetService<IContentIdStore>()!;

						using TelemetrySpan scope = _tracer.StartActiveSpan("copy.attachments").SetAttribute("operation.name", "copy.attachments");

						await foreach (Attachment attachment in _referenceResolver.GetAttachmentsAsync(ns, payloadObject, CancellationToken.None))
						{
							BlobId blobId = BlobId.FromIoHash(attachment.AsIoHash());

							// by default the raw hash is also the expected blob
							List<BlobId> referencedBlobs = new() { blobId };
							if (attachment is ContentIdAttachment contentIdAttachment)
							{
								// we are remapping the blobs so the raw hash is not valid
								referencedBlobs.Clear();

								IAsyncEnumerable<ContentIdMapping> mappings = contentIdStore.GetContentIdMappingsAsync(ns, contentIdAttachment.Identifier, CancellationToken.None);
								await foreach (ContentIdMapping mapping in mappings)
								{
									// copy the content id
									// TODO: The raw size is not the weight
									await contentIdStore.PutAsync(targetNamespace, contentIdAttachment.Identifier, mapping.ReferencedBlobs, (ulong)mapping.Weight, (ulong)mapping.Weight);

									referencedBlobs.AddRange(mapping.ReferencedBlobs);
								}
							}

							BlobId? blockMetadata = await _blockStore.GetBlockMetadataAsync(ns, blobId);
							if (blockMetadata != null)
							{
								await _blobStore.CopyBlobAsync(ns, targetNamespace, blockMetadata, bucket);
								await _blockStore.PutBlockMetadataAsync(targetNamespace, blobId, blockMetadata);
							}

							contentIdToBlobs[blobId] = referencedBlobs;
						}
					}

					List<ContentHash> missingHashes = new List<ContentHash>();
					const int RetryAttempts = 3;
					for (int i = 0; i < RetryAttempts; i++)
					{
						if (buildPart != null)
						{
							missingHashes = await PutBuildPartRefAsync(targetNamespace, targetBucket, newBuildId, buildPart.Value, newBlockContext, payloadObject, payloadHash);
						}
						else
						{
							(ContentId[] missingReferences, BlobId[] missingBlobs) = await _refService.PutAsync(targetNamespace, targetBucket, newRefId, payloadHash, payloadObject, cancellationToken: HttpContext.RequestAborted);

							missingHashes = new List<ContentHash>(missingReferences);
							missingHashes.AddRange(missingBlobs);
						}
						
						if (missingHashes.Count != 0)
						{
							using TelemetrySpan scope = _tracer.StartActiveSpan("copy.blobs").SetAttribute("operation.name", "copy.blobs");

							// copy missing blobs and try to finalize again
							foreach (ContentHash missingHash in missingHashes)
							{
								if (contentIdToBlobs.TryGetValue(missingHash, out List<BlobId>? blobs))
								{
									foreach (BlobId blobId in blobs)
									{
										await _blobStore.CopyBlobAsync(ns, targetNamespace, blobId, bucket);
									}
								}
							}
						}
					}

					if (missingHashes.Count != 0)
					{
						throw new Exception($"Hashes missing when copying build after {RetryAttempts} attempts.");
					}
				}

				await foreach (string name in _buildStore.GetBuildAppendixesAsync(ns, bucket, buildId))
				{
					RefId oldAppendixRef = RefId.FromName($"{buildId}/appendix/{name}");
					RefId newAppendixRef = RefId.FromName($"{newBuildId}/appendix/{name}");
					await CopyRefAsync(ns, bucket, oldAppendixRef, targetNamespace, targetBucket, newAppendixRef, buildPart: null);

					await _buildStore.PutBuildAppendixAsync(targetNamespace, targetBucket, newBuildId, name, ttl);
				}

				await foreach ((string partName, CbObjectId partId) in _buildStore.GetBuildPartsAsync(ns, bucket, buildId))
				{
					RefId oldRefId = RefId.FromName($"{buildId}/{partId}");
					RefId newRefId = RefId.FromName($"{newBuildId}/{partId}");
					await CopyRefAsync(ns, bucket, oldRefId, targetNamespace, targetBucket, newRefId, buildPart: partId);
					await _buildStore.PutBuildPartAsync(targetNamespace, targetBucket, newBuildId, partId, partName, ttl);
				}

				return Ok(new CopyBuildResponse {BuildId = newBuildId});
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}
		#endregion

		#region Build parts endpoints
		[HttpGet("{ns}/{bucket}/{buildId}/parts/{partId}.{format?}")]
		[Produces(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Json)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetBuildPartAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string partId, [FromRoute] string? format = null)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket, BuildPartType, partId), new[] { JupiterAclAction.ReadBuildPart });
			if (accessResult != null)
			{
				return accessResult;
			}

			LogAuditEntry(HttpMethod.Get, nameof(GetBuildPartAsync), ns, bucket, buildId, partId);

			try
			{
				Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
				Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

				RefId refId = RefId.FromName($"{buildId}/{partId}");
				(RefRecord objectRecord, BlobContents? maybeBlob) = await _refService.GetAsync(ns, bucket, refId, Array.Empty<string>());

				if (maybeBlob == null)
				{
					throw new InvalidOperationException($"Blob was null when attempting to fetch {ns} {bucket} {buildId}");
				}

				await using BlobContents blob = maybeBlob;

				if (!objectRecord.IsFinalized)
				{
					// we do not consider un-finalized objects as valid
					return BadRequest(new ProblemDetails { Title = $"Build {buildId} in bucket {objectRecord.Bucket} with part {partId} stored as ref {objectRecord.Name} is not finalized." });
				}

				Response.Headers[CommonHeaders.HashHeaderName] = objectRecord.BlobIdentifier.ToString();
				Response.Headers[CommonHeaders.LastAccessHeaderName] = objectRecord.LastAccess.ToString(CultureInfo.InvariantCulture);

				async Task WriteBody(BlobContents blobContents, string contentType)
				{
					IServerTiming? serverTiming = Request.HttpContext.RequestServices.GetService<IServerTiming>();
					using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope("body.write", "Time spent writing body");

					long contentLength = blobContents.Length;
					using TelemetrySpan scope = _tracer.StartActiveSpan("body.write").SetAttribute("operation.name", "body.write");
					scope.SetAttribute("content-length", contentLength);
					const int BufferSize = 64 * 1024;
					Stream outputStream = Response.Body;
					Response.ContentLength = contentLength;
					Response.ContentType = contentType;
					Response.StatusCode = StatusCodes.Status200OK;
					try
					{
						await StreamCopyOperation.CopyToAsync(blobContents.Stream, outputStream, count: null, bufferSize: BufferSize, cancel: Response.HttpContext.RequestAborted);
					}
					catch (OperationCanceledException)
					{
						// do not raise exceptions for cancelled writes
						// as we have already started writing a response we can not change the status code
						// so we just drop a warning and proceed
						_logger.LogWarning("The operation was canceled while writing the body");
					}
				}

				string responseType = _formatResolver.GetResponseType(Request, format, CustomMediaTypeNames.UnrealCompactBinary);
				Tracer.CurrentSpan.SetAttribute("response-type", responseType);

				switch (responseType)
				{
					case CustomMediaTypeNames.UnrealCompactBinary:
						{
							// for compact binary we can just serialize our internal object
							await WriteBody(blob, CustomMediaTypeNames.UnrealCompactBinary);

							break;
						}
					case MediaTypeNames.Application.Json:
						{
							byte[] blobMemory;
							{
								using TelemetrySpan scope = _tracer.StartActiveSpan("json.readblob").SetAttribute("operation.name", "json.readblob");
								blobMemory = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
							}
							CbObject cb = new CbObject(blobMemory);
							string s = cb.ToJson();
							await using BlobContents contents = new BlobContents(Encoding.UTF8.GetBytes(s));
							await WriteBody(contents, MediaTypeNames.Application.Json);
							break;
						}
					default:
						throw new NotImplementedException($"Unknown expected response type {responseType}");
				}

				// this result is ignored as we write to the body explicitly
				return new EmptyResult();

			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RefNotFoundException)
			{
				return NotFound(new ProblemDetails { Title = $"Build part \'{partId}\' in build {buildId} and bucket {bucket} did not exist" });
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Blob {e.Blob} in {e.Ns} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPut("{ns}/{bucket}/{buildId}/parts/{partId}/{partName}.{format?}")]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> PutBuildPartAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObjectId partId, string partName)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteBuildPart });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);
			
			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			LogAuditEntry(HttpMethod.Put, nameof(PutBuildPartAsync), ns, bucket, buildId, partName);

			CbObject payloadObject;
			BlobId payloadHash;
			try
			{
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-build-part", HttpContext.RequestAborted);

				switch (Request.ContentType)
				{
					case MediaTypeNames.Application.Json:
						{
							await using MemoryStream ms = new MemoryStream();
							await using Stream payloadStream = payload.GetStream();
							await payloadStream.CopyToAsync(ms);
							byte[] b = ms.ToArray();
							string s = Encoding.UTF8.GetString(b);
							payloadObject = CbObject.FromJson(s);
							payloadHash = BlobId.FromBlob(payloadObject.GetView().ToArray());
							break;
						}
					case CustomMediaTypeNames.UnrealCompactBinary:
						{
							await using MemoryStream ms = new MemoryStream();
							await using Stream payloadStream = payload.GetStream();
							await payloadStream.CopyToAsync(ms);
							payloadObject = new CbObject(ms.ToArray());
							payloadHash = BlobId.FromBlob(payloadObject.GetView().ToArray());
							break;
						}
					default:
						throw new Exception($"Unknown request type {Request.ContentType}");
				}
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}

			BuildRecord? buildRecord = await GetBuildWithRetriesAsync(ns, bucket, buildId);
			if (buildRecord == null)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"No build found for namespace {ns} bucket {bucket} name {buildId}"
				});
			}

			// build is finalized, we should not modify it
			if (buildRecord.IsFinalized)
			{
				return BadRequest($"Build {buildId} is already finalized - it is read only.");
			}

			BlockContext blockContext = BlockContext.FromObject(buildRecord.BuildObject);

			try
			{
				uint ttl = (uint)_namespacePolicyResolver.GetPoliciesForNs(ns).DefaultTTL.TotalSeconds;
				await _buildStore.PutBuildPartAsync(ns, bucket, buildId, partId, partName, ttl);
			}
			catch (ResourceHasToManyRequestsException)
			{
				return StatusCode(StatusCodes.Status429TooManyRequests);
			}

			try
			{
				List<ContentHash> missingHashes = await PutBuildPartRefAsync(ns, bucket, buildId, partId, blockContext, payloadObject, payloadHash);
				return Ok(new PutObjectResponse(missingHashes.ToArray()));
			}
			catch (RefAlreadyExistsException)
			{
				return Problem($"Build already exists {ns} {bucket} {buildId}", statusCode: (int)HttpStatusCode.Conflict);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		private async Task<List<ContentHash>> PutBuildPartRefAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObjectId partId, BlockContext blockContext, CbObject payloadObject, BlobId payloadHash)
		{
			RefId refId = RefId.FromName($"{buildId}/{partId}");
			{
				using TelemetrySpan scope = _tracer.StartActiveSpan("ref.put").SetAttribute("operation.name", "ref.put").SetAttribute("resource.name", refId.ToString());

				ConcurrentBag<Task<BlobId?>> addBlockToContextTasks = new ConcurrentBag<Task<BlobId?>>();
				void OnBlobFound(BlobId blobId)
				{
					addBlockToContextTasks.Add(Task.Run(async () =>
					{
						BlobId? metadataBlob = await _blockStore.GetBlockMetadataAsync(ns, blobId);
						if (metadataBlob != null)
						{
							return metadataBlob;
						}

						return null;
					}));
				}

				(ContentId[] missingReferences, BlobId[] missingBlobs) = await _refService.PutAsync(ns, bucket, refId, payloadHash, payloadObject, OnBlobFound, cancellationToken: HttpContext.RequestAborted);

				List<ContentHash> missingHashes = new List<ContentHash>(missingReferences);
				missingHashes.AddRange(missingBlobs);
				ContentHash[] missingArray = missingHashes.ToArray();
				scope.SetAttribute("NeedsCount", missingArray.Length);

				await Task.WhenAll(addBlockToContextTasks);

				if (missingHashes.Count == 0)
				{
					using TelemetrySpan finalizeScope = _tracer.StartActiveSpan("build.finalize").SetAttribute("operation.name", "build.finalize");
					
					List<BlobId> blockMetadataIds = new List<BlobId>();
					foreach (Task<BlobId?> addBlockToContextTask in addBlockToContextTasks)
					{
						BlobId? b = await addBlockToContextTask;
						if (b != null)
						{
							blockMetadataIds.Add(b);
						}
					}
					finalizeScope.SetAttribute("BlocksCount", blockMetadataIds.Count);
					await DoFinalizeBuildPartAsync(ns, blockContext, blockMetadataIds, cancellationToken: HttpContext.RequestAborted);
				}

				return missingHashes;
			}
		}

		[HttpPost("{ns}/{bucket}/{buildId}/parts/{partId}/finalize/{hash}.{format?}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> FinalizeBuildPartAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObjectId partId, BlobId hash)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
				Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

				// do not fetch with retries here as the build should already have been PUT so it's very odd for it to not be findable at this point
				BuildRecord? buildRecord = await _buildStore.GetBuildAsync(ns, bucket, buildId);
				if (buildRecord == null)
				{
					return NotFound($"No build found for namespace {ns} bucket {bucket} name {buildId}");
				}

				// build is finalized, we should not modify it
				if (buildRecord.IsFinalized)
				{
					return BadRequest($"Build {buildId} is already finalized - it is read only.");
				}

				BlockContext blockContext = BlockContext.FromObject(buildRecord.BuildObject);

				Tracer.CurrentSpan.SetAttribute("blockContext", blockContext.ToString());

				ConcurrentBag<Task<BlobId?>> addBlockToContextTasks = new ConcurrentBag<Task<BlobId?>>();
				void OnBlobFound(BlobId blobId)
				{
					addBlockToContextTasks.Add(Task.Run(async () =>
					{
						BlobId? metadataBlob = await _blockStore.GetBlockMetadataAsync(ns, blobId);
						if (metadataBlob != null)
						{
							return metadataBlob;
						}

						return null;
					}));
				}

				RefId refId = RefId.FromName($"{buildId}/{partId}");
				(ContentId[] missingReferences, BlobId[] missingBlobs) = await _refService.FinalizeAsync(ns, bucket, refId, hash, OnBlobFound, HttpContext.RequestAborted);
				List<ContentHash> missingHashes = new List<ContentHash>(missingReferences);
				missingHashes.AddRange(missingBlobs);

				await Task.WhenAll(addBlockToContextTasks);
				Tracer.CurrentSpan.SetAttribute("NeedsCount", missingHashes.Count);
				if (missingHashes.Count == 0)
				{
					using TelemetrySpan finalizeScope = _tracer.StartActiveSpan("build.finalize").SetAttribute("operation.name", "build.finalize");

					List<BlobId> blockMetadataIds = new List<BlobId>();
					foreach (Task<BlobId?> addBlockToContextTask in addBlockToContextTasks)
					{
						BlobId? b = await addBlockToContextTask;
						if (b != null)
						{
							blockMetadataIds.Add(b);
						}
					}
					finalizeScope.SetAttribute("BlocksCount", blockMetadataIds.Count);
					await DoFinalizeBuildPartAsync(ns, blockContext, blockMetadataIds, cancellationToken: HttpContext.RequestAborted);
				}

				return Ok(new PutObjectResponse(missingHashes.ToArray()));
			}
			catch (ObjectHashMismatchException e)
			{
				return BadRequest(e.Message);
			}
			catch (RefNotFoundException e)
			{
				return NotFound(e.Message);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		private async Task<BuildRecord?> GetBuildWithRetriesAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
		{
			const int MaxRetries = 3;
			TimeSpan[] delayTimes = new [] {TimeSpan.FromSeconds(0.5), TimeSpan.FromSeconds(3), TimeSpan.FromSeconds(0)};

			// We read this build right after submitting it in some cases which due to Scylla being eventually consistent can mean that the build doesn't exist
			// As it should be unusual to fetch a build that doesn't exist we just retry a few times to make sure it doesn't settle into existing
			for (int i = 0; i < MaxRetries; i++)
			{
				BuildRecord? buildRecord = await _buildStore.GetBuildAsync(ns, bucket, buildId);
				if (buildRecord == null)
				{
					// wait a while and try again
					await Task.Delay(delayTimes[i]);
					continue;
				}

				return buildRecord;

			}

			return null;
		}

		private async Task DoFinalizeBuildPartAsync(NamespaceId ns, BlockContext blockContext, List<BlobId> blockMetadataIds, CancellationToken cancellationToken)
		{
			await Parallel.ForEachAsync(blockMetadataIds, cancellationToken, async (metadataBlobId, token) =>
			{
				// this is a valid block because it has metadata submitted
				await _blockStore.AddBlockToContextAsync(ns, blockContext, metadataBlobId);
			});
		}

		#region Appendix file endpoints
		[HttpGet("{ns}/{bucket}/{buildId}/appendix/{appendixFileName}")]
		[Produces(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Json)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetAppendixFileAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string appendixFileName)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			LogAuditEntry(HttpMethod.Get, nameof(GetAppendixFileAsync), ns, bucket, buildId, null);

			try
			{
				Tracer.CurrentSpan.SetAttribute("filename", appendixFileName);
				Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
				Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

				
				RefId refId = RefId.FromName($"{buildId}/appendix/{appendixFileName}");
				(RefRecord objectRecord, BlobContents? maybeBlob) = await _refService.GetAsync(ns, bucket, refId, Array.Empty<string>());

				if (maybeBlob == null)
				{
					throw new InvalidOperationException($"Blob was null when attempting to fetch {ns} {bucket} {buildId}");
				}

				await using BlobContents blob = maybeBlob;

				if (!objectRecord.IsFinalized)
				{
					// we do not consider un-finalized objects as valid
					return BadRequest(new ProblemDetails { Title = $"Build {buildId} in bucket {objectRecord.Bucket} with appendix {appendixFileName} stored as ref {objectRecord.Name} is not finalized." });
				}

				Response.Headers[CommonHeaders.HashHeaderName] = objectRecord.BlobIdentifier.ToString();
				Response.Headers[CommonHeaders.LastAccessHeaderName] = objectRecord.LastAccess.ToString(CultureInfo.InvariantCulture);

				async Task WriteBody(BlobContents blobContents, string contentType)
				{
					IServerTiming? serverTiming = Request.HttpContext.RequestServices.GetService<IServerTiming>();
					using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope("body.write", "Time spent writing body");

					long contentLength = blobContents.Length;
					using TelemetrySpan scope = _tracer.StartActiveSpan("body.write").SetAttribute("operation.name", "body.write");
					scope.SetAttribute("content-length", contentLength);
					const int BufferSize = 64 * 1024;
					Stream outputStream = Response.Body;
					Response.ContentLength = contentLength;
					Response.ContentType = contentType;
					Response.StatusCode = StatusCodes.Status200OK;
					try
					{
						await StreamCopyOperation.CopyToAsync(blobContents.Stream, outputStream, count: null, bufferSize: BufferSize, cancel: Response.HttpContext.RequestAborted);
					}
					catch (OperationCanceledException)
					{
						// do not raise exceptions for cancelled writes
						// as we have already started writing a response we can not change the status code
						// so we just drop a warning and proceed
						_logger.LogWarning("The operation was canceled while writing the body");
					}
				}

				string responseType = _formatResolver.GetResponseType(Request, null, CustomMediaTypeNames.UnrealCompactBinary);
				Tracer.CurrentSpan.SetAttribute("response-type", responseType);

				switch (responseType)
				{
					case CustomMediaTypeNames.UnrealCompactBinary:
						{
							// for compact binary we can just serialize our internal object
							await WriteBody(blob, CustomMediaTypeNames.UnrealCompactBinary);

							break;
						}
					case MediaTypeNames.Application.Json:
						{
							byte[] blobMemory;
							{
								using TelemetrySpan scope = _tracer.StartActiveSpan("json.readblob").SetAttribute("operation.name", "json.readblob");
								blobMemory = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
							}
							CbObject cb = new CbObject(blobMemory);
							string s = cb.ToJson();
							await using BlobContents contents = new BlobContents(Encoding.UTF8.GetBytes(s));
							await WriteBody(contents, MediaTypeNames.Application.Json);
							break;
						}

					case CustomMediaTypeNames.UnrealCompressedBuffer:
					case MediaTypeNames.Application.Octet:
						{
							byte[] blobMemory = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
							CbObject cb = new CbObject(blobMemory);
							IoHash rawHash = cb["RawHash"].AsBinaryAttachment().Hash;
							(BlobContents blobContents, string mediaType, BlobId? contentHash) = await _blobStore.GetCompressedObjectAsync(ns, ContentId.FromIoHash(rawHash), HttpContext.RequestServices);

							StringValues acceptHeader = Request.Headers["Accept"];
							if (!acceptHeader.Contains("*/*") && acceptHeader.Count != 0 && !acceptHeader.Contains(mediaType))
							{
								return new UnsupportedMediaTypeResult();
							}

							if (contentHash != null && Request.Headers.Range.Count == 0)
							{
								// send the hash of the object is we are fetching the full blob
								Response.Headers[CommonHeaders.HashHeaderName] = contentHash.ToString();
							}

							if (blobContents.RedirectUri != null)
							{
								return Redirect(blobContents.RedirectUri.ToString());
							}

							if (_nginxRedirectHelper.CanRedirect(Request, blobContents))
							{
								return _nginxRedirectHelper.CreateActionResult(blobContents, mediaType);
							}

							return File(blobContents.Stream, mediaType, enableRangeProcessing: true);
						}
					default:
						throw new NotImplementedException($"Unknown expected response type {responseType}");
				}

				// this result is ignored as we write to the body explicitly
				return new EmptyResult();

			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RefNotFoundException)
			{
				return NotFound(new ProblemDetails { Title = $"Appendix \'{appendixFileName}\' in build {buildId} and bucket {bucket} did not exist" });
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Blob {e.Blob} in {e.Ns} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPut("{ns}/{bucket}/{buildId}/appendix/{appendixFileName}")]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json, CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Octet)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> PutAppendixFileAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string appendixFileName)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);
			
			Tracer.CurrentSpan.SetAttribute("filename", appendixFileName);
			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			LogAuditEntry(HttpMethod.Put, nameof(PutAppendixFileAsync), ns, bucket, buildId, null);

			CbObject payloadObject;
			BlobId payloadHash;
			try
			{
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-appendix", HttpContext.RequestAborted);

				switch (Request.ContentType)
				{
					case MediaTypeNames.Application.Json:
						{
							await using MemoryStream ms = new MemoryStream();
							await using Stream payloadStream = payload.GetStream();
							await payloadStream.CopyToAsync(ms);
							byte[] b = ms.ToArray();
							string s = Encoding.UTF8.GetString(b);
							payloadObject = CbObject.FromJson(s);
							payloadHash = BlobId.FromBlob(payloadObject.GetView().ToArray());
							break;
						}
					case CustomMediaTypeNames.UnrealCompactBinary:
						{
							await using MemoryStream ms = new MemoryStream();
							await using Stream payloadStream = payload.GetStream();
							await payloadStream.CopyToAsync(ms);
							payloadObject = new CbObject(ms.ToArray());
							payloadHash = BlobId.FromBlob(payloadObject.GetView().ToArray());
							break;
						}
					case CustomMediaTypeNames.UnrealCompressedBuffer:
						{
							bool? bypassCache = _namespacePolicyResolver.GetPoliciesForNs(ns).BypassCacheOnWrite;
							ContentId cid = await _blobStore.PutCompressedObjectAsync(ns, payload, null, HttpContext.RequestServices, bucketHint: bucket, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);

							CbWriter writer = new CbWriter();
							writer.BeginObject();
							writer.WriteBinaryAttachment("RawHash", cid.AsIoHash());
							writer.WriteInteger("RawSize", payload.Length);
							writer.EndObject();

							byte[] blob = writer.ToByteArray();
							payloadObject = new CbObject(blob);
							payloadHash = BlobId.FromBlob(blob);

							break;
						}
					case MediaTypeNames.Application.Octet:
						{
							await using MemoryStream ms = new MemoryStream();
							await using Stream payloadStream = payload.GetStream();
							await payloadStream.CopyToAsync(ms);
							byte[] b = ms.ToArray();
							BlobId hash = BlobId.FromBlob(b);

							bool? bypassCache = _namespacePolicyResolver.GetPoliciesForNs(ns).BypassCacheOnWrite;
							BlobId id = await _blobStore.PutObjectAsync(ns, b, hash, bucketHint:bucket, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);

							CbWriter writer = new CbWriter();
							writer.BeginObject();
							writer.WriteBinaryAttachment("RawHash", id.AsIoHash());
							writer.WriteInteger("RawSize", payload.Length);
							writer.EndObject();

							byte[] blob = writer.ToByteArray();
							payloadObject = new CbObject(blob);
							payloadHash = BlobId.FromBlob(blob);

							break;
						}
					default:
						throw new Exception($"Unknown request type {Request.ContentType}");
				}
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}

			BuildRecord? buildRecord = await GetBuildWithRetriesAsync(ns, bucket, buildId);
			if (buildRecord == null)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"No build found for namespace {ns} bucket {bucket} name {buildId}"
				});
			}

			try
			{
				RefId refId = RefId.FromName($"{buildId}/appendix/{appendixFileName}");
				using TelemetrySpan scope = _tracer.StartActiveSpan("ref.put").SetAttribute("operation.name", "ref.put").SetAttribute("resource.name", refId.ToString());

				(ContentId[] missingReferences, BlobId[] missingBlobs) = await _refService.PutAsync(ns, bucket, refId, payloadHash, payloadObject, cancellationToken: HttpContext.RequestAborted);

				List<ContentHash> missingHashes = new List<ContentHash>(missingReferences);
				missingHashes.AddRange(missingBlobs);
				ContentHash[] missingArray = missingHashes.ToArray();
				scope.SetAttribute("NeedsCount", missingArray.Length);

				if (missingHashes.Count == 0)
				{
					uint ttl = (uint)_namespacePolicyResolver.GetPoliciesForNs(ns).DefaultTTL.TotalSeconds;
					await _buildStore.PutBuildAppendixAsync(ns, bucket, buildId, appendixFileName, ttl);
				}
				return Ok(new PutObjectResponse(missingHashes.ToArray()));
			}
			catch (RefAlreadyExistsException)
			{
				return Problem($"Appendix {appendixFileName} for build {ns} {bucket} {buildId} already exists", statusCode: (int)HttpStatusCode.Conflict);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpDelete("{ns}/{bucket}/{buildId}/appendix/{appendixFileName}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> DeleteAppendixFileAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string appendixFileName)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.DeleteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);
			
			Tracer.CurrentSpan.SetAttribute("filename", appendixFileName);
			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			LogAuditEntry(HttpMethod.Delete, nameof(DeleteAppendixFileAsync), ns, bucket, buildId, null);

			BuildRecord? buildRecord = await GetBuildWithRetriesAsync(ns, bucket, buildId);
			if (buildRecord == null)
			{
				return BadRequest(new ProblemDetails { Title = $"No build found for namespace {ns} bucket {bucket} name {buildId}" });
			}

			try
			{
				RefId refId = RefId.FromName($"{buildId}/appendix/{appendixFileName}");
				using TelemetrySpan scope = _tracer.StartActiveSpan("ref.delete").SetAttribute("operation.name", "ref.delete").SetAttribute("resource.name", refId.ToString());

				await _refService.DeleteAsync(ns, bucket, refId, cancellationToken: HttpContext.RequestAborted);

				await _buildStore.DeleteBuildAppendixAsync(ns, bucket, buildId, appendixFileName);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return Ok();
		}

		[HttpGet("{ns}/{bucket}/{buildId}/appendix")]
		[Produces(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> ListAppendixFilesAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			LogAuditEntry(HttpMethod.Get, nameof(ListAppendixFilesAsync), ns, bucket, buildId, null);

			try
			{
				Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
				Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

				List<string> appendixes = await _buildStore.GetBuildAppendixesAsync(ns, bucket, buildId).ToListAsync();

				return Ok(new ListAppendixFileResponse {Appendices = appendixes});
			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		#endregion

		[HttpGet("{ns}/{bucket}/{buildId}/parts/{partId}/fileList")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetBuildPartFileListAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObjectId partId)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			try
			{
				await foreach ((string partName, CbObjectId foundPartId) in _buildStore.GetBuildPartsAsync(ns, bucket, buildId))
				{
					if (!CbObjectId.Equals(foundPartId, partId))
					{
						continue;
					}

					ActionResult? accessResultPart = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket, BuildPartType, partName), new[] { JupiterAclAction.ReadBuildPart });
					if (accessResultPart != null)
					{
						return accessResultPart;
					}

					(RefRecord refRecord, BlobContents? contents)  = await _refService.GetAsync(ns, bucket, RefId.FromName($"{buildId}/{partId}"));
					if (contents == null)
					{
						// large blob that was not inlined, fetch its content
						contents = await _blobStore.GetObjectAsync(ns, refRecord.BlobIdentifier);
					}

					await using BlobContents blobContents = contents;
					CbObject payloadObject = new CbObject(await blobContents.Stream.ReadAllBytesAsync());

					ulong totalSize = payloadObject["totalSize"].AsUInt64();
					CbObject files = payloadObject["files"].AsObject();
					string[] paths = files["paths"].AsArray().Select(field => field.AsString()).ToArray();
					IoHash[] hashes = files["rawhashes"].AsArray().Select(field => field.AsHash()).ToArray();
					ulong[] rawSizes = files["rawsizes"].AsArray().Select(field => field.AsUInt64()).ToArray();
					uint[] attr = files["attributes"].AsArray().Select(field => field.AsUInt32()).ToArray();
					uint[] mode = files["mode"].AsArray().Select(field => field.AsUInt32()).ToArray();

					FileList fileList = new FileList
					{
						TotalSize = totalSize,
						Paths = paths,
						RawHashes = hashes,
						RawSizes = rawSizes,
						Attributes = attr,
						Modes = mode
					};

					return Ok(fileList);
				}
			}
			catch (RefNotFoundException e)
			{
				return NotFound(e.Message);
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(e.Message);
			}
			catch (AggregateException e)
			{
				if (e.InnerExceptions.Any(exception => exception.GetType() == typeof(RemoteServiceException)))
				{
					return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
				}

				throw;
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return NotFound();
		}

		
		[HttpPut("{ns}/{bucket}/{buildId}/parts/{partId}/stats")]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> PutBuildPartStatsAsync(
			NamespaceId ns, 
			BucketId bucket, 
#pragma warning disable IDE0060
			CbObjectId buildId, 
			CbObjectId partId, 
#pragma warning restore IDE0060
			[Required] [FromBody] BuildPartStatsRequest statsRequest)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			const string StatPrefixMeter = "build.stats.";
			const string StatPrefixTrace = "stats.";
			KeyValuePair<string, object?>[] tags = {
				new("namespace", ns),
				new("bucket", bucket)
			};

			// we do not use the singleton meter here as these are not continuous values, they are only reported when this endpoint is run
			// this way they are reported once and then dropped until the next time the endpoint is run
			using Meter meter = new Meter("UnrealCloudDDC");

			foreach (KeyValuePair<string, double> stat in statsRequest.FloatStats)
			{
				Histogram<double> h = meter.CreateHistogram<double>(StatPrefixMeter + stat.Key);

				h.Record(stat.Value, tags);

				Tracer.CurrentSpan.SetAttribute(StatPrefixTrace + stat.Key, stat.Value);
			}

			return Ok();
		}

		#endregion

		#region Blob endpoints

		[HttpGet("{ns}/{bucket}/{buildId}/parts/{partId}/blobs/{id}")]
		[HttpGet("{ns}/{bucket}/{buildId}/blobs/{id}")]
		[ProducesResponseType(type: typeof(byte[]), 200)]
		[ProducesResponseType(type: typeof(ValidationProblemDetails), 400)]
		[Produces(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Octet)]

		public async Task<IActionResult> GetBlobAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[Required] ContentId id,
			[FromQuery] bool supportsRedirect = false)
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (result != null)
			{
				return result;
			}

			try
			{
				(BlobContents blobContents, string mediaType, BlobId? contentHash) = await _blobStore.GetCompressedObjectAsync(ns, id, HttpContext.RequestServices, supportsRedirectUri: supportsRedirect);

				StringValues acceptHeader = Request.Headers["Accept"];
				if (!acceptHeader.Contains("*/*") && acceptHeader.Count != 0 && !acceptHeader.Contains(mediaType))
				{
					return new UnsupportedMediaTypeResult();
				}

				if (contentHash != null && Request.Headers.Range.Count == 0)
				{
					// send the hash of the object is we are fetching the full blob
					Response.Headers[CommonHeaders.HashHeaderName] = contentHash.ToString();
				}

				if (blobContents.RedirectUri != null)
				{
					return Redirect(blobContents.RedirectUri.ToString());
				}

				if (_nginxRedirectHelper.CanRedirect(Request, blobContents))
				{
					return _nginxRedirectHelper.CreateActionResult(blobContents, mediaType);
				}

				// IActionResult? response = _requestHelper.EnsureSingleRangeRequest(HttpContext);
				// if (response != null)
				// {
				// 	return response;
				// }
				return File(blobContents.Stream, mediaType, enableRangeProcessing: true);
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Blob {e.Blob} not found" });
			}
			catch (ContentIdResolveException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Content Id {e.ContentId} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPut("{ns}/{bucket}/{buildId}/parts/{partId}/blobs/{id}.{format?}")]
		[HttpPut("{ns}/{bucket}/{buildId}/blobs/{id}.{format?}")]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Octet)]

		public async Task<IActionResult> PutBlobAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[Required] BlobId id)
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (result != null)
			{
				return result;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);

			try
			{
				bool? bypassCache = _namespacePolicyResolver.GetPoliciesForNs(ns).BypassCacheOnWrite;
				
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-build-blob", HttpContext.RequestAborted);
				if (Request.ContentType == CustomMediaTypeNames.UnrealCompressedBuffer)
				{
					ContentId cid = ContentId.FromBlobIdentifier(id);

					ContentId identifier = await _blobStore.PutCompressedObjectAsync(ns, payload, cid, HttpContext.RequestServices, bucketHint: bucket, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);

					return Ok(new BlobUploadResponse(identifier.AsBlobIdentifier()));
				}
				else if (Request.ContentType == MediaTypeNames.Application.Octet)
				{
					Uri? uri = await _blobStore.MaybePutObjectWithRedirectAsync(ns, id, bucketHint: bucket, cancellationToken: HttpContext.RequestAborted);
					if (uri != null)
					{
						return Ok(new BlobUploadUriResponse(id, uri));
					}

					BlobId identifier = await _blobStore.PutObjectAsync(ns, payload, id, bucketHint: bucket, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);
					return Ok(new BlobUploadResponse(identifier));
				}
				else
				{
					throw new NotImplementedException("Unsupported mediatype: " + Request.ContentType);
				}
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			catch (ResourceHasToManyRequestsException)
			{
				return StatusCode(StatusCodes.Status429TooManyRequests);
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (TaskCanceledException)
			{
				return StatusCode(StatusCodes.Status408RequestTimeout, "Request cancelled");
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPost("{ns}/{bucket}/{buildId}/parts/{partId}/blobs/{id}/startMultipartUpload")]
		[HttpPost("{ns}/{bucket}/{buildId}/blobs/{id}/startMultipartUpload")]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]

		public async Task<IActionResult> StartBlobMultipartUploadAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromBody] [Required] StartMultipartUploadRequest request)
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (result != null)
			{
				return result;
			}
			
			if (!_blobStore.IsMultipartUploadSupported(ns))
			{
				return BadRequest("Multipart uploads not supported on any storage implementation in use");
			}

			(string? uploadId, string? blobTempName) = await _blobStore.StartMultipartUploadAsync(ns);
			if (uploadId == null || blobTempName == null)
			{
				throw new NotImplementedException("Failed to initialize multipart upload, no upload id returned");
			}

			// generate part ids for each chunk
			List<MultipartByteRange> ranges = _blobStore.GetMultipartRanges(ns, blobTempName, uploadId, request.BlobLength);
			List<MultipartPartDescription> parts = ranges.Select(r => new MultipartPartDescription {FirstByte = r.FirstByte, LastByte = r.LastByte, PartId = r.PartId, QueryString = $"?blobName={blobTempName}&uploadId={uploadId}&partId={r.PartId}"}).ToList();
			return Ok(new MultipartUploadIdResponse {UploadId = uploadId, BlobName = blobTempName, Parts = parts});
		}

		[HttpPut("{ns}/{bucket}/{buildId}/parts/{partId}/blobs/{id}/uploadMultipart")]
		[HttpPut("{ns}/{bucket}/{buildId}/blobs/{id}/uploadMultipart")]
		[DisableRequestSizeLimit]
		[RequiredContentType(MediaTypeNames.Application.Octet)]

		public async Task<IActionResult> PutBlobMultipartPartAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromQuery] [Required] string blobName,
			[FromQuery] [Required] string uploadId,
			[FromQuery] [Required] string partId,
			[FromQuery] bool supportsRedirect = false)
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (result != null)
			{
				return result;
			}

			if (!_blobStore.IsMultipartUploadSupported(ns))
			{
				return BadRequest("Multipart uploads not supported on any storage implementation in use");
			}

			if (supportsRedirect)
			{
				Uri? uri = await _blobStore.MaybePutMultipartUploadWithRedirectAsync(ns, blobName, uploadId, partId);
				if (uri != null)
				{
					return Redirect(uri.ToString());
				}
			}

			using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-blob-multipart", HttpContext.RequestAborted);
			await using Stream s = payload.GetStream();
			byte[] payloadData = await s.ReadAllBytesAsync();
			await _blobStore.PutMultipartUploadAsync(ns, blobName, uploadId, partId, payloadData);
			return Ok();
		}
		
		[HttpPost("{ns}/{bucket}/{buildId}/parts/{partId}/blobs/{id}/completeMultipart")]
		[HttpPost("{ns}/{bucket}/{buildId}/blobs/{id}/completeMultipart")]
		public async Task<IActionResult> CompleteBlobMultipartUploadAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute] [Required] BlobId id,
			[FromBody] [Required] CompleteMultipartUploadRequest request)
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (result != null)
			{
				return result;
			}

			if (!_blobStore.IsMultipartUploadSupported(ns))
			{
				return BadRequest("Multipart uploads not supported on any storage implementation in use");
			}

			List<string> missingParts = new List<string>();
			try
			{
				await _blobStore.CompleteMultipartUploadAsync(ns, request.BlobName, request.UploadId, request.PartIds);
			}
			catch (MissingMultipartPartsException e)
			{
				missingParts = e.MissingParts;
			}
			if (missingParts.Count > 0)
			{
				return BadRequest(new CompleteMultipartUploadResponse(missingParts));
			}

			try
			{
				await _blobStore.VerifyMultipartUpload(ns, id, request.BlobName, request.IsCompressed, cancellationToken: HttpContext.RequestAborted);
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			
			return Ok(new CompleteMultipartUploadResponse(missingParts));
			
		}
		#endregion

		#region Block endpoints

		[HttpPut("{ns}/{bucket}/{buildId}/parts/{partId}/blocks/{id}.{format?}")]
		[HttpPut("{ns}/{bucket}/{buildId}/blocks/{id}.{format?}")]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Octet)]
		public async Task<IActionResult> PutBlockAsync(NamespaceId ns, BucketId bucket, BlobId id)
		{
			return await PutBlobAsync(ns, bucket, id);
		}

		[HttpPut("{ns}/{bucket}/{buildId}/parts/{partId}/blocks/{id}/metadata")]
		[HttpPut("{ns}/{bucket}/{buildId}/blocks/{id}/metadata")]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> PutBlockMetadataAsync(NamespaceId ns, BucketId bucket, BlobId id)
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (result != null)
			{
				return result;
			}
			
			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);

			
			try
			{
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-block-metadata", HttpContext.RequestAborted);

				await using MemoryStream ms = new MemoryStream();
				await using Stream payloadStream = payload.GetStream();
				await payloadStream.CopyToAsync(ms);
				byte[] metadataBuffer = ms.ToArray();
				BlobId metadataBlockId = BlobId.FromBlob(metadataBuffer);
				CbObject metadataObject = new CbObject(metadataBuffer);
				ActionResult? verifyResult = VerifyBlockMetadata(metadataObject);
				if (verifyResult != null)
				{
					return verifyResult;
				}
				await _blobStore.PutObjectAsync(ns, payload, metadataBlockId, bucketHint: bucket, cancellationToken: HttpContext.RequestAborted);

				BlobId[]? blobIds = await _referenceResolver.ResolveIdAsync(ns, id, HttpContext.RequestAborted);
				if (blobIds == null)
				{
					return NotFound($"Unable to find block id {id}");
				}

				if (blobIds.Length > 1)
				{
					throw new NotSupportedException($"Block {id} has more than 1 part when resolved from a content id, this is not supported when used as metadata");
				}

				BlobId blockId = blobIds.First();

				bool blockExists = await _blobStore.ExistsAsync(ns, blockId);
				if (!blockExists)
				{
					return NotFound($"Unable to find block with id {blockId} as it does not exist, unable to upload metadata for it");
				}

				Tracer.CurrentSpan.SetAttribute("blockId", blockId.ToString());
				await _blockStore.PutBlockMetadataAsync(ns, blockId, metadataBlockId);

				// blob references track incoming references e.g. the block requires the metadata block to be kept around
				await _blobIndex.AddBlobReferencesAsync(ns, metadataBlockId, blockId);

				return Ok();
			}
			catch (ResourceHasToManyRequestsException)
			{
				return StatusCode(StatusCodes.Status429TooManyRequests);
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		private ActionResult? VerifyBlockMetadata(CbObject metadataObject)
		{
			if (metadataObject["rawHashes"].Equals(CbField.Empty))
			{
				return BadRequest("Missing \'rawHashes\' field in metadata");
			}

			return null;
		}

		[HttpGet("{ns}/{bucket}/{buildId}/parts/{partId}/blocks/{blockIdentifier}")]
		[HttpGet("{ns}/{bucket}/{buildId}/blocks/{blockIdentifier}")]
		[Produces(MediaTypeNames.Application.Octet, CustomMediaTypeNames.UnrealCompressedBuffer)]
		public async Task<IActionResult> GetBlockAsync(NamespaceId ns, BucketId bucket, ContentId blockIdentifier, [FromQuery] bool supportsRedirect = false)
		{
			return await GetBlobAsync(ns, bucket, blockIdentifier, supportsRedirect: supportsRedirect);
		}

		[HttpPost("{ns}/{bucket}/{buildId}/parts/{partId}/blocks/getBlockMetadata")]
		[HttpPost("{ns}/{bucket}/{buildId}/blocks/getBlockMetadata")]
		[Produces(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
		public async Task<IActionResult> GetBlockMetadataAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, GetBlockMetadataRequest request)
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (result != null)
			{
				return result;
			}

			BuildRecord? buildRecord = await GetBuildWithRetriesAsync(ns, bucket, buildId);
			if (buildRecord == null)
			{
				return NotFound($"Build {buildId} in bucket {bucket} and namespace {ns}");
			}
			CbWriter responseObject = new CbWriter();
			responseObject.BeginObject();
			responseObject.BeginArray("blocks");

			try
			{
				await Parallel.ForEachAsync(request.Blocks, async (blockHash, token) =>
				{
					// in case the block is a raw hash we need to resolve it into the actual hash of the block
					BlobId id = BlobId.FromIoHash(blockHash);
					BlobId[]? blobIds = await _referenceResolver.ResolveIdAsync(ns, id, HttpContext.RequestAborted);
					if (blobIds == null)
					{
						// if we are unable to resolve the block into an id we can not fetch it
						return;
					}

					BlobId blockId = blobIds.First();

					BlobId? metadataHash = await _blockStore.GetBlockMetadataAsync(ns, blockId);
					if (metadataHash == null)
					{
						// no metadata found for this id, likely not a block
						return;
					}

					try
					{
						await using BlobContents contents = await _blobStore.GetObjectAsync(ns, metadataHash, cancellationToken: token);
						byte[] b = await contents.Stream.ReadAllBytesAsync(cancellationToken: token);
						CbObject o = new CbObject(b);
						lock (responseObject)
						{
							responseObject.WriteObject(o);
						}
					}
					catch (BlobNotFoundException)
					{
						// if the metadata doesn't exist we can not display it
						_logger.LogWarning("Blob {Blob} is a block but unable to fetch the CAS object {CASBlob} thus unable to return it", blockId, metadataHash);
					}
				});
			}
			catch (AggregateException e)
			{
				if (e.InnerExceptions.Any(exception => exception.GetType() == typeof(RemoteServiceException)))
				{
					return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
				}

				throw;
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			responseObject.EndArray();
			responseObject.EndObject();

			string responseType = _formatResolver.GetResponseType(Request, null, CustomMediaTypeNames.UnrealCompactBinary);
			Tracer.CurrentSpan.SetAttribute("response-type", responseType);

			switch (responseType)
			{
				case CustomMediaTypeNames.UnrealCompactBinary:
					{
						Response.ContentType = CustomMediaTypeNames.UnrealCompactBinary;
						Response.StatusCode = 200;
						await Response.Body.WriteAsync(responseObject.ToByteArray());
						return new EmptyResult();
					}
				case MediaTypeNames.Application.Json:
					{
						CbObject o = responseObject.ToObject();
						string s = o.ToJson();
						Response.ContentType = MediaTypeNames.Application.Json;
						return Ok(s);
					}
				default:
					throw new NotImplementedException($"Unknown response type: {responseType}");
			}
		}

		[HttpGet("{ns}/{bucket}/{buildId}/parts/{partId}/blocks/listBlocks")]
		[HttpGet("{ns}/{bucket}/{buildId}/blocks/listBlocks")]
		[Produces(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
		public async Task<IActionResult> ListBlockAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string? format = null, [FromQuery] int count = 5000, [FromQuery] int skip = 0, [FromQuery] int maxConsidered = 25_000, [FromQuery] bool sort = true)
		{
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.EnumerateBucket });
			if (result != null)
			{
				return result;
			}

			BuildRecord? buildRecord = await GetBuildWithRetriesAsync(ns, bucket, buildId);
			if (buildRecord == null)
			{
				return NotFound($"Build {buildId} in bucket {bucket} and namespace {ns}");
			}

			async Task<bool> WriteBlockMetadata(BlockContext context, CbWriter writer)
			{
				using TelemetrySpan scope = _tracer.StartActiveSpan("list.blocks").SetAttribute("operation.name", "list.blocks");
				scope.SetAttribute("blockContext", context.ToString());

				// we sort these client side as a temporary solution as sorting these in Scylla has proven difficult
				List<(DateTime, BlobId)> metadataBlobs = new();

				int countOfBlocksFound = 0;
				bool blocksFound = false;
				{
					await foreach (BlockMetadata blockMetadata in _blockStore.ListBlockIndexAsync(ns, context).Take(maxConsidered))
					{
						blocksFound = true;
						lock (metadataBlobs)
						{
							metadataBlobs.Add((blockMetadata.LastUpdate, blockMetadata.MetadataBlobId));
						}
					}
				}

				IEnumerable<(DateTime, BlobId)> enumerable = metadataBlobs.AsEnumerable();
				if (sort)
				{
					enumerable = metadataBlobs.OrderByDescending(tuple => tuple.Item1);
				}

				ulong totalSize = 0;
				await Parallel.ForEachAsync(enumerable.Skip(skip).Take(count), async (pair, token) =>
				{
					(DateTime _, BlobId blockMetadata) = pair;
					Task<bool> blobExistsTask = _blobIndex.BlobExistsInRegionAsync(ns, blockMetadata, cancellationToken: token);
					try
					{
						await using BlobContents contents = await _blobStore.GetObjectAsync(ns, blockMetadata, cancellationToken: token);
						byte[] b = await contents.Stream.ReadAllBytesAsync(cancellationToken: token);
						if (!await blobExistsTask)
						{
							// we got blob contents but the blob is reported to not exist in the index
							// to recover we flag this block as not existing so that its content is uploaded again
							return;
						}

						CbObject o = new CbObject(b);
						lock (writer)
						{
							writer.WriteObject(o);

							int objectSize = o.GetSize();
							Interlocked.Add(ref totalSize, (ulong)objectSize);
							_logger.LogInformation("Adding object of size {ObjectSize} total size is now {TotalSize}", objectSize, totalSize);
						}

						Interlocked.Increment(ref countOfBlocksFound);
					}
					catch (BlobNotFoundException)
					{
						// we are unable to return any block that doesn't exist, we do clean this up here so that it's not found next time
						await _blockStore.DeleteBlockAsync(ns, blockMetadata);
					}
				});

				scope.SetAttribute("countOfBlocksFound", countOfBlocksFound);

				return blocksFound;
			}
			
			CbWriter responseObject = new CbWriter();
			responseObject.BeginObject();
			responseObject.BeginArray("blocks");

			BlockContext context = BlockContext.FromObject(buildRecord.BuildObject, useBaseBranch: false);

			try
			{
				bool blocksFound = await WriteBlockMetadata(context, responseObject);

				if (!blocksFound)
				{
					// no blocks found in the branch, fallback to the base branch
					BlockContext baseContext = BlockContext.FromObject(buildRecord.BuildObject, useBaseBranch: true);
					await WriteBlockMetadata(baseContext, responseObject);
				}
			}
			catch (AggregateException e)
			{
				if (e.InnerExceptions.Any(exception => exception.GetType() == typeof(RemoteServiceException)))
				{
					return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
				}

				throw;
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			responseObject.EndArray();
			responseObject.EndObject();

			string responseType = _formatResolver.GetResponseType(Request, format, CustomMediaTypeNames.UnrealCompactBinary);
			Tracer.CurrentSpan.SetAttribute("response-type", responseType);

			switch (responseType)
			{
				case CustomMediaTypeNames.UnrealCompactBinary:
					{
						Tracer.CurrentSpan.SetAttribute("response-length", responseObject.GetSize());
						Response.ContentType = CustomMediaTypeNames.UnrealCompactBinary;
						Response.StatusCode = 200;
						await Response.Body.WriteAsync(responseObject.ToByteArray());
						return new EmptyResult();
					}
				case MediaTypeNames.Application.Json:
					{
						CbObject o = responseObject.ToObject();
						string s = o.ToJson();
						Response.ContentType = MediaTypeNames.Application.Json;
						return Ok(s);
					}
				default:
					throw new NotImplementedException($"Unknown response type: {responseType}");
			}
		}
		
		#endregion

		#region Enumeration endpoints

		// adds a empty options endpoint for endpoints we expose to a webpage to pass cors preflight
		// this should be improved if we decide we are maintaining the admin pages
		[HttpOptions("")]
		[HttpOptions("{ns}")]
		[HttpOptions("{ns}/{bucket}/search")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> EmptyOptionsAsync()
		{
			await Task.CompletedTask;
			return NoContent();
		}

		[HttpGet("")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetNamespacesAsync()
		{
			NamespaceId[] namespaces;
			try
			{
				namespaces = await _refService.GetNamespacesAsync(HttpContext.RequestAborted).ToArrayAsync(HttpContext.RequestAborted);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			// filter namespaces down to only the namespaces the user has access to
			List<NamespaceId> namespacesWithAccess = new();
			foreach (NamespaceId ns in namespaces)
			{
				ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.EnumerateBucket });
				if (accessResult == null)
				{
					namespacesWithAccess.Add(ns);
				}
			}

			LogAuditEntry(HttpMethod.Get, nameof(GetNamespacesAsync), null, null, null, null);

			return Ok(new GetNamespacesResponse(namespacesWithAccess.ToArray()));
		}

		[HttpGet("{ns}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetBucketsAsync(NamespaceId ns)
		{
			ActionResult? nsAccessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.EnumerateBucket });

			if (nsAccessResult != null)
			{
				return new ForbidResult();
			}
			
			LogAuditEntry(HttpMethod.Get, nameof(GetBucketsAsync), ns, null, null, null);

			BucketId[] potentialBuckets = await _refService.GetBucketsAsync(ns, HttpContext.RequestAborted).ToArrayAsync(HttpContext.RequestAborted);

			// filter buckets down to only the ones the user has access to
			List<BucketId> verifiedBuckets = new();
			foreach (BucketId bucket in potentialBuckets)
			{
				ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.EnumerateBucket });
				if (accessResult == null)
				{
					verifiedBuckets.Add(bucket);
				}
			}

			return Ok(new GetBucketsResponse(verifiedBuckets.ToArray()));
		}

		#endregion

		private void LogAuditEntry(HttpMethod method, string endpoint, NamespaceId? ns, BucketId? bucketId, CbObjectId? buildId, string? partId)
		{
			string sessionHeader = "None";
			if (Request?.Headers == null)
			{
				// we have seen null dereference when processing the request, adding this logging to try and track down the cause
				_logger.LogWarning("Request was null");
			}

			if (Request!.Headers.TryGetValue("ue-session", out StringValues sessionHeaderValues))
			{
				sessionHeader = sessionHeaderValues.FirstOrDefault() ?? "None";
			}

			string jti = User.Claims.FirstOrDefault(claim => claim.Type == "jti")?.Value ?? "<unknown>";

			string userAgentString = string.Join(' ', Request.Headers.UserAgent.ToArray());
			string username = User.Identity?.Name ?? "Unknown-user";
			string ip = Request.HttpContext.Connection.RemoteIpAddress?.ToString() ?? "Unknown";

			LogAuditEntry(method, endpoint, sessionHeader, username, userAgentString, ip, jti, ns, bucketId, buildId, partId);
		}

		private void LogAuditEntry(HttpMethod method, string endpoint, string sessionHeader, string username, string userAgent, string ip, string jti, NamespaceId? ns, BucketId? bucketId, CbObjectId? buildId, string? partId)
		{
			if (ns == null)
			{
				_auditLogger?.Information("{HttpMethod} {Endpoint} IP:{IP} User:{Username} UserAgent:\"{Useragent}\" SessionId: {SessionId} Jti:{Jti}", method, endpoint, ip, username, userAgent, sessionHeader, jti);
				return;
			}

			if (bucketId == null)
			{
				_auditLogger?.Information("{HttpMethod} {Endpoint} '{Namespace}' IP:{IP} User:{Username} UserAgent:\"{Useragent}\" SessionId: {SessionId} Jti:{Jti}", method, endpoint, ns, ip, username, userAgent, sessionHeader, jti);
				return;
			}

			if (partId == null || buildId == null)
			{
				_auditLogger?.Information("{HttpMethod} {Endpoint} '{Namespace}/{BucketId}' IP:{IP} User:{Username} UserAgent:\"{Useragent}\" SessionId: {SessionId} Jti:{Jti}", method, endpoint, ns, bucketId, ip, username, userAgent, sessionHeader, jti);
				return;
			}
			
			_auditLogger?.Information("{HttpMethod} {Endpoint} '{Namespace}/{BucketId}:{BuildId}' Part:{PartId} IP:{IP} User:{Username} UserAgent:\"{Useragent}\" SessionId: {SessionId} Jti:{Jti}", method, endpoint, ns, bucketId, buildId, partId, ip, username, userAgent, sessionHeader, jti);
		}
	}

	public class FileList
	{
		[CbField("totalSize")]
		public ulong TotalSize { get;set; }

		[CbField("paths")] 
		public string[] Paths { get; set; } = Array.Empty<string>();

		[CbField("rawHashes")]
		public IoHash[] RawHashes { get; set; } = Array.Empty<IoHash>();

		[CbField("rawSizes")]
		public ulong[] RawSizes { get;set; } = Array.Empty<ulong>();
		[CbField("attributes")]
		public uint[] Attributes { get; set; } = Array.Empty<uint>();
		[CbField("modes")]
		public uint[] Modes { get; set; } = Array.Empty<uint>();
	}

	public class BuildsSettings
	{
		public bool EnableAuditLog { get; set; }
	}

	public class GetBlockMetadataRequest
	{
		[CbField("blocks")] 
#pragma warning disable CA2227
		public List<IoHash> Blocks { get; set; } = new List<IoHash>();
#pragma warning restore CA2227
	}

	public class BuildPartStatsRequest
	{
		[CbField("floatStats")] 
#pragma warning disable CA2227
		public Dictionary<string, double> FloatStats { get; set; } = new Dictionary<string, double>();
#pragma warning restore CA2227
	}

	public class PutBuildResponse
	{
		[CbField("chunkSize")]
		public uint ChunkSize { get; set; }
	}

	public class StartMultipartUploadRequest
	{
		[CbField("blobLength")]
		public ulong BlobLength { get; set; }
	}

	public class CompleteMultipartUploadRequest
	{
		[CbField("blobName")]
		public string BlobName { get; set; } = null!;

		[CbField("uploadId")]
		public string UploadId { get; set; } = null!;

		[CbField("isCompressed")]
		public bool IsCompressed { get; set; }

		[CbField("partIds")]
#pragma warning disable CA2227
		public List<string> PartIds { get; set; } = new List<string>();
#pragma warning restore CA2227
	}

	public class CompleteMultipartUploadResponse
	{
		public CompleteMultipartUploadResponse()
		{
		}
		
		public CompleteMultipartUploadResponse(List<string> missingParts)
		{
			MissingParts = missingParts;
		}

		[CbField("missingParts")]
#pragma warning disable CA2227
		public List<string> MissingParts { get; set; } = new List<string>();
#pragma warning restore CA2227

	}

	public class MultipartUploadIdResponse
	{
		[CbField("uploadId")]
		public string UploadId { get; set; } = null!;

		[CbField("blobName")]
		public string BlobName { get; set; } = null!;
		
		[CbField("parts")]
#pragma warning disable CA2227
		public List<MultipartPartDescription> Parts { get; set; } = new List<MultipartPartDescription>();
#pragma warning restore CA2227
	}

	public class MultipartPartDescription
	{
		[CbField("firstByte")]
		public ulong FirstByte { get; set; }
		[CbField("lastByte")]
		public ulong LastByte { get; set; }
		[CbField("partId")]
		public string PartId { get; set; } = null!;
		[CbField("queryString")]
		public string QueryString { get; set; } = null!;
	}

	public class GetTTLResponse
	{
		[CbField("ttl")]
		public uint TTL { get; set; }
	}

	public class UpdateTTLRequest
	{
		/// <summary>
		/// New TTL to set in seconds, lowest valid value is 120 seconds and max is 1 year (31536000 seconds)
		/// </summary>
		[Range(120, 31536000)]
		[Required]
		[CbField("ttl")]
		public uint TTL { get; set; }
	}

	public class CopyBuildResponse
	{
		[CbField("buildId")]
		public CbObjectId BuildId { get; set; }
	}

	public class CopyBuildRequest
	{
		[CbField("bucketId")]
		public BucketId NewBucket { get; set; }

		[CbField("newNamespace")] 
		public NamespaceId? NewNamespace { get; set; } = null;

		[CbField("newBranch")]
		public string? NewBranch { get; set; }

		[CbField("newProject")]
		public string? NewProject { get; set; }

		[CbField("newPlatform")]
		public string? NewPlatform { get; set; }
	}

	public class ListAppendixFileResponse
	{
		[CbField("appendices")]
#pragma warning disable CA2227
		public List<string> Appendices { get; set; } = null!;
#pragma warning restore CA2227
	}

	public class BuildSearchResult
	{
		[CbField("buildId")]
		public CbObjectId BuildId { get; set; }

		[CbField("ttl")]
		public uint? TTL { get; set; }
		[CbField("metadata")]
		public CbObject Metadata { get; set; } = null!;

		[CbField("isFinalized")]
		public bool IsFinalized { get; set; } = true;
	}
	
	public class SearchResult
	{
#pragma warning disable CA2227
		[CbField("results")]
		[CbConverter(typeof(CbUniformObjectListConverter<BuildSearchResult>))]
		public List<BuildSearchResult> Results { get; set; }
#pragma warning restore CA2227

		[CbField("partialResult")]
		public bool PartialResult { get; set; }

		public SearchResult()
		{
			Results = new List<BuildSearchResult>();
			PartialResult = false;
		}

		public SearchResult(List<BuildMetadata> foundResults, bool partialResult)
		{
			PartialResult = partialResult;
			Results = foundResults.Select(b => new BuildSearchResult {BuildId = b.BuildId, Metadata = b.Metadata, TTL = b.Ttl, IsFinalized = b.IsFinalized}).ToList();
		}
	}

	public class SearchOptions
	{
		[CbField("limit")]
		public int Limit { get; set; } = 500;

		[CbField("skip")]
		public int Skip { get; set; } = 0;

		[CbField("max")]
		public int Max { get; set; } = 40_000;

		[CbField("maxTimeMS")]
		public uint MaxTimeMs { get; set; } = Debugger.IsAttached ? 0u : 120_000; // default to 2 minutes max per search unless debugging in which case we disable it

		[CbField("includeTTL")]
		public bool IncludeTTL { get; set; } = false;

		
		// a custom option to include builds that are not finalized, e.g. they are only partially uploaded
		[CbField("includePartial")]
		public bool IncludePartial { get; set; } = false;
	}

	public class SearchRequest
	{
		// ReSharper disable once AutoPropertyCanBeMadeGetOnly.Global
#pragma warning disable CA2227
		[CbField("query")] [JsonIgnore] public CbObject? QueryCB { get; set; } = null;
#pragma warning restore CA2227

		// ReSharper disable once AutoPropertyCanBeMadeGetOnly.Global
#pragma warning disable CA2227
		[CbIgnore]
		public object Query { get; set; } = new();
#pragma warning restore CA2227

		[CbField("options")]
		public SearchOptions Options { get; set; } = new ();

		internal List<ISearchOp> ToSearchOps()
		{
			List<ISearchOp> ops = new List<ISearchOp>();
			if (Query is JsonElement jsonElement)
			{
				foreach (JsonProperty prop in jsonElement.EnumerateObject())
				{
					ops.AddRange(SearchOpHelpers.Parse(prop));
				}
			}

			if (QueryCB != null)
			{
				foreach (CbField field in QueryCB)
				{
					ops.AddRange(SearchOpHelpers.Parse(field));
				}
			}

			return ops;
		}
	}
	public class SearchBucketsRequest : SearchRequest
	{
		[CbField("bucketRegex")]
		public string BucketRegex { get; set; } = ".*"; // find all buckets if no regex has been specified
	}

	public class BuildSearchResultBucket
	{
		[CbField("buildId")]
		public CbObjectId BuildId { get; set; }

		[CbField("bucketId")]
		public BucketId BucketId { get; set; }

		[CbField("ttl")]
		public uint? TTL { get; set; }

		[CbField("metadata")]
		public CbObject Metadata { get; set; } = null!;
	}

	public class SearchMultipleBucketsResult
	{
#pragma warning disable CA2227
		[CbField("results")]
		[CbConverter(typeof(CbUniformObjectListConverter<BuildSearchResultBucket>))]
		public List<BuildSearchResultBucket> Results { get; set; }
#pragma warning restore CA2227

		[CbField("partialResult")]
		public bool PartialResult { get; set; }

		public SearchMultipleBucketsResult()
		{
			Results = new List<BuildSearchResultBucket>();
			PartialResult = false;
		}

		public SearchMultipleBucketsResult(List<BuildMetadata> foundResults, bool partialResult)
		{
			PartialResult = partialResult;
			Results = foundResults.Select(b => new BuildSearchResultBucket {BuildId = b.BuildId, BucketId = b.Bucket, Metadata = b.Metadata, TTL = b.Ttl}).ToList();
		}
	}
}
