// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Linq;
using System.Net;
using System.Net.Mime;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Serialization;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Implementation;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Primitives;
using OpenTelemetry.Trace;
using ContentId = Jupiter.Implementation.ContentId;

namespace Jupiter.Controllers
{
	using BlobNotFoundException = BlobNotFoundException;
	using IDiagnosticContext = Serilog.IDiagnosticContext;

	[ApiController]
	[Authorize]
	[Route("api/v1/compressed-blobs")]
	public class CompressedBlobController : ControllerBase
	{
		private readonly IBlobService _storage;
		private readonly IContentIdStore _contentIdStore;
		private readonly IDiagnosticContext _diagnosticContext;
		private readonly IRequestHelper _requestHelper;
		private readonly BufferedPayloadFactory _bufferedPayloadFactory;
		private readonly NginxRedirectHelper _nginxRedirectHelper;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;

		public CompressedBlobController(IBlobService storage, IContentIdStore contentIdStore, IDiagnosticContext diagnosticContext, IRequestHelper requestHelper, BufferedPayloadFactory bufferedPayloadFactory, NginxRedirectHelper nginxRedirectHelper, INamespacePolicyResolver namespacePolicyResolver)
		{
			_storage = storage;
			_contentIdStore = contentIdStore;
			_diagnosticContext = diagnosticContext;
			_requestHelper = requestHelper;
			_bufferedPayloadFactory = bufferedPayloadFactory;
			_nginxRedirectHelper = nginxRedirectHelper;
			_namespacePolicyResolver = namespacePolicyResolver;
		}

		[HttpGet("{ns}/{id}")]
		[ProducesResponseType(type: typeof(byte[]), 200)]
		[ProducesResponseType(type: typeof(ValidationProblemDetails), 400)]
		[Produces(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Octet)]

		public async Task<IActionResult> GetAsync(
			[Required] NamespaceId ns,
			[Required] ContentId id,
			[FromQuery] bool supportsRedirect = false)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			try
			{
				(BlobContents blobContents, string mediaType, BlobId? contentHash) = await _storage.GetCompressedObjectAsync(ns, id, HttpContext.RequestServices, supportsRedirectUri: supportsRedirect, cancellationToken: HttpContext.RequestAborted);

				StringValues acceptHeader = Request.Headers["Accept"];
				if (!acceptHeader.Contains("*/*") && acceptHeader.Count != 0 && !acceptHeader.Contains(mediaType))
				{
					return new UnsupportedMediaTypeResult();
				}

				if (contentHash != null && Request.Headers.Range.Count == 0)
				{
					// send the hash of the object is we are fetching the full blob
					Response.Headers[CommonHeaders.HashHeaderName] = contentHash.ToString();
				}

				if (blobContents.RedirectUri != null)
				{
					return Redirect(blobContents.RedirectUri.ToString());
				}

				if (_nginxRedirectHelper.CanRedirect(Request, blobContents))
				{
					return _nginxRedirectHelper.CreateActionResult(blobContents, mediaType);
				}

				return File(blobContents.Stream, mediaType, enableRangeProcessing: true);
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Object {e.Blob} not found" });
			}
			catch (ContentIdResolveException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Content Id {e.ContentId} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpHead("{ns}/{id}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> HeadAsync(
			[Required] NamespaceId ns,
			[Required] ContentId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			BlobId[]? chunks = await _contentIdStore.ResolveAsync(ns, id, mustBeContentId: false, cancellationToken: HttpContext.RequestAborted);
			if (chunks == null || chunks.Length == 0)
			{
				return NotFound();
			}

			Task<bool>[] tasks = new Task<bool>[chunks.Length];
			for (int i = 0; i < chunks.Length; i++)
			{
				tasks[i] = _storage.ExistsAsync(ns, chunks[i]);
			}

			await Task.WhenAll(tasks);

			bool exists = tasks.All(task => task.Result);

			if (!exists)
			{
				return NotFound();
			}

			return Ok();
		}

		[HttpPost("{ns}/exists")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> ExistsMultipleAsync(
			[Required] NamespaceId ns,
			[Required][FromQuery] List<ContentId> id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			ConcurrentBag<ContentId> partialContentIds = new ConcurrentBag<ContentId>();
			ConcurrentBag<ContentId> invalidContentIds = new ConcurrentBag<ContentId>();

			// we limit the concurrency to see if this helps reduce broken pipe / connection resets from nginx when this method is used
			await Parallel.ForEachAsync(id, new ParallelOptions {MaxDegreeOfParallelism = 4},async (blob, token) =>
			{
				BlobId[]? chunks = await _contentIdStore.ResolveAsync(ns, blob, mustBeContentId: false, cancellationToken: token);

				if (chunks == null)
				{
					invalidContentIds.Add(blob);
					return;
				}

				foreach (BlobId chunk in chunks)
				{
					if (!await _storage.ExistsAsync(ns, chunk, cancellationToken: token))
					{
						partialContentIds.Add(blob);
						break;
					}
				}
			});

			List<ContentId> needs = new List<ContentId>(invalidContentIds);
			needs.AddRange(partialContentIds);

			return Ok(new ExistCheckMultipleContentIdResponse { Needs = needs.ToArray() });
		}

		[HttpPost("{ns}/exist")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> ExistsBodyAsync(
			[Required] NamespaceId ns,
			[FromBody] ContentId[] bodyIds)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			ConcurrentBag<ContentId> partialContentIds = new ConcurrentBag<ContentId>();
			ConcurrentBag<ContentId> invalidContentIds = new ConcurrentBag<ContentId>();

			await Parallel.ForEachAsync(bodyIds, async (blob, token) =>
			{
				BlobId[]? chunks = await _contentIdStore.ResolveAsync(ns, blob, mustBeContentId: false, cancellationToken: token);

				if (chunks == null)
				{
					invalidContentIds.Add(blob);
					return;
				}

				foreach (BlobId chunk in chunks)
				{
					if (!await _storage.ExistsAsync(ns, chunk, cancellationToken: token))
					{
						partialContentIds.Add(blob);
						break;
					}
				}
			});

			List<ContentId> needs = new List<ContentId>(invalidContentIds);
			needs.AddRange(partialContentIds);

			return Ok(new ExistCheckMultipleContentIdResponse { Needs = needs.ToArray() });
		}

		[HttpPut("{ns}/{id}")]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompressedBuffer)]
		public async Task<IActionResult> PutAsync(
			[Required] NamespaceId ns,
			[Required] ContentId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());
			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);

			try
			{
				bool? bypassCache = _namespacePolicyResolver.GetPoliciesForNs(ns).BypassCacheOnWrite;
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-compressed", HttpContext.RequestAborted);

				ContentId identifier = await _storage.PutCompressedObjectAsync(ns, payload, id, HttpContext.RequestServices, HttpContext.RequestAborted, bypassCache: bypassCache);

				return Ok(new BlobUploadResponse(identifier.AsBlobIdentifier()));
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPost("{ns}")]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompressedBuffer)]
		public async Task<IActionResult> PostAsync(
			[Required] NamespaceId ns)
		{
			CancellationToken cancellationToken = HttpContext.RequestAborted;

			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}
			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);

			try
			{
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "post-compressed", cancellationToken);

				ContentId identifier = await _storage.PutCompressedObjectAsync(ns, payload, null, HttpContext.RequestServices, cancellationToken);

				return Ok(new
				{
					Identifier = identifier.ToString()
				});
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpDelete("{ns}/{id}")]
		public async Task<IActionResult> DeleteAsync(
			[Required] NamespaceId ns,
			[Required] ContentId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.DeleteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			BlobId[]? chunks = await _contentIdStore.ResolveAsync(ns, id, mustBeContentId: true);

			if (chunks == null)
			{
				return NotFound();
			}

			foreach (BlobId chunk in chunks)
			{
				try
				{
					await _storage.DeleteObjectAsync(ns, chunk);
				}
				catch (BlobNotFoundException)
				{
					// if the blob is already missing that is fine
				}
				catch (RemoteServiceException e)
				{
					return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
				}
			}

			// TODO: we should delete the cid from the content id store as well, but it has no delete operations yet
			return Ok();
		}
	}

	public class ExistCheckMultipleContentIdResponse
	{
		[CbField("needs")]
		public ContentId[] Needs { get; set; } = null!;
	}
}
