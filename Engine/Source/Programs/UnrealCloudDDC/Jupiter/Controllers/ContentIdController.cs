// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Net.Mime;
using System.Threading.Tasks;
using EpicGames.AspNet;
using Jupiter.Common;
using Jupiter.Implementation;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using ContentId = Jupiter.Implementation.ContentId;

namespace Jupiter.Controllers
{
	[ApiController]
	[FormatFilter]
	[Produces(MediaTypeNames.Application.Json, MediaTypeNames.Application.Octet, CustomMediaTypeNames.UnrealCompactBinary)]
	[Route("api/v1/content-id")]
	[Authorize]
	public class ContentIdController : ControllerBase
	{
		private readonly IRequestHelper _requestHelper;
		private readonly IContentIdStore _contentIdStore;

		public ContentIdController(IRequestHelper requestHelper, IContentIdStore contentIdStore)
		{
			_requestHelper = requestHelper;
			_contentIdStore = contentIdStore;
		}

		/// <summary>
		/// Returns which blobs a content id maps to
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance). Example: `uc4.ddc`</param>
		/// <param name="contentId">The content id to resolve </param>
		[HttpGet("{ns}/{contentId}.{format?}", Order = 500)]
		[ProducesDefaultResponseType]
		[ProducesResponseType(type: typeof(ProblemDetails), 400)]
		public async Task<IActionResult> ResolveAsync(NamespaceId ns, ContentId contentId)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				BlobId[]? blobs = await _contentIdStore.ResolveAsync(ns, contentId, mustBeContentId: true);

				if (blobs == null)
				{
					return NotFound(new ProblemDetails { Title = $"Unable to resolve content id {contentId} ({ns})." });
				}

				return Ok(new ResolvedContentIdResponse(blobs));
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		/// <summary>
		/// Update a content id mapping
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance). Example: `uc4.ddc`</param>
		/// <param name="contentId">The content id to resolve </param>
		/// <param name="blobIdentifier">The blob identifier to map the content id to</param>
		/// <param name="compressedSize">The compressed size of the blob</param>
		/// <param name="rawSize">The raw size of the blob</param>
		[HttpPut("{ns}/{contentId}/update/{blobIdentifier}/{compressedSize}/{rawSize}", Order = 500)]
		[ProducesDefaultResponseType]
		[ProducesResponseType(type: typeof(ProblemDetails), 400)]
		public async Task<IActionResult> UpdateContentIdMappingAsync(NamespaceId ns, ContentId contentId, BlobId blobIdentifier, ulong compressedSize, ulong rawSize)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				await _contentIdStore.PutAsync(ns, contentId, new BlobId[] {blobIdentifier}, compressedSize: compressedSize, rawSize: rawSize, HttpContext.RequestAborted);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return Ok();
		}
	}

	public class ResolvedContentIdResponse
	{
		public BlobId[] Blobs { get; set; }

		public ResolvedContentIdResponse(BlobId[] blobs)
		{
			Blobs = blobs;
		}
	}
}
