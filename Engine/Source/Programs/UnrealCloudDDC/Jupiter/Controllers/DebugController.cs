// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net;
using System.Threading;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace Jupiter.Controllers
{
	[ApiController]
	[Route("api/v1/c/_debug")]
	public class DebugController : Controller
	{
		private readonly IOptionsMonitor<DebugSettings> _settings;
		private int _currentConcurrentCalls;

		public DebugController(IOptionsMonitor<DebugSettings> settings)
		{
			_settings = settings;
		}
		/// <summary>
		/// Return bytes of specified length with auth, used for testing only
		/// </summary>
		/// <returns></returns>
		[HttpGet("getBytes")]
		[Authorize]
		public IActionResult GetBytes([FromQuery] int length = 1)
		{
			if (!_settings.CurrentValue.EnableDebugEndpoints)
			{
				return NotFound();
			}

			try
			{
				int concurrentCalls = Interlocked.Increment(ref _currentConcurrentCalls);
				if (concurrentCalls > _settings.CurrentValue.MaxConcurrentCalls)
				{
					return StatusCode((int)HttpStatusCode.TooManyRequests);
				}

				return GenerateByteResponse(length);
			}
			finally
			{
				Interlocked.Decrement(ref _currentConcurrentCalls);
			}	
		}

		/// <summary>
		/// Return bytes of specified length without auth, used for testing only
		/// </summary>
		/// <returns></returns>
		[HttpGet("getBytesWithoutAuth")]
		public IActionResult GetBytesWithoutAuth([FromQuery] int length = 1)
		{
			if (!_settings.CurrentValue.EnableDebugEndpoints)
			{
				return NotFound();
			}

			try
			{
				int concurrentCalls = Interlocked.Increment(ref _currentConcurrentCalls);
				if (concurrentCalls > _settings.CurrentValue.MaxConcurrentCalls)
				{
					return StatusCode((int)HttpStatusCode.TooManyRequests);
				}
				return GenerateByteResponse(length);
			}
			finally
			{
				Interlocked.Decrement(ref _currentConcurrentCalls);
			}
		}

		private FileContentResult GenerateByteResponse(int length)
		{
			byte[] generatedData = new byte[length];
			Array.Fill(generatedData, (byte)'J');
			return File(generatedData, "application/octet-stream");
		}
	}

	public class DebugSettings
	{
		public bool EnableDebugEndpoints { get; set; } = false;
		public int MaxConcurrentCalls { get; set; } = 10;
	}
}
