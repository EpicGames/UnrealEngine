// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Net.Mime;
using EpicGames.AspNet;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;
using Microsoft.Extensions.Primitives;

namespace Jupiter.Controllers
{
	public class FormatResolver
	{
		private readonly IOptionsMonitor<MvcOptions> _mvcOptions;

		private readonly string[] _validContentTypes = {
			MediaTypeNames.Application.Octet,
			MediaTypeNames.Application.Json,
			CustomMediaTypeNames.UnrealCompactBinary,
			CustomMediaTypeNames.JupiterInlinedPayload,
			CustomMediaTypeNames.UnrealCompressedBuffer,
			CustomMediaTypeNames.UnrealCompactBinaryPackage
		};

		public FormatResolver(IOptionsMonitor<MvcOptions> mvcOptions)
		{
			_mvcOptions = mvcOptions;
		}

		public string GetResponseType(HttpRequest request, string? format, string defaultContentType)
		{
			// if format specifier is used it takes precedence over the accept header
			if (format != null)
			{
				string? typeMapping = _mvcOptions.CurrentValue.FormatterMappings.GetMediaTypeMappingForFormat(format);
				if (typeMapping == null)
				{
					throw new Exception($"No mapping defined from format {format} to mime type");
				}

				return typeMapping;
			}

			StringValues acceptHeader = request.Headers["Accept"];

			if (acceptHeader.Count == 0)
			{
				// no accept header specified, return default type
				return defaultContentType;
			}

			// */* means to accept anything, so we use the default content type
			if (acceptHeader == "*/*")
			{
				return defaultContentType;
			}

			foreach (string? header in acceptHeader)
			{
				if (header == null)
				{
					continue;
				}

				if (_validContentTypes.Contains(header, StringComparer.OrdinalIgnoreCase))
				{
					return header;
				}
			}

			throw new Exception($"Unable to determine response type for header: {acceptHeader}");
		}
	}
}
