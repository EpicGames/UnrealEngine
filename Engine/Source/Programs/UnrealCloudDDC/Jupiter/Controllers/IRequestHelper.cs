// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Security.Claims;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;

namespace Jupiter.Controllers
{
	public interface IRequestHelper
	{
		public Task<ActionResult?> HasAccessToScopeAsync(ClaimsPrincipal user, HttpRequest request, AccessScope scope, JupiterAclAction[] aclActions);

		public bool IsPublicPort(HttpContext context);
		IActionResult? EnsureSingleRangeRequest(HttpContext context);
	}

	public class AuthorizationException : Exception
	{
		public ActionResult Result { get; }

		public AuthorizationException(ActionResult result, string errorMessage) : base(errorMessage)
		{
			Result = result;
		}
	}
}
