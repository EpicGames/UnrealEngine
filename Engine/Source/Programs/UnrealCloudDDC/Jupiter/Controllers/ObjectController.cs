// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Linq;
using System.Net;
using System.Net.Mime;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Serialization;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Implementation;
using Jupiter.Utils;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;

namespace Jupiter.Controllers
{
	using BlobNotFoundException = Jupiter.Implementation.BlobNotFoundException;
	using IDiagnosticContext = Serilog.IDiagnosticContext;

	[ApiController]
	[Route("api/v1/objects", Order = 0)]
	[Authorize]
	[Produces(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Json)]
	public class ObjectController : ControllerBase
	{
		private readonly IBlobService _storage;
		private readonly IDiagnosticContext _diagnosticContext;
		private readonly IRequestHelper _requestHelper;
		private readonly IReferenceResolver _referenceResolver;
		private readonly BufferedPayloadFactory _bufferedPayloadFactory;

		private readonly ILogger _logger;

		public ObjectController(IBlobService storage, IDiagnosticContext diagnosticContext, IRequestHelper requestHelper, IReferenceResolver referenceResolver, BufferedPayloadFactory bufferedPayloadFactory, ILogger<ObjectController> logger)
		{
			_storage = storage;
			_diagnosticContext = diagnosticContext;
			_requestHelper = requestHelper;
			_referenceResolver = referenceResolver;
			_bufferedPayloadFactory = bufferedPayloadFactory;
			_logger = logger;
		}

		[HttpGet("{ns}/{id}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetAsync(
			[Required] NamespaceId ns,
			[Required] BlobId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				BlobContents blobContents = await _storage.GetObjectAsync(ns, id, bucketHint: null);

				return File(blobContents.Stream, CustomMediaTypeNames.UnrealCompactBinary);
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Object {e.Blob} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpHead("{ns}/{id}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> HeadAsync(
			[Required] NamespaceId ns,
			[Required] BlobId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}
			
			try
			{
				bool exists = await _storage.ExistsAsync(ns, id);

				if (!exists)
				{
					return NotFound(new ValidationProblemDetails { Title = $"Object {id} not found" });
				}
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return Ok();
		}

		[HttpPost("{ns}/exists")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> ExistsMultipleAsync(
			[Required] NamespaceId ns,
			[Required][FromQuery] List<BlobId> id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			ConcurrentBag<BlobId> missingBlobs = new ConcurrentBag<BlobId>();

			IEnumerable<Task> tasks = id.Select(async blob =>
			{
				if (!await _storage.ExistsAsync(ns, blob))
				{
					missingBlobs.Add(blob);
				}
			});
			await Task.WhenAll(tasks);

			return Ok(new HeadMultipleResponse { Needs = missingBlobs.ToArray() });
		}

		[HttpPost("{ns}/exist")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> ExistsBodyAsync(
			[Required] NamespaceId ns,
			[FromBody] BlobId[] bodyIds)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			ConcurrentBag<BlobId> missingBlobs = new ConcurrentBag<BlobId>();

			IEnumerable<Task> tasks = bodyIds.Select(async blob =>
			{
				if (!await _storage.ExistsAsync(ns, blob))
				{
					missingBlobs.Add(blob);
				}
			});
			await Task.WhenAll(tasks);

			return Ok(new HeadMultipleResponse { Needs = missingBlobs.ToArray() });
		}

		[HttpPut("{ns}/{id}")]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary)]
		public async Task<IActionResult> PutAsync(
			[Required] NamespaceId ns,
			[Required] BlobId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);
			try
			{
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-object", HttpContext.RequestAborted);

				BlobId identifier = await _storage.PutObjectAsync(ns, payload, id, bucketHint: null, cancellationToken: HttpContext.RequestAborted);
				return Ok(new PutBlobResponse(identifier));
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpGet("{ns}/{id}/references")]
		public async Task<IActionResult> ResolveReferencesAsync(
			[Required] NamespaceId ns,
			[Required] BlobId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			BlobContents blob;
			try
			{
				blob = await _storage.GetObjectAsync(ns, id, bucketHint: null);
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Object {e.Blob} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			byte[] blobContents = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
			if (blobContents.Length == 0)
			{
				_logger.LogWarning("0 byte object found for {Id} {Namespace}", id, ns);
			}

			CbObject compactBinaryObject;
			try
			{
				compactBinaryObject = new CbObject(blobContents);
			}
			catch (IndexOutOfRangeException)
			{
				return Problem(title: $"{id} was not a proper compact binary object.", detail: "Index out of range");
			}

			try
			{
				BlobId[] references = await _referenceResolver.GetReferencedBlobsAsync(ns, compactBinaryObject).ToArrayAsync();
				return Ok(new ResolvedReferencesResult(references));
			}
			catch (PartialReferenceResolveException e)
			{
				return BadRequest(new ValidationProblemDetails { Title = $"Object {id} is missing content ids", Detail = $"Following content ids are invalid: {string.Join(",", e.UnresolvedReferences)}" });
			}
			catch (ReferenceIsMissingBlobsException e)
			{
				return BadRequest(new ValidationProblemDetails { Title = $"Object {id} is missing blobs", Detail = $"Following blobs are missing: {string.Join(",", e.MissingBlobs)}" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpDelete("{ns}/{id}")]
		public async Task<IActionResult> DeleteAsync(
			[Required] NamespaceId ns,
			[Required] BlobId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.DeleteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				await _storage.DeleteObjectAsync(ns, id, HttpContext.RequestAborted);

				return Ok(new DeletedResponse
				{
					DeletedCount = 1
				});
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpDelete("{ns}")]
		public async Task<IActionResult> DeleteNamespaceAsync(
			[Required] NamespaceId ns)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.DeleteNamespace });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				await _storage.DeleteNamespaceAsync(ns, HttpContext.RequestAborted);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return Ok();
		}
	}

	public class PutBlobResponse
	{
		public PutBlobResponse()
		{
			Identifier = null!;
		}

		public PutBlobResponse(BlobId identifier)
		{
			Identifier = identifier;
		}

		[CbField("identifier")]
		public BlobId Identifier { get; set; }
	}

	public class DeletedResponse
	{
		public int DeletedCount { get; set; }
	}

	public class ResolvedReferencesResult
	{
		public ResolvedReferencesResult()
		{
			References = null!;
		}

		public ResolvedReferencesResult(BlobId[] references)
		{
			References = references;
		}

		[CbField("references")]
		public BlobId[] References { get; set; }
	}
}
