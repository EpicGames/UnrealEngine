// Copyright Epic Games, Inc. All Rights Reserved.

namespace Jupiter.Controllers
{
	public static class ProblemTypes
	{
		public static string UseSnapshot { get; set; } = "http://jupiter.epicgames.com/replication/useSnapshot";
		public static string NoDataFound { get; set; } = "http://jupiter.epicgames.com/replication/noReplicationData";
		public static string HashMismatchError { get; set; } = "http://jupiter.epicgames.com/blob/hashMismatch";
	}
}
