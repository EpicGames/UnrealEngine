// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Mime;
using System.Text;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Core;
using EpicGames.Serialization;
using JetBrains.Annotations;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Implementation;
using Jupiter.Implementation.Blob;
using Jupiter.Utils;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Http.Extensions;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Primitives;
using OpenTelemetry.Trace;
using ContentHash = Jupiter.Implementation.ContentHash;
using ContentId = Jupiter.Implementation.ContentId;

namespace Jupiter.Controllers
{
	using IDiagnosticContext = Serilog.IDiagnosticContext;

	[ApiController]
	[FormatFilter]
	[Produces(MediaTypeNames.Application.Json, MediaTypeNames.Application.Octet, CustomMediaTypeNames.UnrealCompactBinary)]
	[Route("api/v1/refs")]
	[Authorize]
	public class ReferencesController : ControllerBase
	{
		private readonly IDiagnosticContext _diagnosticContext;
		private readonly FormatResolver _formatResolver;
		private readonly BufferedPayloadFactory _bufferedPayloadFactory;
		private readonly IReferenceResolver _referenceResolver;
		private readonly NginxRedirectHelper _nginxRedirectHelper;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly IRequestHelper _requestHelper;
		private readonly Tracer _tracer;

		private readonly ILogger _logger;
		private readonly IRefService _refService;
		private readonly IBlobService _blobStore;
		private readonly IBlobIndex _blobIndex;
		private readonly IPeerStatusService _peerStatusService;

		public ReferencesController(IRefService refService, IBlobService blobStore, IBlobIndex blobIndex, IPeerStatusService peerStatusService, IDiagnosticContext diagnosticContext, FormatResolver formatResolver, BufferedPayloadFactory bufferedPayloadFactory, IReferenceResolver referenceResolver, NginxRedirectHelper nginxRedirectHelper, INamespacePolicyResolver namespacePolicyResolver, IRequestHelper requestHelper, Tracer tracer, ILogger<ReferencesController> logger)
		{
			_refService = refService;
			_blobStore = blobStore;
			_blobIndex = blobIndex;
			_peerStatusService = peerStatusService;
			_diagnosticContext = diagnosticContext;
			_formatResolver = formatResolver;
			_bufferedPayloadFactory = bufferedPayloadFactory;
			_referenceResolver = referenceResolver;
			_nginxRedirectHelper = nginxRedirectHelper;
			_namespacePolicyResolver = namespacePolicyResolver;
			_requestHelper = requestHelper;
			_tracer = tracer;
			_logger = logger;
		}

		/// <summary>
		/// Returns all the known namespace the token has access to
		/// </summary>
		/// <returns></returns>
		[HttpGet("")]
		[ProducesDefaultResponseType]
		[ProducesResponseType(type: typeof(ProblemDetails), 400)]
		public async Task<IActionResult> GetNamespacesAsync()
		{
			try
			{
				NamespaceId[] namespaces = await _refService.GetNamespacesAsync(HttpContext.RequestAborted).ToArrayAsync(HttpContext.RequestAborted);

				// filter namespaces down to only the namespaces the user has access to
				List<NamespaceId> namespacesWithAccess = new();
				foreach (NamespaceId ns in namespaces)
				{
					ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
					if (accessResult == null)
					{
						namespacesWithAccess.Add(ns);
					}
				}

				if (!namespacesWithAccess.Any())
				{
					return new ForbidResult();
				}

				return Ok(new GetNamespacesResponse(namespacesWithAccess.ToArray()));
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		/// <summary>
		/// Returns a refs key
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance). Example: `uc4.ddc`</param>
		/// <param name="bucket">The category/type of record you are caching. Is a clustered key together with the actual key, but all records in the same bucket can be dropped easily. Example: `terrainTexture` </param>
		/// <param name="key">The unique name of this particular key. `iAmAVeryValidKey`</param>
		/// <param name="format">Optional specifier to set which output format is used json/raw/cb</param>
		/// <param name="forceConsistentRead">Optional header to force consistent reads disabling all uses of caches</param>
		[HttpGet("{ns}/{bucket}/{key}.{format?}", Order = 500)]
		[Produces(MediaTypeNames.Application.Json, MediaTypeNames.Application.Octet, CustomMediaTypeNames.UnrealCompactBinary, CustomMediaTypeNames.JupiterInlinedPayload, CustomMediaTypeNames.UnrealCompactBinaryPackage)]
		public async Task<IActionResult> GetAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key,
			[FromRoute] string? format = null,
			[FromHeader(Name = CommonHeaders.ForceConsistentRead)] bool forceConsistentRead = false)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
				Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

				(RefRecord objectRecord, BlobContents? maybeBlob) = await _refService.GetAsync(ns, bucket, key, Array.Empty<string>(), consistentRead: forceConsistentRead);

				if (maybeBlob == null)
				{
					throw new InvalidOperationException($"Blob was null when attempting to fetch {ns} {bucket} {key}");
				}

				await using BlobContents blob = maybeBlob;

				if (!objectRecord.IsFinalized)
				{
					// we do not consider un-finalized objects as valid
					return BadRequest(new ProblemDetails { Title = $"Object {objectRecord.Bucket} {objectRecord.Name} is not finalized." });
				}

				Response.Headers[CommonHeaders.HashHeaderName] = objectRecord.BlobIdentifier.ToString();
				Response.Headers[CommonHeaders.LastAccessHeaderName] = objectRecord.LastAccess.ToString(CultureInfo.InvariantCulture);

				async Task WriteBody(BlobContents blobContents, string contentType)
				{
					IServerTiming? serverTiming = Request.HttpContext.RequestServices.GetService<IServerTiming>();
					using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope("body.write", "Time spent writing body");

					long contentLength = blobContents.Length;
					using TelemetrySpan scope = _tracer.StartActiveSpan("body.write").SetAttribute("operation.name", "body.write");
					scope.SetAttribute("content-length", contentLength);
					const int BufferSize = 64 * 1024;
					Stream outputStream = Response.Body;
					Response.ContentLength = contentLength;
					Response.ContentType = contentType;
					Response.StatusCode = StatusCodes.Status200OK;
					try
					{
						await StreamCopyOperation.CopyToAsync(blobContents.Stream, outputStream, count: null, bufferSize: BufferSize, cancel: Response.HttpContext.RequestAborted);
					}
					catch (OperationCanceledException)
					{
						// do not raise exceptions for cancelled writes
						// as we have already started writing a response we can not change the status code
						// so we just drop a warning and proceed
						_logger.LogWarning("The operation was canceled while writing the body");
					}
				}

				string responseType = _formatResolver.GetResponseType(Request, format, CustomMediaTypeNames.UnrealCompactBinary);
				Tracer.CurrentSpan.SetAttribute("response-type", responseType);

				switch (responseType)
				{
					case CustomMediaTypeNames.UnrealCompactBinary:
						{
							if (_nginxRedirectHelper.CanRedirect(Request, blob))
							{
								return _nginxRedirectHelper.CreateActionResult(blob, responseType);
							}

							// for compact binary we can just serialize our internal object
							await WriteBody(blob, CustomMediaTypeNames.UnrealCompactBinary);

							break;
						}
					case MediaTypeNames.Application.Octet:
						{
							byte[] blobMemory = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
							CbObject cb = new CbObject(blobMemory);

							(int, CbField?) CountFields(CbObject o)
							{
								int count = 0;
								CbField? foundField = null;
								o.IterateAttachments(field =>
								{
									++count;
									if (field.IsBinaryAttachment())
									{
										foundField = field;
									}
								});

								return (count, foundField);
							}

							// breaking lambda call into private method to workaround incorrect triggering of CA1508 - https://github.com/dotnet/roslyn-analyzers/issues/5254
							(int countOfAttachmentFields, CbField? binaryAttachmentField) = CountFields(cb);

							if (countOfAttachmentFields == 1 && binaryAttachmentField != null)
							{
								// there is a single attachment field and that is of the binary attachment type, fetch that attachment and return it instead of the compact binary
								// this is so that we match the uploaded that happened as a octet-stream which generates a small cb object with a single attachment

								IoHash hash = binaryAttachmentField.AsBinaryAttachment();

								BlobContents referencedBlobContents = await _blobStore.GetObjectAsync(ns, BlobId.FromIoHash(hash), bucketHint: bucket);

								if (_nginxRedirectHelper.CanRedirect(Request, referencedBlobContents))
								{
									return _nginxRedirectHelper.CreateActionResult(referencedBlobContents, MediaTypeNames.Application.Octet);
								}
								await WriteBody(referencedBlobContents, MediaTypeNames.Application.Octet);
								break;
							}

							// this doesn't look like the generated compact binary so we just return the payload
							await using BlobContents contents = new(blobMemory);
							await WriteBody(contents, MediaTypeNames.Application.Octet);
							break;
						}
					case MediaTypeNames.Application.Json:
						{
							byte[] blobMemory;
							{
								using TelemetrySpan scope = _tracer.StartActiveSpan("json.readblob").SetAttribute("operation.name", "json.readblob");
								blobMemory = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
							}
							CbObject cb = new CbObject(blobMemory);
							string s = cb.ToJson();
							await using BlobContents contents = new BlobContents(Encoding.UTF8.GetBytes(s));
							await WriteBody(contents, MediaTypeNames.Application.Json);
							break;

						}
					case CustomMediaTypeNames.UnrealCompactBinaryPackage:
						{
							using TelemetrySpan packageScope = _tracer.StartActiveSpan("cbpackage.fetch").SetAttribute("operation.name", "cbpackage.fetch");
							byte[] blobMemory = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
							CbObject cb = new CbObject(blobMemory);

							IAsyncEnumerable<Attachment> attachments = _referenceResolver.GetAttachmentsAsync(ns, cb, HttpContext.RequestAborted);

							using CbPackageBuilder writer = new CbPackageBuilder();
							writer.AddAttachment(objectRecord.BlobIdentifier.AsIoHash(), CbPackageAttachmentFlags.IsObject, blobMemory);

							await Parallel.ForEachAsync(attachments, async (attachment, token) =>
							{
								IoHash attachmentHash = attachment.AsIoHash();
								CbPackageAttachmentFlags flags = 0;

								try
								{
									BlobContents attachmentContents;
									if (attachment is BlobAttachment blobAttachment)
									{
										BlobId referencedBlob = blobAttachment.Identifier;
										attachmentContents = await _blobStore.GetObjectAsync(ns, referencedBlob, bucketHint: bucket, cancellationToken: HttpContext.RequestAborted);
									}
									else if (attachment is ObjectAttachment objectAttachment)
									{
										flags |= CbPackageAttachmentFlags.IsObject;
										BlobId referencedBlob = objectAttachment.Identifier;
										attachmentContents = await _blobStore.GetObjectAsync(ns, referencedBlob, bucketHint: bucket, cancellationToken: HttpContext.RequestAborted);
									}
									else if (attachment is ContentIdAttachment contentIdAttachment)
									{

										ContentId contentId = contentIdAttachment.Identifier;
										(attachmentContents, string mime, BlobId? _) = await _blobStore.GetCompressedObjectAsync(ns, contentId, HttpContext.RequestServices, cancellationToken: HttpContext.RequestAborted);
										if (mime == CustomMediaTypeNames.UnrealCompressedBuffer)
										{
											flags |= CbPackageAttachmentFlags.IsCompressed;
										}
										else
										{
											// this resolved to a uncompressed blob, the content id existed the the compressed blob didn't
											// so resetting flags to indicate this.
											flags = 0;
										}
									}
									else
									{
										throw new NotSupportedException($"Unknown attachment type {attachment.GetType()}");
									}

									writer.AddAttachment(attachmentHash, flags, attachmentContents.Stream, (ulong)attachmentContents.Length);
								}
								catch (Exception e)
								{
									(CbObject errorObject, HttpStatusCode _) = ToErrorResult(e);
									writer.AddAttachment(attachmentHash,
										CbPackageAttachmentFlags.IsError | CbPackageAttachmentFlags.IsObject,
										errorObject.GetView().ToArray());
								}
							});

							byte[] packageBytes;
							{
								using TelemetrySpan _ = _tracer.StartActiveSpan("cbpackage.buffer").SetAttribute("operation.name", "cbpackage.buffer");
								packageBytes = await writer.ToByteArrayAsync();
							}
							await using BlobContents contents = new BlobContents(packageBytes);
							await WriteBody(contents, CustomMediaTypeNames.UnrealCompactBinaryPackage);
							break;
						}
					case CustomMediaTypeNames.JupiterInlinedPayload:
						{
							byte[] blobMemory = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
							CbObject cb = new CbObject(blobMemory);

							static (int, int) CountFields(CbObject o)
							{
								int countOfBinaryAttachmentFields = 0;
								int countOfAttachmentFields = 0;

								o.IterateAttachments(field =>
								{
									if (field.IsBinaryAttachment())
									{
										++countOfBinaryAttachmentFields;
									}

									if (field.IsAttachment())
									{
										++countOfAttachmentFields;
									}
								});

								return (countOfBinaryAttachmentFields, countOfAttachmentFields);
							}
							// breaking lambda call into private method to workaround incorrect triggering of CA1508 - https://github.com/dotnet/roslyn-analyzers/issues/5254
							(int countOfAttachmentFields, int countOfBinaryAttachmentFields) = CountFields(cb);

							// if the object consists of a single attachment field we return this attachment field instead
							if (countOfBinaryAttachmentFields == 1 && countOfAttachmentFields == 1)
							{
								// fetch the blob so we can resolve any content ids in it
								List<BlobId> referencedBlobs;
								try
								{
									IAsyncEnumerable<BlobId> referencedBlobsEnumerable = _referenceResolver.GetReferencedBlobsAsync(ns, cb);
									referencedBlobs = await referencedBlobsEnumerable.ToListAsync();
								}
								catch (PartialReferenceResolveException)
								{
									Tracer.CurrentSpan.SetAttribute("error-type", "MissingContentIds");
									return NotFound(new ProblemDetails { Title = $"Object {bucket} {key} in namespace {ns} was missing some content ids" });
								}
								catch (ReferenceIsMissingBlobsException e)
								{
									BlobId? blobId = e.MissingBlobs.FirstOrDefault();
									Tracer.CurrentSpan.SetAttribute("error-type", "MissingBlobs");
									return NotFound(new ProblemDetails { Title = $"Object {bucket} {key} in namespace {ns} was missing {e.MissingBlobs.Count} blobs. Sample: {blobId}" });
								}

								if (referencedBlobs.Count == 1)
								{
									BlobId attachmentToSend = referencedBlobs.First();
									try
									{
										BlobContents referencedBlobContents = await _blobStore.GetObjectAsync(ns, attachmentToSend, bucketHint: bucket);
										Response.Headers[CommonHeaders.InlinePayloadHash] = attachmentToSend.ToString();

										if (_nginxRedirectHelper.CanRedirect(Request, referencedBlobContents))
										{
											return _nginxRedirectHelper.CreateActionResult(referencedBlobContents, CustomMediaTypeNames.JupiterInlinedPayload);
										}
										await WriteBody(referencedBlobContents, CustomMediaTypeNames.JupiterInlinedPayload);
									}
									catch (BlobNotFoundException)
									{
										return NotFound(new ProblemDetails { Title = $"Object {bucket} {key} in namespace {ns} was missing blob {attachmentToSend}" });
									}
									catch (Exception ex)
									{
										Tracer.CurrentSpan.SetStatus(Status.Error);
										Tracer.CurrentSpan.RecordException(ex);
										_logger.LogError(ex, "Unknown exception encountered while writing body for jupiter inlined payload.");
										throw;
									}
									return new EmptyResult();
								}
								else if (referencedBlobs.Count == 0)
								{
									return NotFound(new ProblemDetails
									{
										Title =
											$"Object {objectRecord.Bucket} {objectRecord.Name} did not resolve into any objects that we could find."
									});
								}

								return BadRequest(new ProblemDetails
								{
									Title =
										$"Object {objectRecord.Bucket} {objectRecord.Name} contained a content id which resolved to more then 1 blob, unable to inline this object. Use compact object response instead."
								});
							}
							else if (countOfBinaryAttachmentFields == 0 && countOfAttachmentFields == 0)
							{
								// no attachments so we just return the compact object instead
								await using BlobContents contents = new BlobContents(blobMemory);
								await WriteBody(contents, CustomMediaTypeNames.JupiterInlinedPayload);
								return new EmptyResult();
							}

							return BadRequest(new ProblemDetails
							{
								Title =
									$"Object {objectRecord.Bucket} {objectRecord.Name} had more then 1 binary attachment field, unable to inline this object. Use compact object response instead."
							});
						}
					default:
						throw new NotImplementedException($"Unknown expected response type {responseType}");
				}

				// this result is ignored as we write to the body explicitly
				return new EmptyResult();

			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RefNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Object {e.Bucket} {e.Key} did not exist" });
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Object {e.Blob} in {e.Ns} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		/// <summary>
		/// Returns the metadata about a ref key
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance). Example: `uc4.ddc`</param>
		/// <param name="bucket">The category/type of record you are caching. Is a clustered key together with the actual key, but all records in the same bucket can be dropped easily. Example: `terrainTexture` </param>
		/// <param name="key">The unique name of this particular key. `iAmAVeryValidKey`</param>
		/// <param name="fields">The fields to include in the response, omit this to include everything.</param>
		[HttpGet("{ns}/{bucket}/{key}/metadata", Order = 500)]
		public async Task<IActionResult> GetMetadataAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key,
			[FromQuery] string[] fields)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				(RefRecord objectRecord, BlobContents? _) = await _refService.GetAsync(ns, bucket, key, fields);

				return Ok(new RefMetadataResponse(objectRecord));
			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RefNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Object {e.Bucket} {e.Key} did not exist" });
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Object {e.Blob} in {e.Ns} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		/// <summary>
		/// Returns the replicated state of this ref across all known cloud ddc regions
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance). Example: `uc4.ddc`</param>
		/// <param name="bucket">The category/type of record you are caching. Is a clustered key together with the actual key, but all records in the same bucket can be dropped easily. Example: `terrainTexture` </param>
		/// <param name="key">The unique name of this particular key. `iAmAVeryValidKey`</param>
		[HttpGet("{ns}/{bucket}/{key}/replicationState", Order = 500)]
		public async Task<IActionResult> GetReplicationStateAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.AdminAction });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				List<BlobId> blobs = await _refService.GetReferencedBlobsAsync(ns, bucket, key, ignoreMissingBlobs: true);
				Dictionary<string, Dictionary<string, bool>> blobStatePerRegion = new Dictionary<string, Dictionary<string, bool>>();

				await Parallel.ForEachAsync(blobs, async (blobId, cancellationToken) =>
				{
					Dictionary<string, bool> blobState = new Dictionary<string, bool>();

					foreach (string region in _peerStatusService.GetRegions())
					{
						bool exists = await _blobIndex.BlobExistsInRegionAsync(ns, blobId, region, CancellationToken.None);
						blobState.TryAdd(region, exists);
					}

					lock (blobStatePerRegion)
					{
						blobStatePerRegion[blobId.ToString()] = blobState;
					}
				});

				return Ok(new RefReplicationStateResponse(ns, bucket, key, blobStatePerRegion));
			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RefNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Object {e.Bucket} {e.Key} did not exist" });
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Object {e.Blob} in {e.Ns} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		/// <summary>
		/// Returns the blobs referenced by this ref
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance). Example: `uc4.ddc`</param>
		/// <param name="bucket">The category/type of record you are caching. Is a clustered key together with the actual key, but all records in the same bucket can be dropped easily. Example: `terrainTexture` </param>
		/// <param name="key">The unique name of this particular key. `iAmAVeryValidKey`</param>
		[HttpGet("{ns}/{bucket}/{key}/references", Order = 500)]
		public async Task<IActionResult> GetReferencedBlobsAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			RefRecord refRecord;
			BlobContents? blob;
			try
			{
				(refRecord, blob) = await _refService.GetAsync(ns, bucket, key, fields: Array.Empty<string>(), doLastAccessTracking: false, cancellationToken: HttpContext.RequestAborted);
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Object {e.Blob} not found" });
			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RefNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Object {e.Bucket} {e.Key} did not exist" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			if (blob == null)
			{
				throw new Exception("Failed to find blob contents for ref");
			}

			byte[] blobContents = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
			if (blobContents.Length == 0)
			{
				_logger.LogWarning("0 byte object found for {Id} {Namespace}", refRecord.BlobIdentifier, ns);
			}

			CbObject compactBinaryObject;
			try
			{
				compactBinaryObject = new CbObject(blobContents);
			}
			catch (IndexOutOfRangeException)
			{
				return Problem(title: $"{refRecord.BlobIdentifier} was not a proper compact binary object.", detail: "Index out of range");
			}

			try
			{
				List<BlobId> references = await _referenceResolver.GetReferencedBlobsAsync(ns, compactBinaryObject).ToListAsync();

				if (refRecord.InlinePayload == null)
				{
					// payload is not inlined, so it needs to be replicated
					references.Add(refRecord.BlobIdentifier);
				}
				return Ok(new ResolvedReferencesResult(references.ToArray()));
			}
			catch (PartialReferenceResolveException e)
			{
				return BadRequest(new ValidationProblemDetails { Title = $"Object {bucket} {key} did not exist", Detail = $"Following content ids are invalid: {string.Join(",", e.UnresolvedReferences)}" });
			}
			catch (ReferenceIsMissingBlobsException e)
			{
				return BadRequest(new ValidationProblemDetails { Title = $"Object {bucket} {key} did not exist", Detail = $"Following blobs are missing: {string.Join(",", e.MissingBlobs)}" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		/// <summary>
		/// Checks if a object exists
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance). Example: `uc4.ddc`</param>
		/// <param name="bucket">The category/type of record you are caching. Is a clustered key together with the actual key, but all records in the same bucket can be dropped easily. Example: `terrainTexture` </param>
		/// <param name="key">The unique name of this particular key. `iAmAVeryValidKey`</param>
		/// <returns>200 if it existed, 400 otherwise</returns>
		[HttpHead("{ns}/{bucket}/{key}", Order = 500)]
		[ProducesResponseType(type: typeof(OkResult), 200)]
		[ProducesResponseType(type: typeof(ValidationProblemDetails), 400)]
		public async Task<IActionResult> HeadAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			BlobContents? blob = null;
			try
			{
				(RefRecord record, blob) = await _refService.GetAsync(ns, bucket, key);

				blob ??= await _blobStore.GetObjectAsync(ns, record.BlobIdentifier, bucketHint: bucket);

				Response.Headers[CommonHeaders.HashHeaderName] = record.BlobIdentifier.ToString();

				if (!record.IsFinalized)
				{
					return NotFound(new ProblemDetails { Title = $"Object {bucket} {key} in namespace {ns} is not finalized." });
				}

				// we have to verify the blobs are available locally, as the record of the key is replicated a head of the content
				// TODO: Once we support inline replication this step is not needed as at least one region as this blob, just maybe not this current one
				byte[] blobContents = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
				CbObject compactBinaryObject = new CbObject(blobContents);
				// the reference resolver will throw if any blob is missing, so no need to do anything other then process each reference
				IAsyncEnumerable<BlobId> references = _referenceResolver.GetReferencedBlobsAsync(ns, compactBinaryObject);
				List<BlobId>? _ = await references.ToListAsync();

				// if the record does not inline its blob the actual blob needs to be present as well
				if (record.InlinePayload == null)
				{
					BlobId[] unknownBlobs =
						await _blobStore.FilterOutKnownBlobsAsync(ns, new BlobId[] { record.BlobIdentifier }, HttpContext.RequestAborted);
					if (unknownBlobs.Length != 0)
					{
						return NotFound(new ProblemDetails { Title = $"Object {bucket} {key} in namespace {ns} had at least one missing blob." });
					}
				}
			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Blob {e.Blob} in namespace {ns} did not exist" });
			}
			catch (RefNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Object {e.Bucket} {e.Key} in namespace {e.Namespace} did not exist" });
			}
			catch (PartialReferenceResolveException)
			{
				return NotFound(new ProblemDetails { Title = $"Object {bucket} {key} in namespace {ns} was missing some content ids" });
			}
			catch (ReferenceIsMissingBlobsException)
			{
				return NotFound(new ProblemDetails { Title = $"Object {bucket} {key} in namespace {ns} was missing some blobs" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
			finally
			{
				if (blob != null)
				{
					await blob.DisposeAsync();
				}
			}

			return Ok();
		}

		[HttpGet("{ns}/exists")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> ExistsMultipleAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromQuery][Required] List<string> names)
		{
			ActionResult? accessResultNamespace = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadObject });
			if (accessResultNamespace != null)
			{
				return accessResultNamespace;
			}
			ConcurrentBag<(BucketId, RefId)> missingObject = new();

			List<(BucketId, RefId)> requestedNames = new List<(BucketId, RefId)>();
			foreach (string name in names)
			{
				int separatorIndex = name.IndexOf(".", StringComparison.Ordinal);
				if (separatorIndex == -1)
				{
					return BadRequest(new ProblemDetails() { Title = $"Key {name} did not contain a '.' separator" });
				}

				BucketId bucket = new BucketId(name.Substring(0, separatorIndex));
				RefId key = new RefId(name.Substring(separatorIndex + 1));
				requestedNames.Add((bucket, key));
			}

			// verify we have access to each of the buckets we are requesting info on
			foreach (BucketId bucket in requestedNames.Select(tuple => tuple.Item1).Distinct())
			{
				ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
				if (accessResult != null)
				{
					return accessResult;
				}
			}

			IEnumerable<Task> tasks = requestedNames.Select(async pair =>
			{
				(BucketId bucket, RefId key) = pair;
				BlobContents? blob = null;
				try
				{
					(RefRecord record, blob) = await _refService.GetAsync(ns, bucket, key);

					blob ??= await _blobStore.GetObjectAsync(ns, record.BlobIdentifier, bucketHint: bucket);

					// we have to verify the blobs are available locally, as the record of the key is replicated a head of the content
					// TODO: Once we support inline replication this step is not needed as at least one region as this blob, just maybe not this current one
					byte[] blobContents = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
					CbObject cb = new CbObject(blobContents);
					// the reference resolver will throw if any blob is missing, so no need to do anything other then process each reference
					IAsyncEnumerable<BlobId> references = _referenceResolver.GetReferencedBlobsAsync(ns, cb);
					List<BlobId>? _ = await references.ToListAsync();
				}
				catch (RefNotFoundException)
				{
					missingObject.Add((bucket, key));
				}
				catch (PartialReferenceResolveException)
				{
					missingObject.Add((bucket, key));
				}
				catch (ReferenceIsMissingBlobsException)
				{
					missingObject.Add((bucket, key));
				}
				catch (BlobNotFoundException)
				{
					missingObject.Add((bucket, key));
				}
				finally
				{
					if (blob != null)
					{
						await blob.DisposeAsync();
					}
				}
			});
			await Task.WhenAll(tasks);

			return Ok(new ExistCheckMultipleRefsResponse(missingObject.ToList()));
		}

		[HttpPut("{ns}/{bucket}/{key}.{format?}", Order = 500)]
		[DisableRequestSizeLimit]
		public async Task<IActionResult> PutObjectAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key,
			[FromHeader(Name = CommonHeaders.AllowOverwrite)] bool allowOverwrite = false)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);
			
			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			CbObject payloadObject;
			BlobId blobHeader;

			try
			{
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-blob-ref", HttpContext.RequestAborted);

				BlobId headerHash;
				if (Request.Headers.TryGetValue(CommonHeaders.HashHeaderName, out StringValues headers))
				{
					if (!StringValues.IsNullOrEmpty(headers))
					{
						headerHash = new BlobId(headers.ToString());
					}
					else
					{
						return BadRequest(new ProblemDetails
						{
							Title = $"Header {CommonHeaders.HashHeaderName} was empty"
						});
					}
				}
				else
				{
					return BadRequest(new ProblemDetails
					{
						Title = $"Missing expected header {CommonHeaders.HashHeaderName}"
					});
				}

				blobHeader = headerHash;

				switch (Request.ContentType)
				{
					case MediaTypeNames.Application.Json:
						{
							// TODO: define a scheme for how a json object specifies references

							blobHeader = await _blobStore.PutObjectAsync(ns, payload, headerHash, bucketHint: bucket, cancellationToken: HttpContext.RequestAborted);

							// TODO: convert the json object into a compact binary instead
							CbWriter writer = new CbWriter();
							writer.BeginObject();
							writer.WriteBinaryAttachmentValue(blobHeader.AsIoHash());
							writer.EndObject();

							byte[] blob = writer.ToByteArray();
							payloadObject = new CbObject(blob);
							blobHeader = BlobId.FromBlob(blob);
							break;
						}
					case CustomMediaTypeNames.UnrealCompactBinary:
						{
							await using MemoryStream ms = new MemoryStream();
							await using Stream payloadStream = payload.GetStream();
							await payloadStream.CopyToAsync(ms);
							payloadObject = new CbObject(ms.ToArray());
							break;
						}
					case MediaTypeNames.Application.Octet:
						{
							blobHeader = await _blobStore.PutObjectAsync(ns, payload, headerHash, bucketHint: bucket, cancellationToken: HttpContext.RequestAborted);

							CbWriter writer = new CbWriter();
							writer.BeginObject();
							writer.WriteBinaryAttachment("RawHash", blobHeader.AsIoHash());
							writer.WriteInteger("RawSize", payload.Length);
							writer.EndObject();

							byte[] blob = writer.ToByteArray();
							payloadObject = new CbObject(blob);
							blobHeader = BlobId.FromBlob(blob);
							break;
						}
					default:
						throw new Exception($"Unknown request type {Request.ContentType}, if submitting a blob please use {MediaTypeNames.Application.Octet}");
				}
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (RemoteServiceException e)
			{
				Tracer.CurrentSpan.RecordException(e);
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			try
			{
				{
					using TelemetrySpan scope = _tracer.StartActiveSpan("ref.put").SetAttribute("operation.name", "ref.put");

					(ContentId[] missingReferences, BlobId[] missingBlobs) = await _refService.PutAsync(ns, bucket, key, blobHeader, payloadObject, allowOverwrite: allowOverwrite, cancellationToken: HttpContext.RequestAborted);

					List<ContentHash> missingHashes = new List<ContentHash>(missingReferences);
					missingHashes.AddRange(missingBlobs);
					ContentHash[] missingArray = missingHashes.ToArray();
					scope.SetAttribute("NeedsCount", missingArray.Length);
					return Ok(new PutObjectResponse(missingArray));
				}
			}
			catch (RefAlreadyExistsException e)
			{
				byte[]? payload = e.OldObject.InlinePayload;
				if (payload == null || payload.Length == 0)
				{
					using TelemetrySpan scope = _tracer.StartActiveSpan("blob.get").SetAttribute("operation.name", "blob.get");

					try
					{
						await using BlobContents blobContents = await _blobStore.GetObjectAsync(ns, e.OldObject.BlobIdentifier, cancellationToken: HttpContext.RequestAborted); 
						payload = await blobContents.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
					}
					catch (BlobNotFoundException)
					{
						(CbObject errorObject, _) = ToErrorResult("Failed to find old ref during overwrite");
						payload = errorObject.GetView().ToArray();
					}
				}
				
				// if a ref exists we send back the existing ref, always as a compact binary as that is how it is stored
				Response.ContentType = CustomMediaTypeNames.UnrealCompactBinary;
				Response.StatusCode = (int)HttpStatusCode.Conflict;
				await Response.Body.WriteAsync(payload);
				return new EmptyResult();
			}
			catch (AggregateException e)
			{
				if (e.InnerExceptions.Any(exception => exception.GetType() == typeof(RemoteServiceException)))
				{
					return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
				}

				throw;
			}
			catch (RemoteServiceException e)
			{
				Tracer.CurrentSpan.RecordException(e);
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPut("{ns}/{bucket}/{key}", Order = 300)]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinaryPackage)]
		public async Task<IActionResult> PutPackageAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);
			Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
			Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

			byte[] b = await RequestUtil.ReadRawBodyAsync(Request);
			CbPackageReader packageReader = await CbPackageReader.CreateAsync(new MemoryStream(b));

			try
			{
				await foreach ((CbPackageAttachmentEntry entry, byte[] blob) in packageReader.IterateAttachmentsAsync())
				{
					if (entry.Flags.HasFlag(CbPackageAttachmentFlags.IsError))
					{
						return BadRequest(new ProblemDetails
						{
							Title = $"Package contained attachment with error {entry.AttachmentHash}\""
						});
					}
					if (entry.Flags.HasFlag(CbPackageAttachmentFlags.IsCompressed))
					{
#pragma warning disable CA2000 // Dispose objects before losing scope
						using MemoryBufferedPayload payload = new MemoryBufferedPayload(blob);
#pragma warning restore CA2000 // Dispose objects before losing scope
						await _blobStore.PutCompressedObjectAsync(ns, payload, ContentId.FromIoHash(entry.AttachmentHash), HttpContext.RequestServices, HttpContext.RequestAborted);
					}
					else
					{
						await _blobStore.PutObjectAsync(ns, blob, BlobId.FromIoHash(entry.AttachmentHash), bucketHint: bucket, cancellationToken: HttpContext.RequestAborted);
					}
				}
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			CbObject rootObject = packageReader.RootObject;
			BlobId rootObjectHash = BlobId.FromIoHash(packageReader.RootHash);

			try
			{
				(ContentId[] missingReferences, BlobId[] missingBlobs) = await _refService.PutAsync(ns, bucket, key, rootObjectHash, rootObject, cancellationToken: HttpContext.RequestAborted);

				List<ContentHash> missingHashes = new List<ContentHash>(missingReferences);
				missingHashes.AddRange(missingBlobs);
				return Ok(new PutObjectResponse(missingHashes.ToArray()));
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPost("{ns}/{bucket}/{key}/finalize/{hash}.{format?}")]
		public async Task<IActionResult> FinalizeObjectAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key,
			[FromRoute][Required] BlobId hash)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				Tracer.CurrentSpan.SetAttribute("bucket", bucket.ToString());
				Tracer.CurrentSpan.SetAttribute("namespace", ns.ToString());

				(ContentId[] missingReferences, BlobId[] missingBlobs) = await _refService.FinalizeAsync(ns, bucket, key, hash, cancellationToken: HttpContext.RequestAborted);
				List<ContentHash> missingHashes = new List<ContentHash>(missingReferences);
				missingHashes.AddRange(missingBlobs);
				Tracer.CurrentSpan.SetAttribute("NeedsCount", missingHashes.Count);

				return Ok(new PutObjectResponse(missingHashes.ToArray()));
			}
			catch (ObjectHashMismatchException e)
			{
				return BadRequest(e.Message);
			}
			catch (RefNotFoundException e)
			{
				return NotFound(e.Message);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpGet("{ns}/{bucket}/{key}/blobs/{id}")]
		[ProducesResponseType(type: typeof(byte[]), 200)]
		[ProducesResponseType(type: typeof(ValidationProblemDetails), 400)]
		[Produces(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Octet)]

		public async Task<IActionResult> GetBlobAsync(
			[FromRoute][Required] NamespaceId ns,
#pragma warning disable IDE0060
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key,
#pragma warning restore IDE0060
			[Required] ContentId id,
			[FromQuery] bool supportsRedirect = false)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				(BlobContents blobContents, string mediaType, BlobId? contentHash) = await _blobStore.GetCompressedObjectAsync(ns, id, HttpContext.RequestServices, supportsRedirectUri: supportsRedirect);

				StringValues acceptHeader = Request.Headers["Accept"];
				if (!acceptHeader.Contains("*/*") && acceptHeader.Count != 0 && !acceptHeader.Contains(mediaType))
				{
					return new UnsupportedMediaTypeResult();
				}

				if (contentHash != null && Request.Headers.Range.Count == 0)
				{
					// send the hash of the object is we are fetching the full blob
					Response.Headers[CommonHeaders.HashHeaderName] = contentHash.ToString();
				}

				if (blobContents.RedirectUri != null)
				{
					return Redirect(blobContents.RedirectUri.ToString());
				}

				if (_nginxRedirectHelper.CanRedirect(Request, blobContents))
				{
					return _nginxRedirectHelper.CreateActionResult(blobContents, mediaType);
				}

				return File(blobContents.Stream, mediaType, enableRangeProcessing: true);
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Object {e.Blob} not found" });
			}
			catch (ContentIdResolveException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Content Id {e.ContentId} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPut("{ns}/{bucket}/{key}/blobs/{id}")]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Octet)]

		public async Task<IActionResult> PutBlobAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
#pragma warning disable IDE0060
			[FromRoute][Required] RefId key,
#pragma warning restore IDE0060
			[Required] BlobId id)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);

			try
			{
				bool? bypassCache = _namespacePolicyResolver.GetPoliciesForNs(ns).BypassCacheOnWrite;
				if (Request.ContentType == CustomMediaTypeNames.UnrealCompressedBuffer)
				{
					ContentId cid = ContentId.FromBlobIdentifier(id);
					using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-blob-ref-cb", HttpContext.RequestAborted);

					ContentId identifier = await _blobStore.PutCompressedObjectAsync(ns, payload, cid, HttpContext.RequestServices, bucketHint: bucket, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);

					return Ok(new  BlobUploadResponse(identifier.AsBlobIdentifier()));
				}
				else if (Request.ContentType == MediaTypeNames.Application.Octet)
				{
					Uri? uri = await _blobStore.MaybePutObjectWithRedirectAsync(ns, id, bucketHint: bucket, cancellationToken: HttpContext.RequestAborted);
					if (uri != null)
					{
						return Ok(new BlobUploadUriResponse(id, uri));
					}
					using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-blob-ref-raw", HttpContext.RequestAborted);

					BlobId identifier = await _blobStore.PutObjectAsync(ns, payload, id, bucketHint: bucket, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);
					return Ok(new  BlobUploadResponse(identifier));
				}
				else
				{
					throw new NotImplementedException("Unsupported mediatype: " + Request.ContentType);
				}
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			catch (ResourceHasToManyRequestsException)
			{
				return StatusCode(StatusCodes.Status429TooManyRequests);
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPost("{ns}")]
		[Consumes(CustomMediaTypeNames.UnrealCompactBinary)]
		[Produces(CustomMediaTypeNames.UnrealCompactBinary)]
		[ApiExplorerSettings(IgnoreApi = true)]
		public async Task<IActionResult> BatchAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromBody][Required] BatchOps ops)
		{
			JupiterAclAction ActionForOp(BatchOps.BatchOp.Operation op)
			{
				switch (op)
				{
					case BatchOps.BatchOp.Operation.GET:
						return JupiterAclAction.ReadObject;
					case BatchOps.BatchOp.Operation.PUT:
						return JupiterAclAction.WriteObject;
					case BatchOps.BatchOp.Operation.HEAD:
						return JupiterAclAction.ReadObject;
					default:
						throw new ArgumentOutOfRangeException(nameof(op), op, null);
				}
			}

			foreach (BatchOps.BatchOp op in ops.Ops)
			{
				ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, op.Bucket), new[] { ActionForOp(op.Op) });
				if (accessResult != null)
				{
					return accessResult;
				}
			}

			Tracer.CurrentSpan.SetAttribute("CountOfOps", ops.Ops.Length);

			HashSet<uint> usedOpIds = new HashSet<uint>();
			foreach (BatchOps.BatchOp batchOp in ops.Ops)
			{
				bool added = usedOpIds.Add(batchOp.OpId);
				if (!added)
				{
					return BadRequest(new ProblemDetails { Title = $"Duplicate op ids used for id: {batchOp.OpId}" });
				}
			}
			ConcurrentDictionary<uint, (CbObject, HttpStatusCode)> results = new();

			async Task<(CbObject, HttpStatusCode)> BatchGetOp(BatchOps.BatchOp op)
			{
				try
				{
					(RefRecord objectRecord, BlobContents? blob) = await _refService.GetAsync(ns, op.Bucket, op.Key, Array.Empty<string>());

					if (!objectRecord.IsFinalized)
					{
						return ToErrorResult("object not finalized", HttpStatusCode.BadRequest);
					}

					if (blob == null)
					{
						throw new Exception();
					}

					CbObject cb = new CbObject(await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted));

					if (op.ResolveAttachments ?? false)
					{
						IAsyncEnumerable<BlobId> references = _referenceResolver.GetReferencedBlobsAsync(ns, cb);
						List<BlobId>? _ = await references.ToListAsync();
					}

					return (cb, HttpStatusCode.OK);
				}
				catch (Exception ex) when (ex is TaskCanceledException)
				{
					return ToErrorResult(ex, HttpStatusCode.RequestTimeout);
				}
				catch (Exception ex) when (ex is RefNotFoundException or PartialReferenceResolveException or ReferenceIsMissingBlobsException)
				{
					// this isn't really an error so no need to build a stacktrace which is very expensive
					return ToErrorResult(ex, HttpStatusCode.NotFound, includeStacktrace: false);
				}
				catch (Exception e)
				{
					Tracer.CurrentSpan.SetStatus(Status.Error);
					Tracer.CurrentSpan.RecordException(e);
					return ToErrorResult(e);
				}
			}

			async Task<(CbObject, HttpStatusCode)> BatchHeadOp(BatchOps.BatchOp op)
			{
				BlobContents? blob = null;
				try
				{
					(RefRecord record, blob) = await _refService.GetAsync(ns, op.Bucket, op.Key);

					if (!record.IsFinalized)
					{
						return (CbObject.Build(writer => writer.WriteBool("exists", false)), HttpStatusCode.NotFound);
					}

					blob ??= await _blobStore.GetObjectAsync(ns, record.BlobIdentifier, bucketHint: op.Bucket);

					if (op.ResolveAttachments ?? false)
					{
						byte[] blobContents = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
						CbObject cb = new CbObject(blobContents);
						// the reference resolver will throw if any blob is missing, so no need to do anything other then process each reference
						IAsyncEnumerable<BlobId> references = _referenceResolver.GetReferencedBlobsAsync(ns, cb);
						List<BlobId>? _ = await references.ToListAsync();
					}

					if (blob == null)
					{
						throw new Exception();
					}

					return (CbObject.Build(writer => writer.WriteBool("exists", true)), HttpStatusCode.OK);
				}
				catch (Exception ex) when (ex is TaskCanceledException)
				{
					return ToErrorResult(ex, HttpStatusCode.RequestTimeout);
				}
				catch (Exception ex) when (ex is RefNotFoundException or PartialReferenceResolveException or ReferenceIsMissingBlobsException)
				{
					return (CbObject.Build(writer => writer.WriteBool("exists", false)), HttpStatusCode.NotFound);
				}
				catch (Exception e)
				{
					return ToErrorResult(e);
				}
				finally
				{
					if (blob != null)
					{
						await blob.DisposeAsync();
					}
				}
			}

			async Task<(CbObject, HttpStatusCode)> BatchPutOp(BatchOps.BatchOp op)
			{
				try
				{
					if (op.Payload == null || op.Payload.Equals(CbObject.Empty))
					{
						throw new Exception($"Missing payload for operation: {op.OpId}");
					}

					if (op.PayloadHash == null)
					{
						throw new Exception($"Missing payload hash for operation: {op.OpId}");
					}
					BlobId headerHash = BlobId.FromContentHash(op.PayloadHash);
					BlobId objectHash = BlobId.FromBlob(op.Payload.GetView().ToArray());

					if (!headerHash.Equals(objectHash))
					{
						throw new HashMismatchException(headerHash, objectHash);
					}

					(ContentId[] missingReferences, BlobId[] missingBlobs) = await _refService.PutAsync(ns, op.Bucket, op.Key, objectHash, op.Payload, cancellationToken: HttpContext.RequestAborted);
					List<ContentHash> missingHashes = new List<ContentHash>(missingReferences);

					return (CbSerializer.Serialize(new PutObjectResponse(missingHashes.ToArray())), HttpStatusCode.OK);
				}
				catch (Exception ex) when (ex is TaskCanceledException)
				{
					return ToErrorResult(ex, HttpStatusCode.RequestTimeout);
				}
				catch (Exception e)
				{
					return ToErrorResult(e);
				}
			}

			await Parallel.ForEachAsync(ops.Ops, CancellationToken.None, async (op, token) =>
			{
				switch (op.Op)
				{
					case BatchOps.BatchOp.Operation.GET:
						results.TryAdd(op.OpId, await BatchGetOp(op));
						break;
					case BatchOps.BatchOp.Operation.PUT:
						results.TryAdd(op.OpId, await BatchPutOp(op));
						break;
					case BatchOps.BatchOp.Operation.HEAD:
						results.TryAdd(op.OpId, await BatchHeadOp(op));
						break;
					case BatchOps.BatchOp.Operation.INVALID:
					default:
						throw new NotImplementedException($"Unknown op type {op.Op}");
				}

				await Task.CompletedTask;
			});

			return Ok(new BatchOpsResponse()
			{
				Results = results.Select(result =>
				{
					return new BatchOpsResponse.OpResponses()
					{
						OpId = result.Key,
						Response = result.Value.Item1,
						StatusCode = (int)result.Value.Item2
					};
				}).ToList()
			});
		}

		private static (CbObject, HttpStatusCode) ToErrorResult(Exception exception, HttpStatusCode statusCode = HttpStatusCode.InternalServerError, bool includeStacktrace = true)
		{
			Exception e = exception;
			CbWriter writer = new CbWriter();
			writer.BeginObject();
			writer.WriteString("title", e.Message);
			writer.WriteInteger("status", (int)statusCode);
			if (includeStacktrace && e.StackTrace != null)
			{
				writer.WriteString("stackTrace", e.StackTrace);
			}
			writer.EndObject();
			return (writer.ToObject(), statusCode);
		}

		private static (CbObject, HttpStatusCode) ToErrorResult(string message, HttpStatusCode statusCode = HttpStatusCode.InternalServerError)
		{
			CbWriter writer = new CbWriter();
			writer.BeginObject();
			writer.WriteString("title", message);
			writer.WriteInteger("status", (int)statusCode);
			writer.EndObject();
			return (writer.ToObject(), statusCode);
		}

		/// <summary>
		/// Drop all refs records in the namespace
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance)</param>
		[HttpDelete("{ns}", Order = 500)]
		[ProducesResponseType(204)]
		public async Task<IActionResult> DeleteNamespaceAsync(
			[FromRoute][Required] NamespaceId ns
		)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.DeleteNamespace });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				await _refService.DropNamespaceAsync(ns, HttpContext.RequestAborted);
			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return NoContent();
		}

		/// <summary>
		/// Drop all refs records in the bucket
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance)</param>
		/// <param name="bucket">The category/type of record you are caching. Is a clustered key together with the actual key, but all records in the same bucket can be dropped easily.</param>
		[HttpDelete("{ns}/{bucket}", Order = 500)]
		[ProducesResponseType(200)]
		public async Task<IActionResult> DeleteBucketAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.DeleteBucket });
			if (accessResult != null)
			{
				return accessResult;
			}

			long countOfDeletedRecords;
			try
			{
				countOfDeletedRecords = await _refService.DeleteBucketAsync(ns, bucket, HttpContext.RequestAborted);
			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			return Ok(new BucketDeletedResponse(countOfDeletedRecords));
		}

		/// <summary>
		/// Delete a individual refs key
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance)</param>
		/// <param name="bucket">The category/type of record you are caching. Is a clustered key together with the actual key, but all records in the same bucket can be dropped easily.</param>
		/// <param name="key">The unique name of this particular key</param>
		[HttpDelete("{ns}/{bucket}/{key}", Order = 500)]
		[ProducesResponseType(200)]
		[ProducesResponseType(404)]
		public async Task<IActionResult> DeleteAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] BucketId bucket,
			[FromRoute][Required] RefId key)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.DeleteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				bool deleted = await _refService.DeleteAsync(ns, bucket, key, HttpContext.RequestAborted);
				return Ok(new RefDeletedResponse(deleted ? 1 : 0));
			}
			catch (NamespaceNotFoundException e)
			{
				return NotFound(new ProblemDetails { Title = $"Namespace {e.Namespace} did not exist" });
			}
			catch (RefNotFoundException)
			{
				return Ok(new RefDeletedResponse(0));
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}
	}

	public class RefReplicationStateResponse
	{
		public RefReplicationStateResponse(NamespaceId ns, BucketId bucket, RefId key, Dictionary<string, Dictionary<string, bool>> blobsPerRegion)
		{
			Ns = ns;
			Bucket = bucket;
			Key = key;
			BlobsPerRegion = blobsPerRegion;
		}

		public NamespaceId Ns { get;set; }
		public BucketId Bucket { get;set; }
		public RefId Key { get;set; }
		public Dictionary<string, Dictionary<string, bool>> BlobsPerRegion { get; init; }
	}

	public class RefDeletedResponse
	{
		public RefDeletedResponse()
		{

		}

		public RefDeletedResponse(int deletedCount)
		{
			DeletedCount = deletedCount;
		}

		[CbField("deletedCount")]
		public int DeletedCount { get; set; }
	}

	public class BucketDeletedResponse
	{
		public BucketDeletedResponse()
		{

		}

		public BucketDeletedResponse(long countOfDeletedRecords)
		{
			CountOfDeletedRecords = countOfDeletedRecords;
		}

		[CbField("countOfDeletedRecords")]
		public long CountOfDeletedRecords { get; set; }
	}

	public class BatchOps
	{
		public BatchOps()
		{
			Ops = Array.Empty<BatchOp>();
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1034:Nested types should not be visible", Justification = "For serialization only")]
		public class BatchOp
		{
			public BatchOp()
			{
				Payload = null;
				PayloadHash = null;
			}

			public enum Operation
			{
				INVALID,
				GET,
				PUT,
				HEAD,
			}

			[Required]
			[CbField("opId")]
			public uint OpId { get; set; }

			[CbField("op")]
			[JsonIgnore]
			[UsedImplicitly]
			public string OpString
			{
				get => Op.ToString();
				set => Op = Enum.Parse<Operation>(value);
			}

			[Required]
			public Operation Op { get; set; } = Operation.INVALID;

			[Required]
			[CbField("bucket")]
			public BucketId Bucket { get; set; }

			[Required]
			[CbField("key")]
			public RefId Key { get; set; }

			[CbField("resolveAttachments")]
			public bool? ResolveAttachments { get; set; } = null;

			[CbField("payload")]
			public CbObject? Payload { get; set; } = null;

			[CbField("payloadHash")]
			public ContentHash? PayloadHash { get; set; } = null;
		}

		[CbField("ops")]
		public BatchOp[] Ops { get; set; }
	}

	public class BatchOpsResponse
	{
		public BatchOpsResponse()
		{

		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1034:Nested types should not be visible", Justification = "For serialization only")]
		public class OpResponses
		{
			public OpResponses()
			{

			}
			[CbField("opId")]
			public uint OpId { get; set; }

			[CbField("response")]
			public CbObject Response { get; set; } = null!;

			[CbField("statusCode")]
			public int StatusCode { get; set; }
		}

		[CbField("results")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<OpResponses> Results { get; set; } = new List<OpResponses>();
	}

	public class RefMetadataResponse
	{
		public RefMetadataResponse()
		{
			PayloadIdentifier = null!;
			InlinePayload = null!;
		}

		[JsonConstructor]
		public RefMetadataResponse(NamespaceId ns, BucketId bucket, RefId name, BlobId payloadIdentifier, DateTime lastAccess, bool isFinalized, byte[]? inlinePayload)
		{
			Ns = ns;
			Bucket = bucket;
			Name = name;
			PayloadIdentifier = payloadIdentifier;
			LastAccess = lastAccess;
			IsFinalized = isFinalized;
			InlinePayload = inlinePayload;
		}

		public RefMetadataResponse(RefRecord objectRecord)
		{
			Ns = objectRecord.Namespace;
			Bucket = objectRecord.Bucket;
			Name = objectRecord.Name;
			PayloadIdentifier = objectRecord.BlobIdentifier;
			LastAccess = objectRecord.LastAccess;
			IsFinalized = objectRecord.IsFinalized;
			InlinePayload = objectRecord.InlinePayload;
		}

		[CbField("ns")]
		public NamespaceId Ns { get; set; }

		[CbField("bucket")]
		public BucketId Bucket { get; set; }

		[CbField("name")]
		public RefId Name { get; set; }

		[CbField("payloadIdentifier")]
		public BlobId PayloadIdentifier { get; set; }

		[CbField("lastAccess")]
		public DateTime LastAccess { get; set; }

		[CbField("isFinalized")]
		public bool IsFinalized { get; set; }

		[CbField("inlinePayload")]
		public byte[]? InlinePayload { get; set; }
	}

	public class EnumerateBucketResponse
	{
		public EnumerateBucketResponse()
		{
			RefIds = null!;
		}

		[JsonConstructor]
		public EnumerateBucketResponse(RefId[] refIds)
		{
			RefIds = refIds;
		}

		[CbField("refs")]
		public RefId[] RefIds { get; set; }
	}

	public class PutObjectResponse
	{
		public PutObjectResponse()
		{
			Needs = null!;
		}

		public PutObjectResponse(ContentHash[] missingReferences)
		{
			Needs = missingReferences;
		}

		[CbField("needs")]
		public ContentHash[] Needs { get; set; }
	}

	public class ExistCheckMultipleRefsResponse
	{
		public ExistCheckMultipleRefsResponse(List<(BucketId, RefId)> missing)
		{
			Missing = missing.Select(pair =>
			{
				(BucketId bucketId, RefId ioHashKey) = pair;
				return new MissingReference()
				{
					Bucket = bucketId,
					Key = ioHashKey,
				};
			}).ToList();
		}

		[JsonConstructor]
		public ExistCheckMultipleRefsResponse(List<MissingReference> missing)
		{
			Missing = missing;
		}

		[CbField("missing")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<MissingReference> Missing { get; set; }

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1034:Nested types should not be visible", Justification = "For serialization only")]
		public class MissingReference
		{
			[CbField("bucket")]
			public BucketId Bucket { get; set; }

			[CbField("key")]
			public RefId Key { get; set; }
		}
	}

	public class GetNamespacesResponse
	{
		public GetNamespacesResponse()
		{
			Namespaces = Array.Empty<NamespaceId>();
		}

		[JsonConstructor]
		public GetNamespacesResponse(NamespaceId[] namespaces)
		{
			Namespaces = namespaces;
		}

		[CbField("namespaces")]
		public NamespaceId[] Namespaces { get; set; }
	}

	public class GetBucketsResponse
	{
		public GetBucketsResponse()
		{
			Buckets = Array.Empty<BucketId>();
		}

		[JsonConstructor]
		public GetBucketsResponse(BucketId[] buckets)
		{
			Buckets = buckets;
		}

		public BucketId[] Buckets { get; set; }
	}
}
