// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Linq;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Implementation;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;

namespace Jupiter.Controllers
{
	[ApiController]
	[FormatFilter]
	[Route("api/v1/g")]
	[Authorize]
	[InternalApiFilter]
	public class ReplicationController : ControllerBase
	{
		private readonly ReplicationService _replicationService;
		private readonly IRequestHelper _requestHelper;

		public ReplicationController(ReplicationService replicationService, IRequestHelper requestHelper)
		{
			_replicationService = replicationService;
			_requestHelper = requestHelper;
		}

		[HttpGet("{ns}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetAsync(
			[Required] NamespaceId ns
		)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadTransactionLog });
			if (accessResult != null)
			{
				return accessResult;
			}

			IEnumerable<IReplicator>? replicators = _replicationService.GetReplicators(ns);

			if (replicators == null)
			{
				return BadRequest(new ValidationProblemDetails
				{
					Title = $"No replication configured for namespace {ns}"
				});
			}

			return Ok(new
			{
				Replicators = replicators.Select(replicator => new ReplicatorStateResponse
				{
					Name = replicator.Info.ReplicatorName,
					Namespace = replicator.Info.NamespaceToReplicate,
					Offset = replicator.Info.State.ReplicatorOffset ?? 0,
					Generation = replicator.Info.State.ReplicatingGeneration ?? Guid.Empty,

					LastEvent = replicator.Info.State.LastEvent ?? Guid.Empty,
					LastBucket = replicator.Info.State.LastBucket ?? "",

					LastReplicationRun = replicator.Info.LastRun,
					CountOfRunningReplications = replicator.Info.CountOfRunningReplications,
				})
			});

		}

		[HttpPost("{ns}/{replicatorName}/{offset}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> SetReplicationOffsetAsync(
			[Required] NamespaceId ns,
			[Required] string replicatorName,
			[Required] long offset
		)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteTransactionLog });
			if (accessResult != null)
			{
				return accessResult;
			}

			IEnumerable<IReplicator>? replicators = _replicationService.GetReplicators(ns);

			if (replicators == null)
			{
				return BadRequest(new ValidationProblemDetails
				{
					Title = $"No replication configured for namespace {ns}"
				});
			}

			IReplicator? replicator = replicators.FirstOrDefault(replicator => string.Equals(replicator.Info.ReplicatorName, replicatorName, StringComparison.OrdinalIgnoreCase));
			if (replicator == null)
			{
				return BadRequest(new ValidationProblemDetails
				{
					Title = $"No replication found in namespace {ns} with name {replicatorName}"
				});
			}

			replicator.SetReplicationOffset(offset);

			return Ok();
		}

		[HttpPost("refs/{ns}/{replicatorName}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> SetReplicationStateAsync(
			[Required] NamespaceId ns,
			[Required] string replicatorName,
			[Required][FromBody] NewReplicationState replicationState
		)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteTransactionLog });
			if (accessResult != null)
			{
				return accessResult;
			}

			IEnumerable<IReplicator>? replicators = _replicationService.GetReplicators(ns);

			if (replicators == null)
			{
				return BadRequest(new ValidationProblemDetails
				{
					Title = $"No replication configured for namespace {ns}"
				});
			}

			IReplicator? replicator = replicators.FirstOrDefault(replicator => string.Equals(replicator.Info.ReplicatorName, replicatorName, StringComparison.OrdinalIgnoreCase));
			if (replicator == null)
			{
				return BadRequest(new ValidationProblemDetails
				{
					Title = $"No replication found in namespace {ns} with name {replicatorName}"
				});
			}

			if (replicator is not RefsReplicator refsReplicator)
			{
				return BadRequest(new ValidationProblemDetails
				{
					Title = $"Replicator in namespace {ns} with name {replicatorName} was not a refs replicator"
				});
			}

			refsReplicator.SetRefState(replicationState.LastBucket, replicationState.LastEvent);
			return Ok();
		}

		[HttpDelete("{ns}")]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> DeleteAsync(
			[Required] NamespaceId ns
		)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteTransactionLog });
			if (accessResult != null)
			{
				return accessResult;
			}

			List<IReplicator> replicators = _replicationService.GetReplicators(ns).ToList();

			if (!replicators.Any())
			{
				return BadRequest(new ValidationProblemDetails
				{
					Title = $"No replication configured for namespace {ns}"
				});
			}

			foreach (IReplicator replicator in replicators)
			{
				await replicator.DeleteStateAsync();
			}

			return Ok();

		}
	}

	public class NewReplicationState
	{
		public string? LastBucket { get; set; }
		public Guid? LastEvent { get; set; }
	}

	public class ReplicatorStateResponse
	{
		public string? Name { get; set; }
		public NamespaceId? Namespace { get; set; }
		public long? Offset { get; set; }
		public Guid? Generation { get; set; }
		public DateTime LastReplicationRun { get; set; }
		public int CountOfRunningReplications { get; set; }
		public string? LastBucket { get; set; }
		public Guid LastEvent { get; set; }
	}
}
