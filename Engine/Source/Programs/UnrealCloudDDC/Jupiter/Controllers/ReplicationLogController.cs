// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Linq;
using System.Net;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Serialization;
using Jupiter.Common;
using Jupiter.Implementation;
using Jupiter.Implementation.TransactionLog;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Options;

namespace Jupiter.Controllers
{
	[ApiController]
	[Route("api/v1/replication-log")]
	[InternalApiFilter]
	[Authorize]
	public class ReplicationLogController : ControllerBase
	{
		private readonly IServiceProvider _provider;
		private readonly IRequestHelper _requestHelper;
		private readonly IReplicationLog _replicationLog;
		private readonly IOptionsMonitor<SnapshotSettings> _snapshotSettings;

		public ReplicationLogController(IServiceProvider provider, IRequestHelper requestHelper, IReplicationLog replicationLog, IOptionsMonitor<SnapshotSettings> snapshotSettings)
		{
			_provider = provider;
			_requestHelper = requestHelper;
			_replicationLog = replicationLog;
			_snapshotSettings = snapshotSettings;
		}

		[HttpGet("snapshots/{ns}")]
		[ProducesDefaultResponseType]
		[ProducesResponseType(type: typeof(ProblemDetails), 400)]
		public async Task<IActionResult> GetSnapshotsAsync(
			[Required] NamespaceId ns
		)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadTransactionLog });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				return Ok(new ReplicationLogSnapshots(await _replicationLog.GetSnapshotsAsync(ns).ToListAsync()));
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPost("snapshots/{ns}/create")]
		[ProducesDefaultResponseType]
		[ProducesResponseType(type: typeof(ProblemDetails), 400)]
		public async Task<IActionResult> CreateSnapshotAsync(
			[Required] NamespaceId ns
		)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.WriteTransactionLog });
			if (accessResult != null)
			{
				return accessResult;
			}

			try
			{
				ReplicationLogSnapshotBuilder builder = ActivatorUtilities.CreateInstance<ReplicationLogSnapshotBuilder>(_provider);
				BlobId snapshotBlob = await builder.BuildSnapshotAsync(ns, _snapshotSettings.CurrentValue.SnapshotStorageNamespace, CancellationToken.None);
				return Ok(new SnapshotCreatedResponse(snapshotBlob));
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpGet("incremental/{ns}")]
		[ProducesDefaultResponseType]
		[ProducesResponseType(type: typeof(ProblemDetails), 400)]
		public async Task<IActionResult> GetIncrementalEventsAsync(
			[Required] NamespaceId ns,
			[FromQuery] string? lastBucket,
			[FromQuery] Guid? lastEvent,
			[FromQuery] int count = 1000
		)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadTransactionLog });
			if (accessResult != null)
			{
				return accessResult;
			}

			if (((lastBucket == null && lastEvent.HasValue) || (lastBucket != null && !lastEvent.HasValue)) && lastBucket != "now")
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Both bucket and event has to be specified, or omit both.",
				});
			}

			try
			{
				IAsyncEnumerable<ReplicationLogEvent> events = _replicationLog.GetAsync(ns, lastBucket, lastEvent);

				List<ReplicationLogEvent> l = await events.Take(count).ToListAsync();
				return Ok(new ReplicationLogEvents(l));
			}
			catch (IncrementalLogNotAvailableException)
			{
				// failed to resume from the incremental log, check for a snapshot instead
				SnapshotInfo? snapshot = await _replicationLog.GetLatestSnapshotAsync(ns);
				if (snapshot != null)
				{
					// no log file is available
					return BadRequest(new ProblemDetails
					{
						Title = $"Log file is not available, use snapshot {snapshot.SnapshotBlob} instead",
						Type = ProblemTypes.UseSnapshot,
						Extensions = { { "SnapshotId", snapshot.SnapshotBlob }, { "BlobNamespace", snapshot.BlobNamespace }, }
					});
				}

				// if no snapshot is available we just give up, they can always reset the replication to the default behavior by not sending in lastBucket and lastEvent

				return BadRequest(new ProblemDetails
				{
					Title = $"No snapshot or bucket found for namespace \"{ns}\"",
					Type = ProblemTypes.NoDataFound,
				});
			}
			catch (NamespaceNotFoundException)
			{
				return NotFound(new ProblemDetails
				{
					Title = $"Namespace {ns} was not found",
				});
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpGet("blobs/{ns}/{replicationBucket}")]
		[ProducesDefaultResponseType]
		[ProducesResponseType(type: typeof(ProblemDetails), 400)]
		public async Task<IActionResult> GetBlobLogAsync(
			[Required] NamespaceId ns,
			[Required] string replicationBucket
		)
		{
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns), new[] { JupiterAclAction.ReadTransactionLog });
			if (accessResult != null)
			{
				return accessResult;
			}
			try
			{
				IAsyncEnumerable<BlobReplicationLogEvent> events = _replicationLog.GetBlobEventsAsync(ns, replicationBucket);

				List<BlobReplicationLogEvent> l = await events.ToListAsync();
				return Ok(new BlobReplicationLogEvents(l));
			}
			catch (IncrementalLogNotAvailableException)
			{
				// failed to resume from the incremental log, check for a snapshot instead
				SnapshotInfo? snapshot = await _replicationLog.GetLatestSnapshotAsync(ns);
				if (snapshot != null)
				{
					// no log file is available
					return BadRequest(new ProblemDetails
					{
						Title = $"Log file is not available, use snapshot {snapshot.SnapshotBlob} instead",
						Type = ProblemTypes.UseSnapshot,
						Extensions = { { "SnapshotId", snapshot.SnapshotBlob }, { "BlobNamespace", snapshot.BlobNamespace }, }
					});
				}

				// if no snapshot is available we just give up, the replication will be started to a old bucket by the BlobReplicator	

				return BadRequest(new ProblemDetails
				{
					Title = $"No snapshot or bucket found for namespace \"{ns}\"",
					Type = ProblemTypes.NoDataFound,
				});
			}
			catch (NamespaceNotFoundException)
			{
				return NotFound(new ProblemDetails
				{
					Title = $"Namespace {ns} was not found",
				});
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}
	}

	public class SnapshotCreatedResponse
	{
		public SnapshotCreatedResponse()
		{
			SnapshotBlobId = null!;
		}

		public SnapshotCreatedResponse(BlobId snapshotBlob)
		{
			SnapshotBlobId = snapshotBlob;
		}

		[CbField("snapshotBlobId")]
		public BlobId SnapshotBlobId { get; set; }
	}

	public class ReplicationLogSnapshots
	{
		public ReplicationLogSnapshots()
		{
			Snapshots = new List<SnapshotInfo>();
		}

		[JsonConstructor]
		public ReplicationLogSnapshots(List<SnapshotInfo> snapshots)
		{
			Snapshots = snapshots;
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<SnapshotInfo> Snapshots { get; set; }
	}

	public class ReplicationLogEvents
	{
		public ReplicationLogEvents()
		{
			Events = new List<ReplicationLogEvent>();
		}

		[JsonConstructor]
		public ReplicationLogEvents(List<ReplicationLogEvent> events)
		{
			Events = events;
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<ReplicationLogEvent> Events { get; set; }
	}

	public class BlobReplicationLogEvents
	{
		public BlobReplicationLogEvents()
		{
			Events = new List<BlobReplicationLogEvent>();
		}

		[JsonConstructor]
		public BlobReplicationLogEvents(List<BlobReplicationLogEvent> events)
		{
			Events = events;
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<BlobReplicationLogEvent> Events { get; set; }
	}
}
