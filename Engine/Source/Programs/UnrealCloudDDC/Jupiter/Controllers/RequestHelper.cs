// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Net;
using System.Security.Claims;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;
using Microsoft.Extensions.Primitives;
using OpenTelemetry.Trace;

namespace Jupiter.Controllers;

public class RequestHelper : IRequestHelper
{
	private readonly IAuthorizationService _authorizationService;
	private readonly INamespacePolicyResolver _namespacePolicyResolver;
	private readonly IOptionsMonitor<JupiterSettings> _settings;
	private readonly IOptionsMonitor<AuthSettings> _authSettings;
	private readonly Tracer _tracer;

	public RequestHelper(IAuthorizationService authorizationService, INamespacePolicyResolver namespacePolicyResolver, IOptionsMonitor<JupiterSettings> settings, IOptionsMonitor<AuthSettings> authSettings, Tracer tracer)
	{
		_authorizationService = authorizationService;
		_namespacePolicyResolver = namespacePolicyResolver;
		_settings = settings;
		_authSettings = authSettings;
		_tracer = tracer;
	}

	public async Task<ActionResult?> HasAccessToScopeAsync(ClaimsPrincipal user, HttpRequest request, AccessScope scope, JupiterAclAction[] aclActions)
	{
		using TelemetrySpan _ = _tracer.StartActiveSpan("authorize").SetAttribute("operation.name", "authorize");
		AuthorizationResult authorizationResult = await _authorizationService.AuthorizeAsync(user, new ScopeAccessRequest
		{
			AccessScope = scope,
			Actions = aclActions
		}, ScopeAccessRequirement.Name);

		if (!authorizationResult.Succeeded)
		{
			return new ForbidResult();
		}

		NamespaceId? ns = scope.Namespace;
		// fetch the value of the issuer claim
		string? issuer = user.FindFirstValue("iss");
		AuthSchemeEntry? authScheme = _authSettings.CurrentValue.Schemes.Values.FirstOrDefault(entry => entry.JwtAuthority == issuer);
		if (authScheme != null)
		{
			if (authScheme.AllowedNamespaces.Length != 0)
			{
				// check if the auth scheme is allowed to grant access to this namespace
				if (!authScheme.AllowedNamespaces.Contains(ns.ToString(), StringComparer.InvariantCultureIgnoreCase))
				{
					// not allowed to grant access to the namespace
					return new ForbidResult();
				}
			}
		}

		if (!ns.HasValue)
		{
			// no namespace set, e.g. it's a global request and these do not allow for per namespace policies
			return null;
		}

		bool isPublicNamespace = _namespacePolicyResolver.GetPoliciesForNs(ns.Value).IsPublicNamespace;

		// public namespaces are always accessible
		if (isPublicNamespace)
		{
			return null;
		}

		// namespace is a restricted namespace, check which port it is being accessed on
		bool isPublicPort = IsPublicPort(request.HttpContext);

		if (isPublicPort)
		{
			// trying to access restricted namespace on a public port, this is not allowed
			return new ForbidResult();
		}

		// restricted namespace in corp or internal port, this is okay
		return null;
	}

	public async Task<ActionResult?> HasAccessToNamespaceAsync(ClaimsPrincipal user, HttpRequest request, NamespaceId ns, JupiterAclAction[] aclActions)
	{
		using TelemetrySpan _ = _tracer.StartActiveSpan("authorize").SetAttribute("operation.name", "authorize");
		AuthorizationResult authorizationResult = await _authorizationService.AuthorizeAsync(user, new NamespaceAccessRequest
		{
			Namespace = ns,
			Actions = aclActions
		}, NamespaceAccessRequirement.Name);

		if (!authorizationResult.Succeeded)
		{
			return new ForbidResult();
		}

		// fetch the value of the issuer claim
		string? issuer = user.FindFirstValue("iss");
		AuthSchemeEntry? authScheme = _authSettings.CurrentValue.Schemes.Values.FirstOrDefault(entry => entry.JwtAuthority == issuer);
		if (authScheme != null)
		{
			if (authScheme.AllowedNamespaces.Length != 0)
			{
				// check if the auth scheme is allowed to grant access to this namespace
				if (!authScheme.AllowedNamespaces.Contains(ns.ToString(), StringComparer.InvariantCultureIgnoreCase))
				{
					// not allowed to grant access to the namespace
					return new ForbidResult();
				}
			}
		}

		bool isPublicNamespace = _namespacePolicyResolver.GetPoliciesForNs(ns).IsPublicNamespace;

		// public namespaces are always accessible
		if (isPublicNamespace)
		{
			return null;
		}

		// namespace is a restricted namespace, check which port it is being accessed on
		bool isPublicPort = IsPublicPort(request.HttpContext);

		if (isPublicPort)
		{
			// trying to access restricted namespace on a public port, this is not allowed
			return new ForbidResult();
		}

		// restricted namespace in corp or internal port, this is okay
		return null;
	}

	public async Task<ActionResult?> HasAccessForGlobalOperationsAsync(ClaimsPrincipal user, JupiterAclAction[] aclActions)
	{
		using TelemetrySpan _ = _tracer.StartActiveSpan("authorize").SetAttribute("operation.name", "authorize");
		AuthorizationResult authorizationResult = await _authorizationService.AuthorizeAsync(user, new GlobalAccessRequest
		{
			Actions = aclActions
		}, GlobalAccessRequirement.Name);

		if (!authorizationResult.Succeeded)
		{
			return new ForbidResult();
		}

		return null;
	}

	public bool IsPublicPort(HttpContext context)
	{
		string? portHeaderValue = null;
		if (context.Request.Headers.TryGetValue("X-Jupiter-Port", out StringValues values))
		{
			portHeaderValue = values.ToString();
		}

		// unit tests do not run on ports, we consider them always on the internal port
		bool isLocalConnection = context.Connection.LocalPort == 0 && context.Connection.LocalIpAddress == null;
		// public port is either running on the public port, or if using domain sockets we check the header that is passed along instead
		bool isPublicPort = _settings!.CurrentValue.PublicApiPorts.Contains(context.Connection.LocalPort);

		if (isLocalConnection && _settings.CurrentValue.AssumeLocalConnectionsHasFullAccess)
		{
			// local connection so granting it full access
			isPublicPort = false;
		}

		if (isLocalConnection && portHeaderValue != null)
		{
			if (string.Equals(portHeaderValue, "Public", StringComparison.OrdinalIgnoreCase))
			{
				isPublicPort = true;
			}
			else if (string.Equals(portHeaderValue, "Corp", StringComparison.OrdinalIgnoreCase))
			{
				isPublicPort = false;
			}
		}

		return isPublicPort;
	}

	public IActionResult? EnsureSingleRangeRequest(HttpContext context)
	{
		// asp.net does not support serving multi range requests so we return an error should that be requested
		if (context.Request.Headers.Range.ToString().Contains(',', StringComparison.OrdinalIgnoreCase))
		{
			return new StatusCodeResult((int)HttpStatusCode.NotImplemented);
		}

		return null;
	}
}
