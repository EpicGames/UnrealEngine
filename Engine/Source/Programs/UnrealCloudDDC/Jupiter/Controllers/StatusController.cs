// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Linq;
using System.Net;
using System.Reflection;
using System.Text.Json.Serialization;
using EpicGames.Serialization;
using Jupiter.Implementation;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Controllers
{
	[ApiController]
	[Route("api/v1/status")]
	[Authorize]
	public class StatusController : Controller
	{
		private readonly VersionFile _versionFile;
		private readonly IOptionsMonitor<JupiterSettings> _jupiterSettings;
		private readonly IOptionsMonitor<ClusterSettings> _clusterSettings;
		private readonly IPeerStatusService _statusService;
		private readonly ILogger<StatusController> _logger;

		public StatusController(VersionFile versionFile, IOptionsMonitor<JupiterSettings> jupiterSettings, IOptionsMonitor<ClusterSettings> clusterSettings, IPeerStatusService statusService, ILogger<StatusController> logger)
		{
			_versionFile = versionFile;
			_jupiterSettings = jupiterSettings;
			_clusterSettings = clusterSettings;
			_statusService = statusService;
			_logger = logger;
		}

		/// <summary>
		/// Fetch information about Jupiter
		/// </summary>
		/// <remarks>
		/// General information about the service, which version it is running and more.
		/// </remarks>
		/// <returns></returns>
		[HttpGet("")]
		[ProducesResponseType(type: typeof(StatusResponse), 200)]
		public IActionResult Status()
		{
			IEnumerable<AssemblyMetadataAttribute> attrs = typeof(StatusController).Assembly.GetCustomAttributes<AssemblyMetadataAttribute>();

			string srcControlIdentifier = "Unknown";
			AssemblyMetadataAttribute? gitHashAttribute = attrs.FirstOrDefault(attr => attr.Key == "GitHash");
			if (gitHashAttribute?.Value != null && !string.IsNullOrEmpty(gitHashAttribute.Value))
			{
				srcControlIdentifier = gitHashAttribute.Value;
			}

			AssemblyMetadataAttribute? p4ChangeAttribute = attrs.FirstOrDefault(attr => attr.Key == "PerforceChangelist");
			if (p4ChangeAttribute?.Value != null && !string.IsNullOrEmpty(p4ChangeAttribute.Value))
			{
				srcControlIdentifier = p4ChangeAttribute.Value;
			}

			return Ok(new StatusResponse(_versionFile.VersionString ?? "Unknown", srcControlIdentifier, GetCapabilities(), _jupiterSettings.CurrentValue.CurrentSite));
		}

		private static string[] GetCapabilities()
		{
			return new string[]
			{
				"transactionlog",
				"ddc"
			};
		}

		/// <summary>
		/// Fetch information about other deployments
		/// </summary>
		/// <remarks>
		/// General information about the Jupiter service, which version it is running and more.
		/// </remarks>
		/// <returns></returns>
		[HttpGet("peers")]
		[ProducesResponseType(type: typeof(PeersResponse), 200)]
		public IActionResult Peers([FromQuery] bool includeInternalEndpoints = false)
		{
			return Ok(new PeersResponse(_jupiterSettings, _clusterSettings, includeInternalEndpoints, _statusService));
		}

		/// <summary>
		/// Get a list of which servers you can talk to based on the ip used to call the endpoint
		/// </summary>
		/// <remarks>
		/// </remarks>
		/// <returns></returns>
		[HttpGet("servers")]
		[ProducesResponseType(type: typeof(PeersResponse), 200)]
		public IActionResult Servers([FromQuery] IPAddress? asIp = null)
		{
			IPAddress? ip = asIp ?? Request.HttpContext.Connection.RemoteIpAddress;

			if (_clusterSettings.CurrentValue.Discovery == null)
			{
				return BadRequest("No DiscoverySettings specified on the cluster settings");
			}

			(List<EndpointDefinition> cacheEndpoints, List<EndpointDefinition> serverEndpoints) = DetermineBestEndpoints(ip, _clusterSettings.CurrentValue.Discovery);

			return Ok(new ServersResponse(cacheEndpoints, serverEndpoints));
		}

		private (List<EndpointDefinition>, List<EndpointDefinition>) DetermineBestEndpoints(IPAddress? ip, DiscoverySettings discoverySettings)
		{
			if (ip == null)
			{
				return (ResolveEndpoints(discoverySettings, discoverySettings.DefaultRule.CacheIds), ResolveEndpoints(discoverySettings, discoverySettings.DefaultRule.ServerIds));
			}

			List<string> cacheIds = new List<string>();
			List<string> serverIds = new List<string>();

			bool hadMatch = false;
			foreach (RoutingRule rule in discoverySettings.Rules)
			{
				bool match = false;
				foreach (string subnet in rule.Subnets)
				{
					// determine if subnet matches ip
					if (IPNetwork.TryParse(subnet, out IPNetwork network))
					{
						if (network.Contains(ip))
						{
							match = true;
							break;
						}
					}
					else
					{
						_logger.LogWarning("Subnet {Subnet} in rule {Rule} is not in a CIDR format", subnet, rule.Name);
					}
				}

				if (match)
				{
					cacheIds.AddRange(rule.CacheIds);
					serverIds.AddRange(rule.ServerIds);
					hadMatch = true;
				}
			}

			if (!hadMatch)
			{
				return (ResolveEndpoints(discoverySettings, discoverySettings.DefaultRule.CacheIds), ResolveEndpoints(discoverySettings, discoverySettings.DefaultRule.ServerIds));
			}

			return (ResolveEndpoints(discoverySettings, cacheIds.Distinct()), ResolveEndpoints(discoverySettings, serverIds.Distinct()));
		}

		private List<EndpointDefinition> ResolveEndpoints(DiscoverySettings discoverySettings, IEnumerable<string> endpoints)
		{
			List<EndpointDefinition> resolvedEndpoints = new List<EndpointDefinition>();
			foreach (string endpoint in endpoints)
			{
				if (discoverySettings.EndpointDefinitions.TryGetValue(endpoint, out EndpointDefinition? value))
				{
					resolvedEndpoints.Add(value);
				}
				else
				{
					_logger.LogWarning("Unable to resolve endpoint id {EndpointId} to a definition, make sure this exists in your discovery settings", endpoint);
				}
			}

			return resolvedEndpoints;
		}
	}

	public class PeersResponse
	{
		public PeersResponse()
		{

		}

		[JsonConstructor]
		public PeersResponse(string currentSite, List<KnownPeer> peers)
		{
			CurrentSite = currentSite;
			Peers = peers;
		}

		public PeersResponse(IOptionsMonitor<JupiterSettings> jupiterSettings, IOptionsMonitor<ClusterSettings> clusterSettings, bool includeInternalEndpoints, IPeerStatusService peerStatusService)
		{
			CurrentSite = jupiterSettings.CurrentValue.CurrentSite;
			Peers = clusterSettings.CurrentValue.Peers.Select(settings => new KnownPeer(settings, includeInternalEndpoints, peerStatusService)).ToList();
		}

		[CbField("currentSite")]
		public string CurrentSite { get; set; } = null!;

		[CbField("peers")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<KnownPeer> Peers { get; set; } = new List<KnownPeer>();
	}

	public class ServersResponse
	{
		public ServersResponse()
		{

		}

		[CbField("cacheEndpoints")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<EndpointDefinition> CacheEndpoints { get; set; } = new List<EndpointDefinition>();
		
		[CbField("serverEndpoints")]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<EndpointDefinition> ServerEndpoints { get; set; } = new List<EndpointDefinition>();

		public ServersResponse(List<EndpointDefinition> cacheEndpoints, List<EndpointDefinition> serverEndpoints)
		{
			CacheEndpoints = cacheEndpoints;
			ServerEndpoints = serverEndpoints;
		}
	}

	public class EndpointDefinition
	{
		[Required]
		[CbField("name")]
		public string Name { get; set; } = null!;

		[Required]
		[CbField("baseUrl")]
		public Uri BaseUrl { get; set; } = null!;

		[CbField("assumeHttp2")]
		public bool AssumeHttp2 { get; set; } = false;
	}

	public class RoutingRule
	{
		[Required] 
		public string Name { get; set; } = null!;
		
		[Required] 
		public string Description { get; set; } = null!;

		[Required]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<string> Subnets { get; set; } = new List<string>();

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<string> CacheIds { get; set; } = new List<string>();
		
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<string> ServerIds { get; set; } = new List<string>();
	}

	public class DiscoverySettings
	{
		[Required]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public Dictionary<string , EndpointDefinition> EndpointDefinitions { get; set; } = new Dictionary<string, EndpointDefinition>();

		[Required]
		public RoutingRule DefaultRule { get; set; } = null!;
		
		[Required]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<RoutingRule> Rules { get; set; } = new List<RoutingRule>();
	}

	public class KnownPeer
	{
		public KnownPeer()
		{
			Site = null!;
			FullName = null!;
			Endpoints = null!;
		}

		[JsonConstructor]
		public KnownPeer(string site, string fullName, List<Uri> endpoints, int latency)
		{
			Site = site;
			FullName = fullName;
			Endpoints = endpoints;
			Latency = latency;
		}

		public KnownPeer(PeerSettings peerSettings, bool includeInternalEndpoints, IPeerStatusService statusService)
		{
			Site = peerSettings.Name;
			FullName = peerSettings.FullName;
			IEnumerable<PeerEndpoints> endpoints = peerSettings.Endpoints;
			if (!includeInternalEndpoints)
			{
				endpoints = endpoints.Where(s => !s.IsInternal);
			}

			Endpoints = endpoints.Select(e => e.Url).ToList();

			PeerStatus? peerStatus = statusService.GetPeerStatus(peerSettings.Name);
			if (peerStatus != null)
			{
				Latency = peerStatus.Latency;
			}
		}

		[CbField("site")]
		public string Site { get; set; }

		[CbField("fullName")]
		public string FullName { get; set; }

		[CbField("latency")]
		public int Latency { get; set; }

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		[CbField("endpoints")]
		public List<Uri> Endpoints { get; set; }
	}

	public class StatusResponse
	{
		public StatusResponse()
		{
			Version = null!;
			GitHash = null!;
			Capabilities = Array.Empty<string>();
			SiteIdentifier = null!;
		}

		public StatusResponse(string version, string gitHash, string[] capabilities, string siteIdentifier)
		{
			Version = version;
			GitHash = gitHash;
			Capabilities = capabilities;
			SiteIdentifier = siteIdentifier;
		}

		public string Version { get; set; }
		public string GitHash { get; set; }
		public string[] Capabilities { get; set; }
		public string SiteIdentifier { get; set; }
	}
}
