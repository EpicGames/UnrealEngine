// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Mime;
using System.Text;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Core;
using EpicGames.Serialization;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Implementation;
using Jupiter.Utils;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Microsoft.Extensions.Primitives;
using OpenTelemetry.Trace;
using Serilog;
using BinaryReader = System.IO.BinaryReader;
using ContentHash = Jupiter.Implementation.ContentHash;
using ILogger = Serilog.ILogger;

namespace Jupiter.Controllers
{
	[ApiController]
	[Produces(MediaTypeNames.Application.Json, MediaTypeNames.Application.Octet, CustomMediaTypeNames.UnrealCompactBinary)]
	[Route("api/v1/symbols")]
	[Authorize]
	public class SymbolsController : ControllerBase
	{
		private readonly IRefService _refService;
		private readonly IBlobService _blobStore;
		private readonly IDiagnosticContext _diagnosticContext;
		private readonly BufferedPayloadFactory _bufferedPayloadFactory;
		private readonly FormatResolver _formatResolver;
		private readonly NginxRedirectHelper _nginxRedirectHelper;
		private readonly IRequestHelper _requestHelper;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly Tracer _tracer;
		private readonly ILogger<SymbolsController> _logger;
		private readonly ILogger? _auditLogger;

		public SymbolsController(IRefService refService, IBlobService blobStore, IDiagnosticContext diagnosticContext, BufferedPayloadFactory bufferedPayloadFactory, FormatResolver formatResolver, NginxRedirectHelper nginxRedirectHelper, IRequestHelper requestHelper, INamespacePolicyResolver namespacePolicyResolver, Tracer tracer, ILogger<SymbolsController> logger, IOptionsMonitor<SymbolsSettings> symbolSettings)
		{
			_refService = refService;
			_blobStore = blobStore;
			_diagnosticContext = diagnosticContext;
			_bufferedPayloadFactory = bufferedPayloadFactory;
			_formatResolver = formatResolver;
			_nginxRedirectHelper = nginxRedirectHelper;
			_requestHelper = requestHelper;
			_namespacePolicyResolver = namespacePolicyResolver;
			_tracer = tracer;
			_logger = logger;
			_auditLogger = symbolSettings.CurrentValue.EnableAuditLog ? Serilog.Log.ForContext("LogType", "Audit") : null;
		}

		/// <summary>
		///  Fetch a symbol file
		/// </summary>
		/// <param name="ns">Namespace. Each namespace is completely separated from each other. Use for different types of data that is never expected to be similar (between two different games for instance). Example: `uc4.ddc`</param>
		/// <param name="moduleName">The name of the module, so Foo.pdb for instance</param>
		/// <param name="identifier">The pdb identifier and age</param>
		/// <param name="fileName">The specific file to fetch, either pdb or ptrs</param>
		[HttpGet("{ns}/{moduleName}/{identifier}/{fileName}", Order = 500)]
		[Produces(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Octet)]
		[ProducesDefaultResponseType]
		public async Task<IActionResult> GetAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] string moduleName,
			[FromRoute][Required] string identifier,
			[FromRoute][Required] string fileName)
		{
			BucketId bucket = new BucketId(moduleName);
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			LogAuditEntry(HttpMethod.Get, ns, moduleName, identifier, fileName);

			try
			{
				(RefRecord symbolRecord, BlobContents? maybeBlob) = await _refService.GetAsync(ns, bucket, RefId.FromName($"{moduleName}.{identifier}.{fileName}"), Array.Empty<string>());
				
				if (maybeBlob == null)
				{
					throw new InvalidOperationException($"Blob was null when attempting to fetch {ns} {bucket} {moduleName} {identifier} {fileName}");
				}

				await using BlobContents blob = maybeBlob;
				
				Response.Headers[CommonHeaders.HashHeaderName] = symbolRecord.BlobIdentifier.ToString();
				Response.Headers[CommonHeaders.LastAccessHeaderName] = symbolRecord.LastAccess.ToString(CultureInfo.InvariantCulture);
				
				const int BufferSize = 64 * 1024;
				
				IServerTiming? serverTiming = Request.HttpContext.RequestServices.GetService<IServerTiming>();
				using ServerTimingMetricScoped? serverTimingScope =
					serverTiming?.CreateServerTimingMetricScope("body.write", "Time spent writing body");

				string responseType = _formatResolver.GetResponseType(Request, null, CustomMediaTypeNames.UnrealCompactBinary);
				Tracer.CurrentSpan.SetAttribute("response-type", responseType);
				
				using TelemetrySpan scope = _tracer.StartActiveSpan("body.write").SetAttribute("operation.name", "body.write");

				switch (responseType)
				{
					case CustomMediaTypeNames.UnrealCompactBinary:
						{
							long contentLength = blob.Length;
							scope.SetAttribute("content-length", contentLength);
							
							Response.ContentLength = contentLength;
							Response.ContentType = responseType;
							Response.StatusCode = StatusCodes.Status200OK;

							await blob.Stream.CopyToAsync(Response.Body, BufferSize, HttpContext.RequestAborted);
							break;
						}
					case MediaTypeNames.Application.Octet:
						{
							byte[] blobContents = await blob.Stream.ToByteArrayAsync(HttpContext.RequestAborted);
							CbObject cb = new CbObject(blobContents);

							long contentLength = cb["pdbSize"].AsInt64();
							scope.SetAttribute("content-length", contentLength);
							
							Response.ContentLength = contentLength;
							Response.ContentType = responseType;
							Response.StatusCode = StatusCodes.Status200OK;

							CbArray chunks = cb["pdbChunks"].AsArray();
					
							foreach (CbField chunk in chunks)
							{
								ContentId contentId = ContentId.FromIoHash(chunk.AsBinaryAttachment().Hash);

								try
								{
									(BlobContents contents, string mimeType, BlobId? _) = await _blobStore.GetCompressedObjectAsync(ns, contentId, HttpContext.RequestServices, cancellationToken: HttpContext.RequestAborted);
									
									Stream streamToCopy;

									switch (mimeType)
									{
										case CustomMediaTypeNames.UnrealCompressedBuffer:
											CompressedBufferUtils bufferUtils = new CompressedBufferUtils(_tracer, _bufferedPayloadFactory);
											(IBufferedPayload decompressedPayload, IoHash _) = await bufferUtils.DecompressContentAsync(contents.Stream, (ulong)contents.Length, cancellationToken: HttpContext.RequestAborted);
											streamToCopy = decompressedPayload.GetStream();
											break;
										case MediaTypeNames.Application.Octet:
											streamToCopy = contents.Stream;
											break;
										default:
											throw new NotImplementedException($"Unknown chunk mime type {mimeType}");
									}
									
									await streamToCopy.CopyToAsync(Response.Body, BufferSize, HttpContext.RequestAborted);
								}
								catch (BlobNotFoundException)
								{
									return NotFound("Symbol contains missing chunks");
								}
							}
							break;
						}
					default:
						throw new NotImplementedException($"Unknown expected response type {responseType}");
				}
			}
			catch (RefNotFoundException)
			{
				return NotFound("Symbol not found");
			}
			catch (OperationCanceledException)
			{
				// do not raise exceptions for canceled writes
				// as we have already started writing a response we can not change the status code
				// so we just drop a warning and proceed
				_logger.LogWarning("The operation was canceled while writing the body");
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}

			// content was written above
			return new EmptyResult();
		}

		[HttpPut("{ns}/{moduleName}", Order = 500)]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompactBinary, MediaTypeNames.Application.Octet)]
		public async Task<IActionResult> PutSymbolsAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] string moduleName)
		{
			BucketId bucket = new BucketId(moduleName);
			ActionResult? accessResult = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (accessResult != null)
			{
				return accessResult;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);

			using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-symbols", HttpContext.RequestAborted);

			CbObject payloadObject;
			BlobId payloadHash;
			string fileName = moduleName;
			string pdbIdentifier;
			int pdbAge;
			
			try
			{
				await using Stream payloadStream = payload.GetStream();
				
				switch (Request.ContentType)
				{
					case CustomMediaTypeNames.UnrealCompactBinary:
						{
							await using MemoryStream ms = new MemoryStream();
							await payloadStream.CopyToAsync(ms);
							payloadObject = new CbObject(ms.ToArray());
							payloadHash = BlobId.FromBlob(payloadObject.GetView().ToArray());
							pdbIdentifier = payloadObject["pdbIdentifier"].AsString();
							pdbAge = payloadObject["pdbAge"].AsInt32();
							break;
						}
					case MediaTypeNames.Application.Octet:
						{
							BlobId attachmentHash = await BlobId.FromStreamAsync(payloadStream, HttpContext.RequestAborted);
							IoHash attachmentIoHash = attachmentHash.AsIoHash();

							(pdbIdentifier, pdbAge) = ExtractModuleInformation(moduleName, payload);

							byte[] blob = CreateCompactBinaryForSymbol(attachmentIoHash, moduleName, pdbIdentifier, pdbAge, payload.Length);
							payloadObject = new CbObject(blob);
							payloadHash = BlobId.FromBlob(blob);
							
							bool? bypassCache = _namespacePolicyResolver.GetPoliciesForNs(ns).BypassCacheOnWrite;
							await _blobStore.PutObjectKnownHashAsync(ns, payload, attachmentHash, bucket, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);
							break;
						}
					default:
						throw new Exception($"Unknown request type {Request.ContentType}");
				}
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}

			LogAuditEntry(HttpMethod.Put, ns, moduleName, pdbIdentifier, fileName);

			try
			{
				List<ContentHash> missingHashes = await PutSymbolsRefAsync(ns, bucket, moduleName, pdbIdentifier, pdbAge, fileName, payloadObject, payloadHash);
				return Ok(new PutObjectResponse(missingHashes.ToArray()));
			}
			catch (RefAlreadyExistsException)
			{
				return Problem($"Symbol already exists {ns} {moduleName} {pdbIdentifier} {fileName}", statusCode: (int)HttpStatusCode.Conflict);
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		private async Task<List<ContentHash>> PutSymbolsRefAsync(NamespaceId ns, BucketId bucket, string moduleName, string identifier, int age, string fileName, CbObject payloadObject, BlobId payloadHash)
		{
			RefId refId = RefId.FromName($"{moduleName}.{identifier}{age}.{fileName}");
			{
				using TelemetrySpan scope = _tracer.StartActiveSpan("ref.put").SetAttribute("operation.name", "ref.put").SetAttribute("resource.name", refId.ToString());

				(ContentId[] missingReferences, BlobId[] missingBlobs) = await _refService.PutAsync(ns, bucket, refId, payloadHash, payloadObject, cancellationToken: HttpContext.RequestAborted);

				List<ContentHash> missingHashes = new List<ContentHash>(missingReferences);
				missingHashes.AddRange(missingBlobs);
				ContentHash[] missingArray = missingHashes.ToArray();
				scope.SetAttribute("NeedsCount", missingArray.Length);

				return missingHashes;
			}
		}

		private static byte[] CreateCompactBinaryForSymbol(IoHash attachmentIoHash, string moduleName, string pdbIdentifier, int pdbAge, long pdbSize)
		{
			CbWriter writer = new CbWriter();
			writer.BeginObject();
							
			writer.BeginArray("pdbChunks");
			writer.WriteBinaryAttachmentValue(attachmentIoHash);
			writer.EndArray();
			
			writer.WriteString("moduleName", moduleName);
			writer.WriteString("pdbIdentifier", pdbIdentifier);
			writer.WriteInteger("pdbAge", pdbAge);
			writer.WriteInteger("pdbSize", pdbSize);
							
			writer.EndObject();

			return writer.ToByteArray();
		}

		#region Blob endpoints

		[HttpGet("{ns}/{moduleName}/blobs/{id}")]
		[ProducesResponseType(type: typeof(byte[]), 200)]
		[ProducesResponseType(type: typeof(ValidationProblemDetails), 400)]
		[Produces(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Octet)]
		public async Task<IActionResult> GetBlobAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] string moduleName,
			[Required] ContentId id,
			[FromQuery] bool supportsRedirect = false)
		{
			BucketId bucket = new BucketId(moduleName);
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.ReadObject });
			if (result != null)
			{
				return result;
			}

			try
			{
				(BlobContents blobContents, string mediaType, BlobId? contentHash) = await _blobStore.GetCompressedObjectAsync(ns, id, HttpContext.RequestServices, supportsRedirectUri: supportsRedirect);

				StringValues acceptHeader = Request.Headers["Accept"];
				if (!acceptHeader.Contains("*/*") && acceptHeader.Count != 0 && !acceptHeader.Contains(mediaType))
				{
					return new UnsupportedMediaTypeResult();
				}

				if (contentHash != null && Request.Headers.Range.Count == 0)
				{
					// send the hash of the object is we are fetching the full blob
					Response.Headers[CommonHeaders.HashHeaderName] = contentHash.ToString();
				}

				if (blobContents.RedirectUri != null)
				{
					return Redirect(blobContents.RedirectUri.ToString());
				}

				if (_nginxRedirectHelper.CanRedirect(Request, blobContents))
				{
					return _nginxRedirectHelper.CreateActionResult(blobContents, mediaType);
				}

				return File(blobContents.Stream, mediaType, enableRangeProcessing: true);
			}
			catch (BlobNotFoundException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Blob {e.Blob} not found" });
			}
			catch (ContentIdResolveException e)
			{
				return NotFound(new ValidationProblemDetails { Title = $"Content Id {e.ContentId} not found" });
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}

		[HttpPut("{ns}/{moduleName}/blobs/{id}", Order = 500)]
		[DisableRequestSizeLimit]
		[RequiredContentType(CustomMediaTypeNames.UnrealCompressedBuffer, MediaTypeNames.Application.Octet)]
		public async Task<IActionResult> PutBlobAsync(
			[FromRoute][Required] NamespaceId ns,
			[FromRoute][Required] string moduleName,
			[Required] BlobId id)
		{
			BucketId bucket = new BucketId(moduleName);
			ActionResult? result = await _requestHelper.HasAccessToScopeAsync(User, Request, new AccessScope(ns, bucket), new[] { JupiterAclAction.WriteObject });
			if (result != null)
			{
				return result;
			}

			_diagnosticContext.Set("Content-Length", Request.ContentLength ?? -1);

			try
			{
				bool? bypassCache = _namespacePolicyResolver.GetPoliciesForNs(ns).BypassCacheOnWrite;
				
				using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromRequestAsync(Request, "put-symbol-blob", HttpContext.RequestAborted);
				if (Request.ContentType == CustomMediaTypeNames.UnrealCompressedBuffer)
				{
					ContentId cid = ContentId.FromBlobIdentifier(id);

					ContentId identifier = await _blobStore.PutCompressedObjectAsync(ns, payload, cid, HttpContext.RequestServices, bucketHint: bucket, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);

					return Ok(new BlobUploadResponse(identifier.AsBlobIdentifier()));
				}
				else if (Request.ContentType == MediaTypeNames.Application.Octet)
				{
					Uri? uri = await _blobStore.MaybePutObjectWithRedirectAsync(ns, id, bucketHint: bucket, cancellationToken: HttpContext.RequestAborted);
					if (uri != null)
					{
						return Ok(new BlobUploadUriResponse(id, uri));
					}

					BlobId identifier = await _blobStore.PutObjectAsync(ns, payload, id, bucketHint: bucket, bypassCache: bypassCache, cancellationToken: HttpContext.RequestAborted);
					return Ok(new BlobUploadResponse(identifier));
				}
				else
				{
					throw new NotImplementedException("Unsupported mediatype: " + Request.ContentType);
				}
			}
			catch (HashMismatchException e)
			{
				return BadRequest(new ProblemDetails
				{
					Title = $"Incorrect hash, got hash \"{e.SuppliedHash}\" but hash of content was determined to be \"{e.ContentHash}\"",
					Type = ProblemTypes.HashMismatchError
				});
			}
			catch (ResourceHasToManyRequestsException)
			{
				return StatusCode(StatusCodes.Status429TooManyRequests);
			}
			catch (ClientSendSlowException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.RequestTimeout);
			}
			catch (TaskCanceledException)
			{
				return StatusCode(StatusCodes.Status408RequestTimeout, "Request cancelled");
			}
			catch (RemoteServiceException e)
			{
				return Problem(e.Message, null, (int)HttpStatusCode.ServiceUnavailable);
			}
		}
		#endregion

		private void LogAuditEntry(HttpMethod method, NamespaceId ns, string moduleName, string identifier, string fileName)
		{
			_auditLogger?.Information("{HttpMethod} '{Namespace}'/'{ModuleName}:{Identifier}' {FileName} IP:{IP} User:{Username} UserAgent:\"{Useragent}\"", method,ns, moduleName, identifier, fileName, Request.HttpContext.Connection.RemoteIpAddress, User?.Identity?.Name ?? "Unknown-user", string.Join(' ', Request.Headers.UserAgent.ToArray()));
		}

		private static (string, int) ExtractModuleInformation(string moduleName, IBufferedPayload decompressedContent)
		{
			using Stream s = decompressedContent.GetStream();
			string extension = Path.GetExtension(moduleName);
			switch (extension)
			{
				case ".pdb":
					return ExtractModuleInformationPdb(s);
				default:
					throw new NotImplementedException($"Unhandled extension type: {extension}");
			}
		}

		private static (string, int) ExtractModuleInformationPdb(Stream s)
		{
			// the pdb format is somewhat documented here: 
			// https://llvm.org/docs/PDB/MsfFile.html
			// as well as here
			// https://github.com/microsoft/microsoft-pdb

			using BinaryReader reader = new BinaryReader(s);
			// extract magic
			const string MagicHeader = "Microsoft C/C++ MSF 7.00\r\n\u001aDS\0\0\0";
			byte[] magicBytes = reader.ReadBytes(MagicHeader.Length);
			string magicString = Encoding.ASCII.GetString(magicBytes);
			if (!string.Equals(magicString, MagicHeader, StringComparison.OrdinalIgnoreCase))
			{
				throw new Exception("Failed to find expected magic for pdb header");
			}

			// parse the super block
			int blockSize = reader.ReadInt32();
			int freeBlockMapIndex = reader.ReadInt32();
			int blockCount = reader.ReadInt32();
			int directoryStreamLength = reader.ReadInt32();
			int _ = reader.ReadInt32();
			int hintBlock = reader.ReadInt32();

			switch (blockSize)
			{
				case 512:
				case 1024:
				case 2048:
				case 4096:
					break;
				default:
					throw new Exception($"Unsupported block size: {blockSize}");
			}

			if (freeBlockMapIndex != 1 && freeBlockMapIndex != 2)
			{
				throw new Exception("Unexpected free block map index");
			}

			if (blockCount * blockSize != reader.BaseStream.Length)
			{
				throw new Exception("Unexpected pdb stream length");
			}

			// build the stream directory
			int freeBlockOffset = blockSize * hintBlock;
			reader.BaseStream.Seek(freeBlockOffset, SeekOrigin.Begin);
			
			int directoryStreamBlockCount = (int)Math.Ceiling((double)directoryStreamLength / blockSize);
			int[] directoryBlocks = new int[directoryStreamBlockCount];
			for (int i = 0; i < directoryStreamBlockCount; ++i)
			{
				directoryBlocks[i] = reader.ReadInt32();
			}
			
			byte[] streamDirectory = new byte[directoryStreamBlockCount * blockSize];
			int streamDirectoryOffset = 0;

			foreach (int directoryBlock in directoryBlocks)
			{
				reader.BaseStream.Seek(blockSize * directoryBlock, SeekOrigin.Begin);
				streamDirectoryOffset += reader.Read(streamDirectory, streamDirectoryOffset, blockSize);
			}
			
			// read the stream directory
			using Stream streamDirectoryStream = new MemoryStream(streamDirectory);
			using BinaryReader streamDirectoryReader = new BinaryReader(streamDirectoryStream);
			int streamCount = streamDirectoryReader.ReadInt32();

			int[] streamLengths = new int[streamCount];
			for (int i = 0; i < streamCount; ++i)
			{
				streamLengths[i] = streamDirectoryReader.ReadInt32();
			}

			// calculate the blocks for each size
			List<List<int>> blocks = new List<List<int>>();
			for (int i = 0; i < streamCount; ++i)
			{
				int streamBlockCount = (int)Math.Ceiling((double)streamLengths[i] / blockSize);

				List<int> blocklist = new List<int>();
				for (int j = 0; j < streamBlockCount; ++j)
				{
					blocklist.Add(streamDirectoryReader.ReadInt32());
				}

				blocks.Add(blocklist);
			}

			int? pdbVersion = null;
			int? pdbSignature = null;
			int? pdbAge = null;
			Guid? pdbGuid = null;

			bool infoFound = false;
			// start reading each stream
			for (int i = 0; i < streamCount; ++i)
			{
				if (infoFound)
				{
					break;
				}
				byte[] streamBuffer = new byte[streamLengths[i]];
				int destinationIndex = 0;

				List<int> streamBlocks = blocks[i];
				foreach (int streamBlock in streamBlocks)
				{
					reader.BaseStream.Seek(streamBlock * blockSize, SeekOrigin.Begin);
					destinationIndex += reader.Read(streamBuffer, destinationIndex, Math.Min(blockSize, streamBuffer.Length - destinationIndex));
				}

				// all blocks are combined into a contiguous stream
				MemoryStream ms = new MemoryStream(streamBuffer);
				using BinaryReader streamReader = new BinaryReader(ms);
				switch (i)
				{
					case 1:
						// PdbInfoStream
						pdbVersion = streamReader.ReadInt32();
						pdbSignature = streamReader.ReadInt32();
						pdbAge = streamReader.ReadInt32();

						pdbGuid = new Guid(streamReader.ReadBytes(16));

						infoFound = true;
						break;

					case 3:
						// DBIStream
						break;
					default:
						break;
				}
			}

			if (pdbGuid == null || pdbAge == null)
			{
				throw new Exception("No PDBInfoStream found, was this really a pdb?");
			}

			return (pdbGuid.Value.ToString("N").ToUpperInvariant(), pdbAge.Value);
		}
	}

	public class SymbolsSettings
	{
		public bool EnableAuditLog { get; set; }
	}
}
