// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading;
using System.Threading.Tasks;
using Jupiter.Implementation;
using Jupiter.Implementation.LeaderElection;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Options;

namespace Jupiter
{
	public class RefCleanupServiceCheck : IHealthCheck
	{
		private readonly RefCleanupService _refCleanupService;

		public RefCleanupServiceCheck(RefCleanupService refCleanupService)
		{
			_refCleanupService = refCleanupService;
		}

		public Task<HealthCheckResult> CheckHealthAsync(HealthCheckContext context, CancellationToken cancellationToken = new CancellationToken())
		{
			if (_refCleanupService.Running)
			{
				return Task.FromResult(HealthCheckResult.Healthy());
			}

			return Task.FromResult(HealthCheckResult.Degraded());
		}
	}

	public class BlobCleanupServiceCheck : IHealthCheck
	{
		private readonly BlobCleanupService _blobCleanupService;

		public BlobCleanupServiceCheck(BlobCleanupService blobCleanupService)
		{
			_blobCleanupService = blobCleanupService;
		}

		public Task<HealthCheckResult> CheckHealthAsync(HealthCheckContext context, CancellationToken cancellationToken = new CancellationToken())
		{
			if (_blobCleanupService.Running)
			{
				return Task.FromResult(HealthCheckResult.Healthy());
			}

			return Task.FromResult(HealthCheckResult.Degraded());
		}
	}

	public class KubernetesLeaderServiceCheck : IHealthCheck
	{
		private readonly KubernetesLeaderElection _leaderElectionService;

		public KubernetesLeaderServiceCheck(KubernetesLeaderElection leaderElectionService)
		{
			_leaderElectionService = leaderElectionService;
		}

		public Task<HealthCheckResult> CheckHealthAsync(HealthCheckContext context, CancellationToken cancellationToken = new CancellationToken())
		{
			if (_leaderElectionService.Running)
			{
				return Task.FromResult(HealthCheckResult.Healthy());
			}

			return Task.FromResult(HealthCheckResult.Degraded());
		}
	}

	public class LastAccessServiceCheck : IHealthCheck
	{
		private readonly LastAccessServiceReferences _lastAccessService;

		public LastAccessServiceCheck(LastAccessServiceReferences lastAccessService)
		{
			_lastAccessService = lastAccessService;
		}

		public Task<HealthCheckResult> CheckHealthAsync(HealthCheckContext context, CancellationToken cancellationToken = new CancellationToken())
		{
			if (_lastAccessService.Running)
			{
				return Task.FromResult(HealthCheckResult.Healthy());
			}

			return Task.FromResult(HealthCheckResult.Degraded());
		}
	}

	public class ReplicationSnapshotServiceCheck : IHealthCheck
	{
		private readonly ReplicationSnapshotService _replicationSnapshotService;
		private readonly IOptionsMonitor<SnapshotSettings> _snapshotSettings;

		public ReplicationSnapshotServiceCheck(ReplicationSnapshotService replicationSnapshotService, IOptionsMonitor<SnapshotSettings> snapshotSettings)
		{
			_replicationSnapshotService = replicationSnapshotService;
			_snapshotSettings = snapshotSettings;
		}

		public Task<HealthCheckResult> CheckHealthAsync(HealthCheckContext context, CancellationToken cancellationToken = new CancellationToken())
		{
			if (!_snapshotSettings.CurrentValue.Enabled)
			{
				return Task.FromResult(HealthCheckResult.Healthy());
			}

			if (_replicationSnapshotService.Running)
			{
				return Task.FromResult(HealthCheckResult.Healthy());
			}

			return Task.FromResult(HealthCheckResult.Degraded());
		}
	}

	public class ReplicatorServiceCheck : IHealthCheck
	{
		private readonly ReplicationService _replicationService;
		private readonly IOptionsMonitor<ReplicationSettings> _replicationSetting;

		public ReplicatorServiceCheck(ReplicationService replicationService, IOptionsMonitor<ReplicationSettings> replicationSetting)
		{
			_replicationService = replicationService;
			_replicationSetting = replicationSetting;
		}

		public Task<HealthCheckResult> CheckHealthAsync(HealthCheckContext context, CancellationToken cancellationToken = new CancellationToken())
		{
			// if replication is disabled we consider it healthy
			if (!_replicationSetting.CurrentValue.Enabled)
			{
				return Task.FromResult(HealthCheckResult.Healthy());
			}

			if (_replicationService.Running)
			{
				return Task.FromResult(HealthCheckResult.Healthy());
			}

			return Task.FromResult(HealthCheckResult.Degraded());
		}
	}
}
