// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Authorization.Infrastructure;

namespace Jupiter
{
	/// <summary>
	/// An implementation of <see cref="IAuthorizationRequirement"/> that combines multiple
	/// requirements such that if any of those inner requirements are met, this requirement is met.
	/// This only supports requirements that handle themselves, i.e. implement both <see cref="IAuthorizationRequirement"/>
	/// and the corresponding <see cref="AuthorizationHandler{TRequirement}"/>.
	/// </summary>
	public class AnyRequirement : AuthorizationHandler<AnyRequirement>, IAuthorizationRequirement
	{
		public AnyRequirement(params IAuthorizationRequirement[] innerRequirements)
		{
			InnerRequirements = innerRequirements ?? throw new ArgumentNullException(nameof(innerRequirements));
		}

		public IEnumerable<IAuthorizationRequirement> InnerRequirements { get; }

		protected override async Task HandleRequirementAsync(AuthorizationHandlerContext context, AnyRequirement requirement)
		{
			foreach (IAuthorizationRequirement innerRequirement in requirement.InnerRequirements)
			{
				// This only supports requirements that "handle themselves".
				IAuthorizationHandler? handler = innerRequirement as IAuthorizationHandler;
				if (handler != null)
				{
					AuthorizationHandlerContext innerContext = new AuthorizationHandlerContext(new[] { innerRequirement }, context.User, context.Resource);
					await handler.HandleAsync(innerContext);
					if (innerContext.HasSucceeded)
					{
						context.Succeed(this);
						break;
					}
				}
			}
		}

		public override string ToString()
		{
			string components = string.Join(',', InnerRequirements.Select(r => r.ToString()));
			return $"Any({components})";
		}
	}

	/// <summary>
	/// A helper class to build an instance of <see cref="AnyRequirement"/>.
	/// </summary>
	public class AnyRequirementBuilder
	{
		private readonly List<IAuthorizationRequirement> _innerRequirements = new List<IAuthorizationRequirement>();
		private readonly AuthorizationPolicyBuilder _policyBuilder;

		public AnyRequirementBuilder(AuthorizationPolicyBuilder policyBuilder)
		{
			_policyBuilder = policyBuilder;
		}

		public AnyRequirementBuilder AddRequirement(IAuthorizationRequirement requirement)
		{
			_innerRequirements.Add(requirement);
			return this;
		}

		public AuthorizationPolicyBuilder AddAny()
		{
			AnyRequirement anyRequirement = new AnyRequirement(_innerRequirements.ToArray());
			return _policyBuilder.AddRequirements(anyRequirement);
		}
	}

	/// <summary>
	/// Helpful extensions related to <see cref="AnyRequirement"/>
	/// </summary>
	public static class AnyRequirementExtensions
	{
		/// <summary>
		/// Starts a building context to add requirements that will be combined as an <see cref="AnyRequirement"/>
		/// and added to the policy builder when <see cref="AnyRequirementBuilder.AddAny"/> is called later.
		/// </summary>
		/// <param name="policyBuilder"></param>
		/// <returns></returns>
		public static AnyRequirementBuilder BeginAnyContext(this AuthorizationPolicyBuilder policyBuilder)
		{
			return new AnyRequirementBuilder(policyBuilder);
		}

		public static AnyRequirementBuilder IncludeClaim(this AnyRequirementBuilder builder, string claimType, params string[] allowedValues)
		{
			return builder.AddRequirement(new ClaimsAuthorizationRequirement(claimType, allowedValues));
		}

		public static AnyRequirementBuilder IncludeClaim(this AnyRequirementBuilder builder, string claimType)
		{
			return builder.AddRequirement(new ClaimsAuthorizationRequirement(claimType, allowedValues: null));
		}

		public static AnyRequirementBuilder IncludeRole(this AnyRequirementBuilder builder, params string[] roles)
		{
			return builder.AddRequirement(new RolesAuthorizationRequirement(roles));
		}
	}
}
