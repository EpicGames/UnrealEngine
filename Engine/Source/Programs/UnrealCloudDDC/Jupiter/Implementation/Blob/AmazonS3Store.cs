// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using Amazon.S3;
using Amazon.S3.Model;
using Amazon.S3.Transfer;
using Amazon.S3.Util;
using EpicGames.Core;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Common.Utils;
using Jupiter.DataStore;
using Jupiter.Implementation.Blob;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;
using Exception = System.Exception;
using KeyNotFoundException = System.Collections.Generic.KeyNotFoundException;

namespace Jupiter.Implementation
{
	public class AmazonS3Store : IBlobStore, IMultipartBlobStore, IDataStoreObjectStore
	{
		private readonly IAmazonS3 _amazonS3;
		private IBlobIndex? _blobIndex = null;
		private readonly Lock _blobIndexLock = new Lock();
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly ILogger<AmazonS3Store> _logger;
		private readonly IServiceProvider _provider;
		private readonly S3Settings _settings;
		private readonly JupiterSettings _jupiterSettings;
		private readonly ConcurrentDictionary<string, AmazonStorageBackend> _backends = new ConcurrentDictionary<string, AmazonStorageBackend>();

		public AmazonS3Store(IAmazonS3 amazonS3, IOptionsMonitor<S3Settings> settings, IOptionsMonitor<JupiterSettings> jupiterSettings, INamespacePolicyResolver namespacePolicyResolver, ILogger<AmazonS3Store> logger, IServiceProvider provider)
		{
			_amazonS3 = amazonS3;
			_namespacePolicyResolver = namespacePolicyResolver;
			_logger = logger;
			_provider = provider;
			_settings = settings.CurrentValue;
			_jupiterSettings = jupiterSettings.CurrentValue;
		}

		private AmazonStorageBackend GetBackend(NamespaceId ns, string? region = null)
		{
			region ??= _jupiterSettings.CurrentSite;
			return _backends.GetOrAdd($"{ns}:{region}", x => ActivatorUtilities.CreateInstance<AmazonStorageBackend>(_provider, GetBucketName(ns, region)));
		}

		private AmazonStorageBackend GetBackend(KeyspaceId keyspace, string? region = null)
		{
			region ??= _jupiterSettings.CurrentSite;

			string? bucketName = null;
			if (_settings.SiteBucketsPerStoragePool.TryGetValue(keyspace.ToString(), out Dictionary<string, string>? siteBuckets))
			{
				if (siteBuckets.TryGetValue(region, out string? b))
				{
					bucketName = b;
				}
			}

			if (bucketName == null)
			{
				// was not a storage pool, we then assume its a namespace
				NamespaceId maybeNs = keyspace.ToNamespaceId();
				bucketName = GetBucketName(maybeNs, region);
			}

			if (bucketName == null)
			{
				throw new Exception($"Unable to determine bucket name for keyspace {keyspace}");
			}

			return _backends.GetOrAdd($"{keyspace}:{region}", x => ActivatorUtilities.CreateInstance<AmazonStorageBackend>(_provider, bucketName));
		}

		public async Task<Uri?> GetObjectByRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			Uri? uri = await GetBackend(ns).GetReadRedirectAsync(identifier.AsS3Key());

			return uri;
		}

		public async Task<BlobMetadata> GetObjectMetadataAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			BlobMetadata? result = await GetBackend(ns).GetMetadataAsync(identifier.AsS3Key(), cancellationToken);
			if (result == null)
			{
				throw new BlobNotFoundException(ns, identifier);
			}

			return result;
		}

		public async Task<ulong> CopyBlobAsync(NamespaceId ns, NamespaceId targetNamespace, BlobId blobId, CancellationToken cancellationToken = default)
		{
			return await GetBackend(targetNamespace).CopyBlobAsync(GetBackend(ns), blobId.AsS3Key(), blobId.AsS3Key(), cancellationToken);
		}

		public async Task<Uri?> PutObjectWithRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			Uri? uri = await GetBackend(ns).GetWriteRedirectAsync(identifier.AsS3Key());

			return uri;
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, ReadOnlyMemory<byte> content, BlobId objectName, CancellationToken cancellationToken = default)
		{
			await using MemoryStream stream = new MemoryStream(content.ToArray());
			return await PutObjectAsync(ns, stream, objectName, cancellationToken);
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, Stream stream, BlobId objectName, CancellationToken cancellationToken = default)
		{
			await GetBackend(ns).WriteAsync(objectName.AsS3Key(), stream, cancellationToken);
			return objectName;
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, byte[] content, BlobId objectName, CancellationToken cancellationToken = default)
		{
			await using MemoryStream stream = new MemoryStream(content);
			return await PutObjectAsync(ns, stream, objectName, cancellationToken);
		}

		private string GetBucketName(NamespaceId ns, string site)
		{
			try
			{
				NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
				string storagePool = policy.StoragePool;

				// Prefer explicitly defined bucket names for the current storage pool and region
				if (_settings.SiteBucketsPerStoragePool.TryGetValue(storagePool, out Dictionary<string, string>? siteBuckets))
				{
					if (siteBuckets.TryGetValue(site, out string? bucketName))
					{
						return bucketName;
					}
				}

				if (site != _jupiterSettings.CurrentSite)
				{
					throw new Exception("Unable to resolve bucket name for specific site. Please define regional bucket names in S3.SiteBucketsPerStoragePool");
				}
				
				// if the bucket to use for the storage pool has been overriden we use the override
				if (_settings.StoragePoolBucketOverride.TryGetValue(storagePool, out string? containerOverride))
				{
					return containerOverride;
				}
				// by default we use the storage pool as a suffix to determine the bucket for that pool
				string storagePoolSuffix = string.IsNullOrEmpty(storagePool) ? "" : $"-{storagePool}";
				return $"{_settings.BucketName}{storagePoolSuffix}";
			}
			catch (KeyNotFoundException)
			{
				throw new NamespaceNotFoundException(ns);
			}
		}

		public async Task<BlobContents> GetObjectAsync(NamespaceId ns, BlobId blob, LastAccessTrackingFlags flags = LastAccessTrackingFlags.DoTracking, bool supportsRedirectUri = false, CancellationToken cancellationToken = default)
		{
			NamespacePolicy policies = _namespacePolicyResolver.GetPoliciesForNs(ns);
			try
			{
				if (supportsRedirectUri && policies.AllowRedirectUris)
				{
					Uri? redirectUri = await GetBackend(ns).GetReadRedirectAsync(blob.AsS3Key());
					if (redirectUri != null)
					{
						return new BlobContents(redirectUri);
					}
				}

				BlobContents? contents = await GetBackend(ns).TryReadAsync(blob.AsS3Key(), flags, cancellationToken);
				if (contents == null)
				{
					throw new BlobNotFoundException(ns, blob);
				}
				return contents;
			}
			catch (AmazonS3Exception e)
			{
				// log information about the failed request except for 404 as its valid to not find objects in S3
				if (e.StatusCode != HttpStatusCode.NotFound)
				{
					_logger.LogWarning("Exception raised from S3 {Exception}. {RequestId} {Id}", e, e.RequestId, e.AmazonId2);
				}

				// rethrow the exception, we just wanted to log more information about the failed request for further debugging
				throw;
			}
		}

		public async Task<bool> ExistsAsync(NamespaceId ns, BlobId blobIdentifier, bool forceCheck, CancellationToken cancellationToken = default)
		{
			NamespacePolicy policies = _namespacePolicyResolver.GetPoliciesForNs(ns);
			if (_settings.UseBlobIndexForExistsCheck && policies.UseBlobIndexForSlowExists && !forceCheck)
			{
				if (_blobIndex == null)
				{
					lock (_blobIndexLock)
					{
						// late resolve the blob index to avoid circular dependencies
						_blobIndex = _provider.GetRequiredService<IBlobIndex>();
					}
				}
				
				return await _blobIndex.BlobExistsInRegionAsync(ns, blobIdentifier, cancellationToken: cancellationToken);
			}
			else
			{
				return await GetBackend(ns).ExistsAsync(blobIdentifier.AsS3Key(), cancellationToken);
			}
		}

		public async Task DeleteNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			string bucketName = GetBucketName(ns, _jupiterSettings.CurrentSite);
			try
			{
				await _amazonS3.DeleteBucketAsync(bucketName, cancellationToken);
			}
			catch (AmazonS3Exception e)
			{
				// if the bucket does not exist we get a not found status code
				if (e.StatusCode == HttpStatusCode.NotFound)
				{
					// deleting a none existent bucket is a success
					return;
				}

				// something else happened, lets just process it as usual
			}
		}

		public async IAsyncEnumerable<(BlobId, DateTime)> ListObjectsAsync(NamespaceId ns, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			IStorageBackend backend = GetBackend(ns);
			await foreach ((string path, DateTime time) in backend.ListAsync(cancellationToken))
			{
				// ignore objects in the temp prefix
				if (path.StartsWith("Temp", StringComparison.OrdinalIgnoreCase))
				{
					continue;
				}
				string identifierString = path.Substring(path.LastIndexOf("/", StringComparison.Ordinal) + 1);
				yield return (new BlobId(identifierString), time);
			}
		}

		public async Task DeleteObjectAsync(NamespaceId ns, BlobId blobIdentifier, CancellationToken cancellationToken = default)
		{
			IStorageBackend backend = GetBackend(ns);
			await backend.DeleteAsync(blobIdentifier.AsS3Key(), cancellationToken);
		}

		public async Task DeleteObjectAsync(IEnumerable<NamespaceId> namespaces, BlobId blob, CancellationToken cancellationToken = default)
		{
			List<NamespaceId> namespaceIds = namespaces.ToList();
			List<string> storagePools = namespaceIds.Select(ns => _namespacePolicyResolver.GetPoliciesForNs(ns).StoragePool).Distinct().ToList();

			Dictionary<string, NamespaceId> storagePoolsToClean = storagePools.ToDictionary(storagePool => storagePool, storagePool => namespaceIds.FirstOrDefault(id => _namespacePolicyResolver.GetPoliciesForNs(id).StoragePool == storagePool));

			foreach ((string _, NamespaceId ns) in storagePoolsToClean)
			{
				await GetBackend(ns).DeleteAsync(blob.AsS3Key(), cancellationToken);
			}
		}
		
		#region IMultipartBlobStore
		
		
		private const string TempMultipartPrefix = "Temp/Multipart";
		public Task<string> StartMultipartUploadAsync(NamespaceId ns, string blobName, CancellationToken cancellationToken = default)
		{
			return GetBackend(ns).StartMultipartUploadAsync($"{TempMultipartPrefix}/{blobName}", cancellationToken);
		}

		public Task CompleteMultipartUploadAsync(NamespaceId ns, string blobName, string uploadId, List<string> partIds, CancellationToken cancellationToken = default)
		{
			return GetBackend(ns).CompleteMultipartUploadAsync($"{TempMultipartPrefix}/{blobName}", uploadId, partIds, cancellationToken);
		}

		public Task<Uri?> GetWriteRedirectForPartAsync(NamespaceId ns, string blobName, string uploadId, string partIdentifier, CancellationToken cancellationToken = default)
		{
			return Task.FromResult(GetBackend(ns).GetWriteRedirectForPart($"{TempMultipartPrefix}/{blobName}", uploadId, partIdentifier));
		}
		
		public async Task<BlobContents?> GetMultipartObjectByNameAsync(NamespaceId ns, string blobName, CancellationToken cancellationToken = default)
		{
			return await GetBackend(ns).TryReadAsync($"{TempMultipartPrefix}/{blobName}", LastAccessTrackingFlags.SkipTracking, cancellationToken);
		}

		public async Task RenameMultipartBlobAsync(NamespaceId ns, string blobName, BlobId blobId, CancellationToken cancellationToken = default)
		{
			await GetBackend(ns).RenameMultipartBlobAsync($"{TempMultipartPrefix}/{blobName}", blobId.AsS3Key(), cancellationToken);
		}
		public Task PutMultipartPartAsync(NamespaceId ns, string blobName, string uploadId, string partIdentifier, byte[] blob, CancellationToken cancellationToken = default)
		{
			return GetBackend(ns).PutMultipartPartAsync($"{TempMultipartPrefix}/{blobName}", uploadId, partIdentifier, blob, cancellationToken);
		}

		public List<MultipartByteRange> GetMultipartRanges(NamespaceId ns, string uploadId, ulong blobLength, CancellationToken cancellationToken = default)
		{
			ulong countOfChunks = (blobLength / IdealS3ChunkSize) + 1;

			if (Math.Min(countOfChunks, MaxS3Chunks) == MaxS3Chunks)
			{
				throw new Exception("Multipart blob would use more then max chunks in S3, this is not supported");
			}

			List<MultipartByteRange> parts = new();
			ulong firstByte = 0;
			ulong lastByte = IdealS3ChunkSize; // last byte is inclusive as per http range requests

			for (int i = 1; i < (int)countOfChunks+1; i++)
			{
				// s3 parts start at 1 and max is 10_000 (inclusive)
				// s3 part ids are simply incrementing numbers
				parts.Add(new MultipartByteRange() {FirstByte = firstByte, LastByte = lastByte, PartId=i.ToString()});

				firstByte += IdealS3ChunkSize;
				lastByte = Math.Min(lastByte + IdealS3ChunkSize, blobLength);
			}

			return parts;
		}

		public MultipartLimits GetMultipartLimits(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			return new MultipartLimits { IdealChunkSize = IdealS3ChunkSize, MaxCountOfChunks = (int)MaxS3Chunks, MinChunkSize = 8 * 1024 * 1024 };
		}

		private const ulong MaxS3Chunks = 10_000;
		private const int IdealS3ChunkSize = 32 * 1024 * 1024; // 32 MB parts - this means the largest file we can upload to S3 is 312GB

		#endregion

		#region IDataStoreObjectStore

		public async Task PutFileAsync(KeyspaceId keyspaceId, FileInfo localPath, string remotePath, CancellationToken cancellationToken)
		{
			await using Stream s = localPath.OpenRead();
			await GetBackend(keyspaceId).WriteAsync(remotePath, s, cancellationToken);
		}

		public async IAsyncEnumerable<(KeyspaceId keyspace, string prefix)> EnumeratePrefixesAsync(TableType typeName, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			if (!_settings.SiteBucketsPerStoragePool.TryGetValue(_jupiterSettings.CurrentSite, out Dictionary<string, string>? bucketsPerStoragePool))
			{
				throw new Exception($"Site buckets per storage pool not configured for site {_jupiterSettings.CurrentSite}");
			}

			foreach (KeyValuePair<string, string> keyspaceAndBucket in bucketsPerStoragePool)
			{
				string keyspace = keyspaceAndBucket.Key;
				string bucketName = keyspaceAndBucket.Value;

				ListObjectsV2Request request = new ListObjectsV2Request
				{
					BucketName = bucketName,
					Prefix = $"Objects/{typeName.ToString()}/",
				};

				Dictionary<string, HashSet<string>> foundPrefixes = [];
				ListObjectsV2Response response;
				do
				{
					try
					{
						response = await _amazonS3.ListObjectsV2Async(request, cancellationToken);
					}
					catch (AmazonS3Exception e)
					{
						// bucket not found means its empty
						if (e.ErrorCode == "NoSuchBucket")
						{
							break;
						}

						throw;
					}
					
					// iterate the common prefixes to find which directories existed
					foreach (S3Object? s3Object in response.S3Objects)
					{
						string[] components = s3Object.Key.Split('/');
						// we expect 5 components, Objects/<type>/<keyspace>/<prefix>/<object>
						if (components.Length <= 4)
						{
							continue;
						}
						string foundKeyspace = components[2];
						string prefix = components[3];

						if (!string.Equals(foundKeyspace, keyspace, StringComparison.OrdinalIgnoreCase))
						{
							continue;
						}

						if (foundPrefixes.TryGetValue(keyspace, out HashSet<string>? set))
						{
							if (set.Add(prefix))
							{
								yield return (new KeyspaceId(foundKeyspace), prefix);
							}
						}
						else
						{
							foundPrefixes.Add(keyspace, new HashSet<string> {prefix});
							yield return (new KeyspaceId(foundKeyspace), prefix);
						}
					}

					request.ContinuationToken = response.NextContinuationToken;
				} while (response.IsTruncated);
			}
		}

		public async Task<bool> GetFileAsync(KeyspaceId keyspaceId, string remotePath, FileInfo localFile, CancellationToken cancellationToken)
		{
			string bucketName = GetBackend(keyspaceId).GetBucketName();

			GetObjectResponse? response = null;
			try
			{
				response = await _amazonS3.GetObjectAsync(bucketName, remotePath, cancellationToken);

				await response.WriteResponseStreamToFileAsync(localFile.FullName, append: false, cancellationToken);
			}
			catch (AmazonS3Exception e)
			{
				if (e.ErrorCode == "NoSuchKey")
				{
					return false;
				}

				if (e.ErrorCode == "NoSuchBucket")
				{
					return false;
				}

				throw;
			}
			finally
			{
				response?.Dispose();
			}

			return true;
		}

		public async IAsyncEnumerable<DataStoreFile> EnumerateFilesAsync(KeyspaceId keyspaceId, string remotePath, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			string bucketName = GetBackend(keyspaceId).GetBucketName();

			ListObjectsV2Request request = new ListObjectsV2Request
			{
				BucketName = bucketName,
				Prefix = remotePath
			};

			ListObjectsV2Response response;
			do
			{
				try
				{
					response = await _amazonS3.ListObjectsV2Async(request, cancellationToken);
				}
				catch (AmazonS3Exception e)
				{
					if (e.ErrorCode == "NoSuchBucket")
					{
						yield break;
					}
					throw;
				}

				foreach (S3Object obj in response.S3Objects)
				{
					yield return new DataStoreFile(Path.GetFileName(obj.Key), obj.Size, obj.LastModified.ToUniversalTime());
				}

				request.ContinuationToken = response.NextContinuationToken;
			} while (response.IsTruncated);
		}

		public async Task DeleteFilesAsync(KeyspaceId keyspaceId, string remotePath, CancellationToken cancellationToken)
		{
			string bucketName = GetBackend(keyspaceId).GetBucketName();

			ListObjectsV2Request listRequest = new ListObjectsV2Request
			{
				BucketName = bucketName,
				Prefix = remotePath
			};

			List<KeyVersion> objectsToDelete = new List<KeyVersion>();
			ListObjectsV2Response response;
			do
			{
				response = await _amazonS3.ListObjectsV2Async(listRequest, cancellationToken);
				foreach (S3Object obj in response.S3Objects)
				{
					objectsToDelete.Add(new KeyVersion {Key = obj.Key});
				}

				listRequest.ContinuationToken = response.NextContinuationToken;
			} while (response.IsTruncated);

			if (objectsToDelete.Count == 0)
			{
				return;
			}

			foreach (IReadOnlyList<KeyVersion> keysToDelete in objectsToDelete.Batch(1000))
			{
				DeleteObjectsRequest request = new DeleteObjectsRequest() { BucketName = bucketName, Objects = keysToDelete.ToList() };
				await _amazonS3.DeleteObjectsAsync(request, cancellationToken);
			}
		}

		#endregion
	}

	public class AmazonStorageBackend : IStorageBackend
	{
		private readonly IAmazonS3 _amazonS3;
		private readonly string _bucketName;
		private readonly IOptionsMonitor<S3Settings> _settings;
		private readonly Tracer _tracer;
		private readonly ILogger<AmazonStorageBackend> _logger;
		private readonly BufferedPayloadFactory _payloadFactory;
		private bool _bucketExistenceChecked;
		private bool _bucketAccessPolicyApplied;

		public AmazonStorageBackend(IAmazonS3 amazonS3, string bucketName, IOptionsMonitor<S3Settings> settings, Tracer tracer, ILogger<AmazonStorageBackend> logger, BufferedPayloadFactory payloadFactory)
		{
			_amazonS3 = amazonS3;
			_bucketName = bucketName;
			_settings = settings;
			_tracer = tracer;
			_logger = logger;
			_payloadFactory = payloadFactory;
		}

		public async Task EnsureBucketExistsAsync(CancellationToken cancellationToken = default)
		{
			if (_settings.CurrentValue.CreateBucketIfMissing)
			{
				if (!_bucketExistenceChecked)
				{
					bool bucketExist = await AmazonS3Util.DoesS3BucketExistV2Async(_amazonS3, _bucketName);
					if (!bucketExist)
					{
						PutBucketRequest putBucketRequest = new PutBucketRequest
						{
							BucketName = _bucketName,
							UseClientRegion = true
						};

						try
						{
							await _amazonS3.PutBucketAsync(putBucketRequest, cancellationToken);
						}
						catch (AmazonS3Exception e)
						{
							if (e.StatusCode == HttpStatusCode.Conflict)
							{
								// bucket already exists so no need to try and create it, most likely a race condition with another write operation that already tried to create the bucket at the same time in this or another instance
								// as this will end up with the bucket existing we can just ignore the error
							}
							else
							{
								throw;
							}
						}
					}
					_bucketExistenceChecked = true;
				}
			}

			if (_settings.CurrentValue.SetBucketPolicies && !_bucketAccessPolicyApplied)
			{
				// block all public access to the bucket
				try
				{
					await _amazonS3.PutPublicAccessBlockAsync(new PutPublicAccessBlockRequest
					{
						BucketName = _bucketName,
						PublicAccessBlockConfiguration = new PublicAccessBlockConfiguration()
						{
							RestrictPublicBuckets = true,
							BlockPublicAcls = true,
							BlockPublicPolicy = true,
							IgnorePublicAcls = true,
						}
					}, cancellationToken);

					_bucketAccessPolicyApplied = true;
				}
				catch (AmazonS3Exception e)
				{
					// if a conflicting operation is being applied to the public access block we just ignore it, as it will get reset the next time we run
					if (e.StatusCode != HttpStatusCode.Conflict)
					{
						throw;
					}
				}
			}
		}

		public async Task WriteAsync(string path, Stream stream, CancellationToken cancellationToken = default)
		{
			await EnsureBucketExistsAsync(cancellationToken);

			if (_settings.CurrentValue.UseMultiPartUpload)
			{
				await WriteMultipartAsync(path, stream, cancellationToken);
			}
			else
			{
				PutObjectRequest request = new PutObjectRequest
				{
					BucketName = _bucketName,
					Key = path,
					InputStream = stream,
					UseChunkEncoding = _settings.CurrentValue.UseChunkEncoding,
					DisablePayloadSigning = _settings.CurrentValue.DisablePayloadChecksums,
					DisableDefaultChecksumValidation = _settings.CurrentValue.DisablePayloadChecksums,
				};

				try
				{
					await _amazonS3.PutObjectAsync(request, cancellationToken);
				}
				catch (AmazonS3Exception e)
				{
					// if the same object is added twice S3 will raise a error, as we are content addressed we can just accept whichever of the objects so we can ignore that error
					if (e.StatusCode == HttpStatusCode.Conflict)
					{
						return;
					}

					if (e.StatusCode == HttpStatusCode.TooManyRequests)
					{
						throw new ResourceHasToManyRequestsException(e);
					}

					throw;
				}
			}
		}

		private async Task WriteMultipartAsync(string path, Stream stream, CancellationToken cancellationToken = default)
		{
			IBufferedPayload? payload = null;
			try
			{
				string? filePath = null;
				if (stream is FileStream fileStream)
				{
					filePath = fileStream.Name;
				}
				// use multipart transfers when buffer is larger than 16 MB, which is also the S3 default
				else if (stream.Length > 16 * (long)Math.Pow(2, 20))
				{
					// will be chunked by TransferUtility
					using FilesystemBufferedPayloadWriter writer = _payloadFactory.CreateFilesystemBufferedPayloadWriter("s3-upload");
					{
						await using Stream writableStream = writer.GetWritableStream();
						await stream.CopyToAsync(writableStream, cancellationToken);
					}
					FilesystemBufferedPayload filePayload = writer.Done();
					payload = filePayload;

					filePath = filePayload.TempFile.FullName;
				}
				else
				{
					// copy the stream as TransferUtility will dispose this stream if any issue happens making us unable to retry
					payload = await _payloadFactory.CreateFromStreamAsync(stream, stream.Length, "s3-upload", cancellationToken);
				}

				const int MaxRetryAttempts = 3;
				for (int i = 0; i < MaxRetryAttempts; i++)
				{
					using TransferUtility utility = new TransferUtility(_amazonS3);
					try
					{
						if (filePath != null)
						{
							TransferUtilityUploadRequest uploadRequest = new TransferUtilityUploadRequest()
							{
								BucketName = _bucketName,
								Key = path,
								FilePath = filePath,
								DisablePayloadSigning = _settings.CurrentValue.DisablePayloadChecksums,
								DisableDefaultChecksumValidation = _settings.CurrentValue.DisablePayloadChecksums,
							};
							await utility.UploadAsync(uploadRequest, cancellationToken);
						}
						else
						{
							await using Stream localStream = payload!.GetStream();
							TransferUtilityUploadRequest uploadRequest = new TransferUtilityUploadRequest()
							{
								BucketName = _bucketName,
								Key = path,
								InputStream = localStream,
								DisablePayloadSigning = _settings.CurrentValue.DisablePayloadChecksums,
								DisableDefaultChecksumValidation = _settings.CurrentValue.DisablePayloadChecksums,
							};
							await utility.UploadAsync(uploadRequest, cancellationToken);
						}

						break;
					}
					catch (AmazonS3Exception e)
					{
						// if the same object is added twice S3 will raise a error, as we are content addressed we can just accept whichever of the objects so we can ignore that error
						if (e.StatusCode == HttpStatusCode.Conflict)
						{
							return;
						}

						if (e.StatusCode == HttpStatusCode.TooManyRequests)
						{
							throw new ResourceHasToManyRequestsException(e);
						}

						if (e.StatusCode == HttpStatusCode.ServiceUnavailable && (e.ErrorCode == "SlowDown" || e.ErrorCode == "503 SlowDown"))
						{
							throw new ResourceHasToManyRequestsException(e);
						}

						throw;
					}
					catch (HttpIOException e)
					{
						if (i < MaxRetryAttempts-1)
						{
							_logger.LogWarning("Http IO exception when writing multipart blob to S3, retrying");
							// retry
							continue;
						}

						// to many retries, give up
						_logger.LogError(e, "Http IO exception when writing multipart blob to S3");
						throw;
					}
					catch (IOException e)
					{
						if (i < MaxRetryAttempts-1)
						{
							_logger.LogWarning("Http IO exception when writing multipart blob to S3, retrying");
							// retry
							continue;
						}

						// to many retries, give up
						_logger.LogError(e, "IO exception when writing multipart blob to S3");
						throw;
					}
				}
			}
			finally
			{
				payload?.Dispose();
			}
		}

		private async Task<BlobContents?> GetMultipartAsync(string path, CancellationToken cancellationToken = default)
		{
			try
			{
				GetObjectRequest request = new GetObjectRequest { BucketName = _bucketName, Key = path, PartNumber = 1};
				GetObjectResponse response = await _amazonS3.GetObjectAsync(request, cancellationToken);
				// parts are only set of blobs that was uploaded using multipart, otherwise part 1 is the whole file
				if (response.PartsCount > 1)
				{
					// object had parts, we download them in parallel
					// -1 because parts start at 1
					int partsCount = response.PartsCount.Value;

					byte[][] parts = new byte[partsCount][];
					long[] lengths = new long[partsCount];

					try
					{
						{
							parts[0] = ArrayPool<byte>.Shared.Rent((int)response.ContentLength);
							lengths[0] = response.ContentLength;
							await using MemoryStream ms = new MemoryStream(parts[0]);
							await response.ResponseStream.CopyToAsync(ms, cancellationToken);
						}

						// multipart object, download it in parts, we already have the first part
						await Parallel.ForEachAsync(Enumerable.Range(2, partsCount-1), cancellationToken, async (part, token) =>
						{
							GetObjectRequest partRequest = new GetObjectRequest { BucketName = _bucketName, Key = path, PartNumber = part };
							GetObjectResponse partResponse = await _amazonS3.GetObjectAsync(partRequest, cancellationToken);

							// the first part was retrieved in the initial request
							parts[part-1] = ArrayPool<byte>.Shared.Rent((int)partResponse.ContentLength);
							lengths[part-1] = partResponse.ContentLength;
							await using MemoryStream ms = new MemoryStream(parts[part-1]);
							await partResponse.ResponseStream.CopyToAsync(ms, cancellationToken);
						});

						long totalSize = lengths.Sum();
						if (totalSize < _settings.CurrentValue.MultiPartMaxMemoryBufferSize)
						{
							Stream s = new MemoryStream(new byte[(int)totalSize]);
							for (int i = 0; i < parts.Length; i++)
							{
								byte[] part = parts[i];
								await s.WriteAsync(part, 0, (int)lengths[i], cancellationToken);
							}

							s.Position = 0;
							return new BlobContents(s, totalSize);
						}
						else
						{
							// large payload - we have to buffer to file
							using FilesystemBufferedPayloadWriter writer = _payloadFactory.CreateFilesystemBufferedPayloadWriter("s3-download");

							{
								await using Stream s = writer.GetWritableStream();
								for (int i = 0; i < parts.Length; i++)
								{
									byte[] part = parts[i];
									await s.WriteAsync(part, 0, (int)lengths[i], cancellationToken);
								}
							}
						
							return new BlobContents(writer.Done());
						}
					}
					finally
					{
						for (int i = 0; i < partsCount; i++)
						{
							// ReSharper disable once ConditionIsAlwaysTrueOrFalseAccordingToNullableAPIContract
							// This should never be null but during exceptions we have seen it be null so to avoid throwing more exceptions that hide the original exceptions we just null check here
							if (parts[i] != null)
							{
								ArrayPool<byte>.Shared.Return(parts[i]);
							}
						}
					}
				}
				else
				{
					return new BlobContents(response.ResponseStream, response.ContentLength);
				}
			}
			catch (AmazonS3Exception e)
			{
				if (e.ErrorCode == "NoSuchKey")
				{
					return null;
				}

				if (e.ErrorCode == "NoSuchBucket")
				{
					return null;
				}

				if (e.StatusCode == HttpStatusCode.TooManyRequests)
				{
					throw new ResourceHasToManyRequestsException(e);
				}

				throw;
			}
		}

		public async Task<BlobContents?> TryReadAsync(string path, LastAccessTrackingFlags flags = LastAccessTrackingFlags.DoTracking, CancellationToken cancellationToken = default)
		{
			if (_settings.CurrentValue.UseMultiPartDownload)
			{
				return await GetMultipartAsync(path, cancellationToken);
			}
			else
			{
				GetObjectResponse response;
				try
				{
					response = await _amazonS3.GetObjectAsync(_bucketName, path, cancellationToken);
				}
				catch (AmazonS3Exception e)
				{
					if (e.ErrorCode == "NoSuchKey")
					{
						return null;
					}

					if (e.ErrorCode == "NoSuchBucket")
					{
						return null;
					}
					throw;
				}
				return new BlobContents(response.ResponseStream, response.ContentLength);
			}
		}

		public async Task<bool> ExistsAsync(string path, CancellationToken cancellationToken = default)
		{
			try
			{
				await _amazonS3.GetObjectMetadataAsync(_bucketName, path, cancellationToken);
			}
			catch (AmazonS3Exception e)
			{
				// if the object does not exist we get a not found status code
				if (e.StatusCode == HttpStatusCode.NotFound)
				{
					return false;
				}

				// something else happened, lets just process it as usual
			}

			return true;
		}

		public async IAsyncEnumerable<(string, DateTime)> ListAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			if (_settings.CurrentValue.PerPrefixListing)
			{
				List<string> hashPrefixes = new List<string>(65536);
				int i = 0;
				for (int a = 0; a <= byte.MaxValue; a++)
				{
					for (int b = 0; b <= byte.MaxValue; b++)
					{
						hashPrefixes.Add(StringUtils.FormatAsHexString(new byte[] { (byte)a, (byte)b }));
						i++;
					}
				}

				hashPrefixes.Shuffle();

				if (!await AmazonS3Util.DoesS3BucketExistV2Async(_amazonS3, _bucketName))
				{
					yield break;
				}

				foreach (string hashPrefix in hashPrefixes)
				{
					ListObjectsV2Request request = new ListObjectsV2Request
					{
						BucketName = _bucketName,
						Prefix = hashPrefix,
						MaxKeys = _settings.CurrentValue.PerPrefixMaxKeys
					};

					ListObjectsV2Response response;
					do
					{
						response = await _amazonS3.ListObjectsV2Async(request, cancellationToken);
						foreach (S3Object obj in response.S3Objects)
						{
							yield return (obj.Key, obj.LastModified);
						}

						request.ContinuationToken = response.NextContinuationToken;
					} while (response.IsTruncated);
				}
			}
			else
			{
				if (!await AmazonS3Util.DoesS3BucketExistV2Async(_amazonS3, _bucketName))
				{
					yield break;
				}

				ListObjectsV2Request request = new ListObjectsV2Request
				{
					BucketName = _bucketName
				};

				ListObjectsV2Response response;
				do
				{
					response = await _amazonS3.ListObjectsV2Async(request, cancellationToken);
					foreach (S3Object obj in response.S3Objects)
					{
						yield return (obj.Key, obj.LastModified);
					}

					request.ContinuationToken = response.NextContinuationToken;
				} while (response.IsTruncated);
			}
		}

		public async Task DeleteAsync(string path, CancellationToken cancellationToken = default)
		{
			await _amazonS3.DeleteObjectAsync(_bucketName, path, cancellationToken);
		}

		public ValueTask<Uri?> GetReadRedirectAsync(string path)
		{
			return new ValueTask<Uri?>(GetPresignedUrl(path, HttpVerb.GET));
		}

		public ValueTask<Uri?> GetWriteRedirectAsync(string path)
		{
			return new ValueTask<Uri?>(GetPresignedUrl(path, HttpVerb.PUT));
		}

		/// <summary>
		/// Helper method to generate a presigned URL for a request
		/// </summary>
		Uri? GetPresignedUrl(string path, HttpVerb verb, int? partNumber = null, string? uploadId = null)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan("s3.BuildPresignedUrl")
				.SetAttribute("Path", path)
			;

			try
			{
				GetPreSignedUrlRequest signedUrlRequest = new GetPreSignedUrlRequest();
				signedUrlRequest.BucketName = _bucketName;
				signedUrlRequest.Key = path;
				signedUrlRequest.Verb = verb;
				signedUrlRequest.Protocol = _settings.CurrentValue.AssumeHttpForRedirectUri ? Protocol.HTTP : Protocol.HTTPS;
				signedUrlRequest.Expires = DateTime.UtcNow.AddHours(3.0);
				if (partNumber.HasValue)
				{
					signedUrlRequest.PartNumber = partNumber.Value;
				}
				if (uploadId != null)
				{
					signedUrlRequest.UploadId = uploadId;
				}

				string url = _amazonS3.GetPreSignedURL(signedUrlRequest);

				return new Uri(url);
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Unable to get presigned url for {Path} from S3", path);
				return null;
			}
		}

		public async Task<BlobMetadata?> GetMetadataAsync(string path, CancellationToken cancellationToken = default)
		{
			try
			{
				GetObjectAttributesResponse? metadata = await _amazonS3.GetObjectAttributesAsync(new GetObjectAttributesRequest
				{
					BucketName = _bucketName, Key = path, ObjectAttributes = new List<ObjectAttributes>() { ObjectAttributes.ObjectSize }
				}, cancellationToken);
				if (metadata == null)
				{
					return null;
				}

				return new BlobMetadata(metadata.ObjectSize, metadata.LastModified);
			}
			catch (AmazonS3Exception e)
			{
				// if the object does not exist we get a not found status code
				if (e.StatusCode == HttpStatusCode.NotFound)
				{
					return null;
				}

				throw;
			}
			catch (XmlException)
			{
				// Multipart uploaded objects can cause invalid xml objects generated in S3
				return null;
			}
		}

		public async Task<string> StartMultipartUploadAsync(string path, CancellationToken cancellationToken = default)
		{
			InitiateMultipartUploadResponse response = await _amazonS3.InitiateMultipartUploadAsync(new InitiateMultipartUploadRequest {BucketName = _bucketName, Key = path}, cancellationToken);

			return response.UploadId;
		}

		public async Task CompleteMultipartUploadAsync(string path, string uploadId, List<string> partIds, CancellationToken cancellationToken = default)
		{
			Dictionary<int, PartETag> etags = new ();

			int? partNumberMarker = 0;
			for (int i = 0; i < 10; i++)
			{
				// s3 only allows you to list 1000 parts per request, but supports up to 10_000 parts so we need to potentially list this 10 times to get all parts
				ListPartsRequest listPartsRequest = new() {BucketName = _bucketName, Key = path, UploadId = uploadId, PartNumberMarker = partNumberMarker.ToString()};
				ListPartsResponse partList = await _amazonS3.ListPartsAsync(listPartsRequest, cancellationToken);
				foreach (PartDetail? part in partList.Parts)
				{
					etags.Add(part.PartNumber, new PartETag(part.PartNumber, part.ETag));
				}
				if (!partList.IsTruncated)
				{
					break;
				}
				partNumberMarker = partList.NextPartNumberMarker;
			}

			// check if any part is missing
			List<string> missingPartIds = new();
			foreach (string partId in partIds)
			{
				if (!etags.ContainsKey(int.Parse(partId)))
				{
					missingPartIds.Add(partId);
				}
			}

			if (missingPartIds.Count > 0)
			{
				throw new MissingMultipartPartsException(missingPartIds!);
			}

			await _amazonS3.CompleteMultipartUploadAsync(new CompleteMultipartUploadRequest {BucketName = _bucketName, Key = path, UploadId = uploadId, PartETags = etags.Values.ToList()}, cancellationToken);
		}

		public async Task PutMultipartPartAsync(string path, string uploadId, string partIdentifier, byte[] data, CancellationToken cancellationToken = default)
		{
			await _amazonS3.UploadPartAsync(new UploadPartRequest { BucketName = _bucketName, Key = path, UploadId = uploadId, PartNumber = int.Parse(partIdentifier), InputStream = new MemoryStream(data)}, cancellationToken);
		}

		public Uri? GetWriteRedirectForPart(string path, string uploadId, string partIdentifier)
		{
			Uri? uri = GetPresignedUrl(path, HttpVerb.PUT, partNumber: int.Parse(partIdentifier), uploadId: uploadId);

			if (uri != null)
			{
				return uri;
			}

			return null;
		}

		public async Task RenameMultipartBlobAsync(string blobName, string targetBlobName, CancellationToken cancellationToken = default)
		{
			await CopyBlobAsync(this, blobName, targetBlobName, cancellationToken);
			// delete the old multipart object
			await _amazonS3.DeleteObjectAsync(_bucketName, blobName, cancellationToken);
		}

		internal string GetBucketName()
		{
			return _bucketName;
		}

		public async Task<ulong> CopyBlobAsync(AmazonStorageBackend source, string sourcePath, string destinationPath, CancellationToken cancellationToken = default)
		{
			GetObjectMetadataResponse metadata = await source._amazonS3.GetObjectMetadataAsync(new GetObjectMetadataRequest { BucketName = source._bucketName, Key = sourcePath, PartNumber = 1 }, cancellationToken);

			if (string.Equals(source._bucketName, _bucketName, StringComparison.OrdinalIgnoreCase))
			{
				// copy within the bucket
				if (string.Equals(sourcePath, destinationPath, StringComparison.OrdinalIgnoreCase))
				{
					// copy to the same object, this already exists so no need to do any work
					return (ulong)metadata.ContentLength;
				}
			}

			if (!metadata.PartsCount.HasValue)
			{
				// not a multipart object
				CopyObjectResponse? response = await _amazonS3.CopyObjectAsync(new CopyObjectRequest()
				{
					SourceBucket = source._bucketName, DestinationBucket = _bucketName, SourceKey = sourcePath, DestinationKey = destinationPath,
				}, cancellationToken);

				return (ulong)response.ContentLength;
			}

			InitiateMultipartUploadResponse startMultipartUpload = await _amazonS3.InitiateMultipartUploadAsync(_bucketName, destinationPath, cancellationToken);
			string uploadId = startMultipartUpload.UploadId;
			ConcurrentBag<PartETag> parts = new();
			long chunkSize = metadata.ContentLength;
			long objectLength = long.Parse(metadata.ContentRange.Split('/')[1]);

			await Parallel.ForAsync(1, metadata.PartsCount.Value + 1, cancellationToken, async(int i, CancellationToken token) =>
			{
				long startOffset = (i - 1) * chunkSize;
				long endOffset = (i * chunkSize)-1; // last byte is exclusive

				endOffset = Math.Min(endOffset, objectLength-1);
				CopyPartResponse response = await _amazonS3.CopyPartAsync(new CopyPartRequest()
				{
					SourceBucket = source._bucketName,
					DestinationBucket = _bucketName,
					PartNumber = i,
					SourceKey = sourcePath,
					DestinationKey = destinationPath,
					UploadId = uploadId,
					FirstByte = startOffset,
					LastByte = endOffset, 
				}, token);
				parts.Add(new PartETag(i, response.ETag));
			});

			await _amazonS3.CompleteMultipartUploadAsync(new CompleteMultipartUploadRequest {BucketName = _bucketName, UploadId = uploadId, Key = destinationPath, PartETags = parts.ToList()}, cancellationToken);

			return (ulong)metadata.ContentLength;
		}
	}

	public static class BlobIdentifierExtensions
	{
		public static string AsS3Key(this BlobId blobIdentifier)
		{
			string s = blobIdentifier.ToString();
			string prefix = s.Substring(0, 4);
			return $"{prefix}/{s}";
		}
	}
}
