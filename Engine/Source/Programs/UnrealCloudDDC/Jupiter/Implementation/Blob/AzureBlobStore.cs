// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Azure;
using Azure.Storage.Blobs;
using Azure.Storage.Blobs.Models;
using Azure.Storage.Sas;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class AzureBlobStore : IBlobStore // TODO: Implement multipart store for Azure , IMultipartBlobStore
	{
		private readonly IOptionsMonitor<AzureSettings> _settings;
		private readonly ISecretResolver _secretResolver;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly IServiceProvider _provider;
		private readonly ConcurrentDictionary<NamespaceId, AzureStorageBackend> _backends = new ConcurrentDictionary<NamespaceId, AzureStorageBackend>();

		public AzureBlobStore(IOptionsMonitor<AzureSettings> settings, ISecretResolver secretResolver, INamespacePolicyResolver namespacePolicyResolver, IServiceProvider provider)
		{
			_settings = settings;
			_secretResolver = secretResolver;
			_namespacePolicyResolver = namespacePolicyResolver;
			_provider = provider;
		}

		private AzureStorageBackend GetBackend(NamespaceId ns)
		{
			return _backends.GetOrAdd(ns, x => ActivatorUtilities.CreateInstance<AzureStorageBackend>(_provider, GetConnectionString(ns), ns, GetContainerName(ns)));
		}

		private string GetContainerName(NamespaceId ns)
		{
			NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);

			string containerName = !string.IsNullOrEmpty(policy.StoragePool) ? $"jupiter-{policy.StoragePool}" : "jupiter";
			if (_settings.CurrentValue.StoragePoolContainerOverride.TryGetValue(policy.StoragePool, out string? overriddenContainerName))
			{
				containerName = overriddenContainerName;
			}

			return SanitizeContainerName(containerName);
		}

		private string GetConnectionString(NamespaceId ns)
		{
			NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
			string connectionString = _settings.CurrentValue.ConnectionString;
			if (_settings.CurrentValue.StoragePoolConnectionStrings.TryGetValue(policy.StoragePool, out string? connectionStringForPool))
			{
				connectionString = connectionStringForPool;
			}

			string resolvedConnectionString = _secretResolver.Resolve(connectionString);
			return resolvedConnectionString;
		}

		private static string GetPath(BlobId blobIdentifier) => blobIdentifier.ToString();

		public async Task<Uri?> GetObjectByRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			Uri? redirectUri = await GetBackend(ns).GetReadRedirectAsync(GetPath(identifier), cancellationToken);

			return redirectUri;
		}

		public async Task<BlobMetadata> GetObjectMetadataAsync(NamespaceId ns, BlobId blobId, CancellationToken cancellationToken = default)
		{
			BlobMetadata? metadata = await GetBackend(ns).GetMetadataAsync(GetPath(blobId), cancellationToken);

			if (metadata == null)
			{
				throw new BlobNotFoundException(ns, blobId);
			}

			return metadata;
		}

		public Task<ulong> CopyBlobAsync(NamespaceId ns, NamespaceId targetNamespace, BlobId blobId, CancellationToken cancellationToken = default)
		{
			// TODO: Implement support for this on azure
			throw new NotImplementedException();
		}

		public async Task<Uri?> PutObjectWithRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			Uri? redirectUri = await GetBackend(ns).GetWriteRedirectAsync(GetPath(identifier), cancellationToken);

			return redirectUri;
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, ReadOnlyMemory<byte> content, BlobId blobIdentifier, CancellationToken cancellationToken = default)
		{
			// TODO: this is not ideal as we copy the buffer, but there is no upload from memory available so we would need this copy anyway
			await using MemoryStream stream = new MemoryStream(content.ToArray());
			return await PutObjectAsync(ns, stream, blobIdentifier, cancellationToken);
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, Stream content, BlobId blobIdentifier, CancellationToken cancellationToken = default)
		{
			await GetBackend(ns).WriteAsync(GetPath(blobIdentifier), content, cancellationToken);
			return blobIdentifier;
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, byte[] content, BlobId blobIdentifier, CancellationToken cancellationToken = default)
		{
			await using MemoryStream stream = new MemoryStream(content);
			return await PutObjectAsync(ns, stream, blobIdentifier, cancellationToken);
		}

		public async Task<BlobContents> GetObjectAsync(NamespaceId ns, BlobId blobIdentifier, LastAccessTrackingFlags flags, bool supportsRedirectUri = false, CancellationToken cancellationToken = default)
		{
			NamespacePolicy policies = _namespacePolicyResolver.GetPoliciesForNs(ns);
			if (supportsRedirectUri && policies.AllowRedirectUris)
			{
				Uri? redirectUri = await GetBackend(ns).GetReadRedirectAsync(GetPath(blobIdentifier), cancellationToken);
				if (redirectUri != null)
				{
					return new BlobContents(redirectUri);
				}
			}

			BlobContents? contents = await GetBackend(ns).TryReadAsync(GetPath(blobIdentifier), flags, cancellationToken);
			if (contents == null)
			{
				throw new BlobNotFoundException(ns, blobIdentifier);
			}
			return contents;
		}

		public async Task<bool> ExistsAsync(NamespaceId ns, BlobId blobIdentifier, bool forceCheck, CancellationToken cancellationToken = default)
		{
			return await GetBackend(ns).ExistsAsync(GetPath(blobIdentifier), cancellationToken);
		}

		public async Task DeleteObjectAsync(NamespaceId ns, BlobId blobIdentifier, CancellationToken cancellationToken = default)
		{
			await GetBackend(ns).DeleteAsync(GetPath(blobIdentifier), cancellationToken);
		}

		public async Task DeleteObjectAsync(IEnumerable<NamespaceId> namespaces, BlobId blob, CancellationToken cancellationToken = default)
		{
			List<NamespaceId> namespaceIds = namespaces.ToList();
			List<string> storagePools = namespaceIds.Select(ns => _namespacePolicyResolver.GetPoliciesForNs(ns).StoragePool).Distinct().ToList();

			Dictionary<string, NamespaceId> storagePoolsToClean = storagePools.ToDictionary(storagePool => storagePool, storagePool => namespaceIds.FirstOrDefault(id => _namespacePolicyResolver.GetPoliciesForNs(id).StoragePool == storagePool));

			foreach ((string _, NamespaceId ns) in storagePoolsToClean)
			{
				await GetBackend(ns).DeleteAsync(GetPath(blob), cancellationToken);
			}
		}

		public async Task DeleteNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			string fixedNamespace = GetContainerName(ns);
			BlobContainerClient container = new BlobContainerClient(GetConnectionString(ns), fixedNamespace);
			if (await container.ExistsAsync(cancellationToken))
			{
				// we can only delete it if the container exists
				await container.DeleteAsync(cancellationToken: cancellationToken);
			}
		}

		public async IAsyncEnumerable<(BlobId, DateTime)> ListObjectsAsync(NamespaceId ns, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			await foreach ((string path, DateTime time) in GetBackend(ns).ListAsync(cancellationToken))
			{
				yield return (new BlobId(path), time);
			}
		}

		private static string SanitizeContainerName(string containerName)
		{
			return containerName.Replace(".", "-", StringComparison.OrdinalIgnoreCase).Replace("_", "-", StringComparison.OrdinalIgnoreCase).ToLower();
		}
	}

	public class AzureStorageBackend : IStorageBackend
	{
		private readonly NamespaceId _namespaceId;
		private readonly ILogger<AzureStorageBackend> _logger;
		private readonly Tracer _tracer;
		private readonly BlobContainerClient _blobContainer;

		private const string LastTouchedKey = "Io_LastTouched";
		private const string NamespaceKey = "Io_Namespace";

		public AzureStorageBackend(string connectionString, NamespaceId namespaceId, string containerName, ILogger<AzureStorageBackend> logger, Tracer tracer)
		{
			_namespaceId = namespaceId;
			_logger = logger;
			_tracer = tracer;
			_blobContainer = new BlobContainerClient(connectionString, containerName);
		}

		public async Task WriteAsync(string path, Stream content, CancellationToken cancellationToken = default)
		{
			Dictionary<string, string> metadata = new Dictionary<string, string> { { NamespaceKey, _namespaceId.ToString() } };
			await _blobContainer.CreateIfNotExistsAsync(metadata: metadata, cancellationToken: cancellationToken);

			_logger.LogDebug("Fetching blob reference with name {ObjectName}", path);
			try
			{

				await _blobContainer.GetBlobClient(path).UploadAsync(content, cancellationToken);
			}
			catch (RequestFailedException e)
			{
				if (e.Status == 409)
				{
					// the object already existed, that is fine, no need to do anything
					return;
				}

				throw;
			}
			finally
			{
				// we touch the blob so that the last access time is always refreshed even if we didnt actually mutate it to make sure the gc knows this is a active blob
				// see delete operation in Leda blob store cleanup
				await TouchBlobAsync(_blobContainer.GetBlobClient(path), cancellationToken);
				_logger.LogDebug("Upload of blob {ObjectName} completed", path);
			}
		}

		private static async Task TouchBlobAsync(BlobClient blob, CancellationToken cancellationToken = default)
		{
			Dictionary<string, string> metadata = new Dictionary<string, string>
			{
				{
					LastTouchedKey, DateTime.Now.ToString(CultureInfo.InvariantCulture)
				}
			};
			// we update the metadata, we don''t really care about the field being specified as we just want to touch the blob to update its last modified date property which we can not set
			await blob.SetMetadataAsync(metadata, cancellationToken: cancellationToken);
		}

		public async Task<BlobContents?> TryReadAsync(string path, LastAccessTrackingFlags flags, CancellationToken cancellationToken = default)
		{

			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				throw new InvalidOperationException($"Container {_blobContainer.Name} did not exist");
			}

			try
			{
				BlobClient blob = _blobContainer.GetBlobClient(path);
				Response<BlobDownloadInfo> blobInfo = await blob.DownloadAsync(cancellationToken);
				return new BlobContents(blobInfo.Value.Content, blobInfo.Value.ContentLength);
			}
			catch (RequestFailedException e)
			{
				if (e.Status == 404)
				{
					return null;
				}

				throw;
			}
		}

		public async Task<bool> ExistsAsync(string path, CancellationToken cancellationToken = default)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				return false;
			}

			BlobClient blob = _blobContainer.GetBlobClient(path);
			return await blob.ExistsAsync(cancellationToken);
		}

		public async Task DeleteAsync(string path, CancellationToken cancellationToken = default)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				throw new InvalidOperationException($"Container {_blobContainer.Name} did not exist");
			}

			await _blobContainer.DeleteBlobAsync(path, cancellationToken: cancellationToken);
		}

		public async IAsyncEnumerable<(string, DateTime)> ListAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			bool exists = await _blobContainer.ExistsAsync(cancellationToken);
			if (!exists)
			{
				yield break;
			}

			await foreach (BlobItem? item in _blobContainer.GetBlobsAsync(BlobTraits.Metadata, cancellationToken: cancellationToken))
			{
				yield return (item.Name, item.Properties?.LastModified?.DateTime ?? DateTime.Now);
			}
		}

		public async ValueTask<Uri?> GetReadRedirectAsync(string path, CancellationToken cancellationToken = default)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				throw new InvalidOperationException($"Container {_blobContainer.Name} did not exist");
			}

			return GetPresignedUrl(path, BlobSasPermissions.Read);
		}

		public async ValueTask<Uri?> GetWriteRedirectAsync(string path, CancellationToken cancellationToken = default)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				throw new InvalidOperationException($"Container {_blobContainer.Name} did not exist");
			}

			return GetPresignedUrl(path, BlobSasPermissions.Write);
		}

		/// <summary>
		/// Helper method to generate a presigned URL for a request
		/// </summary>
		Uri? GetPresignedUrl(string path, BlobSasPermissions permissions)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan("azure.BuildPresignedUrl")
					.SetAttribute("Path", path)
				;

			try
			{
				BlobClient blob = _blobContainer.GetBlobClient(path);
				return blob.GenerateSasUri(permissions, DateTimeOffset.Now.AddHours(1.0));
			}
			catch (RequestFailedException e)
			{
				if (e.Status == 404)
				{
					return null;
				}

				throw;
			}
		}

		public async Task<BlobMetadata?> GetMetadataAsync(string path, CancellationToken cancellationToken = default)
		{
			if (!await _blobContainer.ExistsAsync(cancellationToken))
			{
				throw new InvalidOperationException($"Container {_blobContainer.Name} did not exist");
			}

			BlobClient blob = _blobContainer.GetBlobClient(path);
			Response<BlobProperties>? response = await blob.GetPropertiesAsync(cancellationToken: cancellationToken);
			if (response == null)
			{
				return null;
			}

			return new BlobMetadata(response.Value.ContentLength, response.Value.CreatedOn.DateTime);
		}
	}
}
