// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Implementation.Blob;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class BlobIndexConsistencyState
	{
	}

	// ReSharper disable once ClassNeverInstantiated.Global
	public class BlobIndexConsistencyCheckService : PollingService<BlobIndexConsistencyState>
	{
		private readonly IOptionsMonitor<ConsistencyCheckSettings> _settings;
		private readonly IOptionsMonitor<JupiterSettings> _jupiterSettings;
		private readonly ILeaderElection _leaderElection;
		private readonly IBlobIndex _blobIndex;
		private readonly IBlobService _blobService;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;

		protected override bool ShouldStartPolling()
		{
			return _settings.CurrentValue.EnableBlobIndexChecks || _settings.CurrentValue.EnableBlobReferenceChecks;
		}

		public BlobIndexConsistencyCheckService(IOptionsMonitor<ConsistencyCheckSettings> settings, IOptionsMonitor<JupiterSettings> jupiterSettings, ILeaderElection leaderElection, IBlobIndex blobIndex, IBlobService blobService, Tracer tracer, ILogger<BlobIndexConsistencyCheckService> logger) : base(serviceName: nameof(BlobIndexConsistencyCheckService), TimeSpan.FromSeconds(settings.CurrentValue.ConsistencyCheckPollFrequencySeconds), new BlobIndexConsistencyState(), logger)
		{
			_settings = settings;
			_jupiterSettings = jupiterSettings;
			_leaderElection = leaderElection;
			_blobIndex = blobIndex;
			_blobService = blobService;
			_tracer = tracer;
			_logger = logger;
		}

		public override async Task<bool> OnPollAsync(BlobIndexConsistencyState state, CancellationToken cancellationToken)
		{
			bool shouldEnable = _settings.CurrentValue.EnableBlobIndexChecks || _settings.CurrentValue.EnableBlobReferenceChecks;
			if (!shouldEnable)
			{
				_logger.LogInformation("Skipped running blob index consistency check as it is disabled");
				return false;
			}

			if (!_leaderElection.IsThisInstanceLeader())
			{
				_logger.LogInformation("Skipped running blob index consistency check because this instance was not the leader");
				return false;
			}

			if (_settings.CurrentValue.EnableBlobReferenceChecks)
			{
				await RunConsistencyCheckBlobReferencesAsync(cancellationToken);
			}
			if (_settings.CurrentValue.EnableBlobIndexChecks)
			{
				await RunConsistencyCheckAsync(cancellationToken);
			}

			return true;
		}

		public async Task RunConsistencyCheckBlobReferencesAsync(CancellationToken cancellationToken)
		{
			ulong countOfBlobsChecked = 0;
			ulong countOfIncorrectBlobsFound = 0;

			_logger.LogInformation("Blob References Consistency check started.");
			await Parallel.ForEachAsync(_blobIndex.GetAllBlobReferencesAsync(cancellationToken), new ParallelOptions
			{
				CancellationToken = cancellationToken,
				MaxDegreeOfParallelism = _settings.CurrentValue.BlobIndexMaxParallelOperations,
			},
				async (tuple, token) =>
				{
					(NamespaceId ns, BaseBlobReference blobReference) = tuple;
					BlobId blobIdentifier = blobReference.SourceBlob;
					Interlocked.Increment(ref countOfBlobsChecked);

					if (countOfBlobsChecked % 100 == 0)
					{
						_logger.LogInformation("Consistency check running on blob references, count of blobs processed so far: {CountOfBlobs}", countOfBlobsChecked);
					}

					using TelemetrySpan scope = _tracer.StartActiveSpan("consistency_check.blob_references").SetAttribute("resource.name", $"{ns}.{blobIdentifier}").SetAttribute("operation.name", "consistency_check.blob_references");

					try
					{
						bool exists = true;
						try
						{
							List<string> regions = await _blobIndex.GetBlobRegionsAsync(ns, blobIdentifier, cancellationToken);
							exists = regions.Count != 0;
						}
						catch (BlobNotFoundException)
						{
							exists = false;
						}

						if (!exists)
						{
							Interlocked.Increment(ref countOfIncorrectBlobsFound);
							_logger.LogWarning("Blob {Blob} in namespace {Namespace} had entries in the ref table but does not exist. Cleaning up.", blobIdentifier, ns);
							await _blobIndex.RemoveReferencesAsync(ns, blobIdentifier, null, cancellationToken);
						}
					}
					catch (Exception e)
					{
						_logger.LogWarning(e, "Unknown exception {Exception} when checking reference consistency for blob {Blob} in namespace {Namespace}", e, blobIdentifier, ns);
					}
				}
			);

			_logger.LogInformation("Blob References Consistency check finished, found {CountOfIncorrectBlobs} incorrect blobs. Processed {CountOfBlobs} blobs.", countOfIncorrectBlobsFound, countOfBlobsChecked);
		}

		private async Task RunConsistencyCheckAsync(CancellationToken cancellationToken)
		{
			ulong countOfBlobsChecked = 0;
			ulong countOfIncorrectBlobsFound = 0;
			string currentRegion = _jupiterSettings.CurrentValue.CurrentSite;
			await Parallel.ForEachAsync(_blobIndex.GetAllBlobsAsync(cancellationToken), new ParallelOptions
			{
				CancellationToken = cancellationToken,
				MaxDegreeOfParallelism = _settings.CurrentValue.BlobIndexMaxParallelOperations,
			},
				async (tuple, token) =>
				{
					(NamespaceId ns, BlobId blobIdentifier) = tuple;
					Interlocked.Increment(ref countOfBlobsChecked);

					if (countOfBlobsChecked % 100 == 0)
					{
						_logger.LogInformation("Consistency check running on blob index, count of blobs processed so far: {CountOfBlobs}", countOfBlobsChecked);
					}

					using TelemetrySpan scope = _tracer.StartActiveSpan("consistency_check.blob_index").SetAttribute("resource.name", $"{ns}.{blobIdentifier}").SetAttribute("operation.name", "consistency_check.blob_index");

					bool issueFound = false;
					bool deleted = false;
					List<string> regions;
					try
					{
						regions = await _blobIndex.GetBlobRegionsAsync(ns, blobIdentifier, cancellationToken);
					}
					catch (BlobNotFoundException)
					{
						regions = new List<string>();
					}
					try
					{
						if (regions.Contains(currentRegion))
						{
							if (!await _blobService.ExistsInRootStoreAsync(ns, blobIdentifier, token))
							{
								Interlocked.Increment(ref countOfIncorrectBlobsFound);
								issueFound = true;

								if (regions.Count > 1)
								{
									_logger.LogWarning("Blob {Blob} in namespace {Namespace} did not exist in root store but is tracked as doing so in the blob index. Attempting to replicate it.", blobIdentifier, ns);

									try
									{
										BlobContents _ = await _blobService.ReplicateObjectAsync(ns, blobIdentifier, force: true, bucketHint: null, cancellationToken: token);
									}
									catch (BlobReplicationException e)
									{
										// we update the blob index to accurately reflect that we do not have the blob, this is not good though as it means a upload that we thought happened now lacks content
										if (_settings.CurrentValue.AllowDeletesInBlobIndex)
										{
											_logger.LogWarning("Updating blob index to remove Blob {Blob} in namespace {Namespace} as we failed to repair it.", blobIdentifier, ns);
											await _blobIndex.RemoveBlobFromRegionAsync(ns, blobIdentifier, cancellationToken: token);
											deleted = true;
										}
										else
										{
											_logger.LogError(e, "Failed to replicate Blob {Blob} in namespace {Namespace}. Unable to repair the blob index", blobIdentifier, ns);
										}
									}
									catch (BlobNotFoundException)
									{
										// the blob does not exist locally, nothing to do, it likely got deleted by the time we started the glob of blobs and us processing it
									}
								}
								else
								{
									// if the blob only exists in the current region there is no point in attempting to replicate it
									_logger.LogWarning("Blob {Blob} in namespace {Namespace} did not exist in root store but is tracked as doing so in the blob index. Does not exist anywhere else so unable to replicate it.", blobIdentifier, ns);

									if (_settings.CurrentValue.AllowDeletesInBlobIndex)
									{
										// this blob can not be repaired so we just delete it from the blob index
										_logger.LogWarning("Blob {Blob} in namespace {Namespace} can not be repaired so removing existence from current region.", blobIdentifier, ns);

										await _blobIndex.RemoveBlobFromRegionAsync(ns, blobIdentifier, cancellationToken: token);
										deleted = true;
									}
								}
							}
						}
					}
					catch (NamespaceNotFoundException)
					{
						if (_settings.CurrentValue.AllowDeletesInBlobIndex)
						{
							_logger.LogWarning("Blob {Blob} in namespace {Namespace} is of a unknown namespace, removing.", blobIdentifier, ns);

							// for entries that are of a unknown namespace we simply remove them
							await _blobIndex.RemoveBlobFromRegionAsync(ns, blobIdentifier, cancellationToken: cancellationToken);
							deleted = true;
						}
					}
					catch (Exception e)
					{
						_logger.LogError(e, "Exception when doing blob index consistency check for {Blob} in namespace {Namespace}", blobIdentifier, ns);
						scope.RecordException(e);
						scope.SetStatus(Status.Error);
					}

					scope.SetAttribute("issueFound", issueFound.ToString());
					scope.SetAttribute("deleted", deleted.ToString());
				}
			);

			_logger.LogInformation("Blob Index Consistency check finished, found {CountOfIncorrectBlobs} incorrect blobs. Processed {CountOfBlobs} blobs.", countOfIncorrectBlobsFound, countOfBlobsChecked);
		}

		protected override Task OnStopping(BlobIndexConsistencyState state)
		{
			return Task.CompletedTask;
		}
	}
}
