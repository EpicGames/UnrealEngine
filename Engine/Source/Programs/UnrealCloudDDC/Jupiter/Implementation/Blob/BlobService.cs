// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics.Metrics;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Security.Claims;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Serialization;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Controllers;
using Jupiter.Implementation.Blob;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation;

public class BlobService : IBlobService
{
	private List<IBlobStore> _blobStores;
	private readonly IServiceProvider _provider;
	private readonly IOptionsMonitor<UnrealCloudDDCSettings> _settings;
	private readonly IBlobIndex _blobIndex;
	private readonly IReplicationLog _replicationLog;
	private readonly IPeerStatusService _peerStatusService;
	private readonly IHttpClientFactory _httpClientFactory;
	private readonly IServiceCredentials _serviceCredentials;
	private readonly INamespacePolicyResolver _namespacePolicyResolver;
	private readonly IHttpContextAccessor _httpContextAccessor;
	private readonly IRequestHelper? _requestHelper;
	private readonly Tracer _tracer;
	private readonly BufferedPayloadFactory _bufferedPayloadFactory;
	private readonly ILogger _logger;
	private readonly Counter<long>? _storeGetHitsCounter;
	private readonly Counter<long>? _storeGetAttemptsCounter;
	private readonly string _currentSite;

	internal IEnumerable<IBlobStore> BlobStore
	{
		get => _blobStores;
		set => _blobStores = value.ToList();
	}

	public BlobService(IServiceProvider provider, IOptionsMonitor<UnrealCloudDDCSettings> settings, IOptionsMonitor<JupiterSettings> jupiterSettings, IBlobIndex blobIndex, IReplicationLog replicationLog, IPeerStatusService peerStatusService, IHttpClientFactory httpClientFactory, IServiceCredentials serviceCredentials, INamespacePolicyResolver namespacePolicyResolver, IHttpContextAccessor httpContextAccessor, IRequestHelper? requestHelper, Tracer tracer, BufferedPayloadFactory bufferedPayloadFactory, ILogger<BlobService> logger, Meter? meter)
	{
		_blobStores = GetBlobStores(provider, settings).ToList();
		_provider = provider;
		_settings = settings;
		_blobIndex = blobIndex;
		_replicationLog = replicationLog;
		_peerStatusService = peerStatusService;
		_httpClientFactory = httpClientFactory;
		_serviceCredentials = serviceCredentials;
		_namespacePolicyResolver = namespacePolicyResolver;
		_httpContextAccessor = httpContextAccessor;
		_requestHelper = requestHelper;
		_tracer = tracer;
		_bufferedPayloadFactory = bufferedPayloadFactory;
		_logger = logger;

		_currentSite = jupiterSettings.CurrentValue.CurrentSite;

		_storeGetHitsCounter = meter?.CreateCounter<long>("store.blob_get.found");
		_storeGetAttemptsCounter = meter?.CreateCounter<long>("store.blob_get.attempt");
	}

	public static IEnumerable<IBlobStore> GetBlobStores(IServiceProvider provider, IOptionsMonitor<UnrealCloudDDCSettings> settings)
	{
		return settings.CurrentValue.GetStorageImplementations().Select(impl => ToStorageImplementation(provider, impl));
	}

	private static IBlobStore ToStorageImplementation(IServiceProvider provider, UnrealCloudDDCSettings.StorageBackendImplementations impl)
	{
		IBlobStore? store = impl switch
		{
			UnrealCloudDDCSettings.StorageBackendImplementations.S3 => provider.GetService<AmazonS3Store>(),
			UnrealCloudDDCSettings.StorageBackendImplementations.Azure => provider.GetService<AzureBlobStore>(),
			UnrealCloudDDCSettings.StorageBackendImplementations.FileSystem => provider.GetService<FileSystemStore>(),
			UnrealCloudDDCSettings.StorageBackendImplementations.Memory => provider.GetService<MemoryBlobStore>(),
			UnrealCloudDDCSettings.StorageBackendImplementations.Relay => provider.GetService<RelayBlobStore>(),
			UnrealCloudDDCSettings.StorageBackendImplementations.Peer => provider.GetService<PeerBlobStore>(),
			_ => throw new NotImplementedException("Unknown blob store {store")
		};
		if (store == null)
		{
			throw new ArgumentException($"Failed to find a provider service for type: {impl}");
		}

		return store;
	}

	public async Task<ContentHash> VerifyContentMatchesHashAsync(Stream content, ContentHash identifier, CancellationToken cancellationToken = default)
	{
		ContentHash blobHash;
		{
			using TelemetrySpan _ = _tracer.StartActiveSpan("web.hash").SetAttribute("operation.name", "web.hash");
			blobHash = await BlobId.FromStreamAsync(content, cancellationToken);
		}

		if (!identifier.Equals(blobHash))
		{
			throw new HashMismatchException(identifier, blobHash);
		}

		return identifier;
	}

	public async Task<BlobId> PutObjectAsync(NamespaceId ns, IBufferedPayload payload, BlobId identifier, BucketId? bucketHint = null, bool? bypassCache = null, CancellationToken cancellationToken = default)
	{
		bool useContentAddressedStorage = _namespacePolicyResolver.GetPoliciesForNs(ns).UseContentAddressedStorage;
		using TelemetrySpan scope = _tracer.StartActiveSpan("put_blob")
			.SetAttribute("operation.name", "put_blob")
			.SetAttribute("resource.name", identifier.ToString())
			.SetAttribute("Content-Length", payload.Length.ToString());

		await using Stream hashStream = payload.GetStream();
		BlobId id = useContentAddressedStorage ? BlobId.FromContentHash(await VerifyContentMatchesHashAsync(hashStream, identifier, cancellationToken)) : identifier;

		BlobId objectStoreIdentifier = await PutObjectToStoresAsync(ns, payload, id, bucketHint,bypassCache, cancellationToken);
		await _blobIndex.AddBlobToIndexAsync(ns, id, (ulong)payload.Length, cancellationToken: cancellationToken);

		return objectStoreIdentifier;

	}

	public async Task<BlobId> PutObjectAsync(NamespaceId ns, byte[] payload, BlobId identifier, BucketId? bucketHint = null, bool? bypassCache = null, CancellationToken cancellationToken = default)
	{
		bool useContentAddressedStorage = _namespacePolicyResolver.GetPoliciesForNs(ns).UseContentAddressedStorage;
		using TelemetrySpan scope = _tracer.StartActiveSpan("put_blob")
			.SetAttribute("operation.name", "put_blob")
			.SetAttribute("resource.name", identifier.ToString())
			.SetAttribute("Content-Length", payload.Length.ToString())
			;

		await using Stream hashStream = new MemoryStream(payload);
		BlobId id = useContentAddressedStorage ? BlobId.FromContentHash(await VerifyContentMatchesHashAsync(hashStream, identifier, cancellationToken)) : identifier;

		BlobId objectStoreIdentifier = await PutObjectToStoresAsync(ns, payload, id, bucketHint: bucketHint, bypassCache, cancellationToken: cancellationToken);
		await _blobIndex.AddBlobToIndexAsync(ns, id, (ulong)payload.Length, cancellationToken: cancellationToken);

		return objectStoreIdentifier;
	}

	public async Task<Uri?> MaybePutObjectWithRedirectAsync(NamespaceId ns, BlobId identifier, BucketId? bucketHint = null, CancellationToken cancellationToken = default)
	{
		bool allowRedirectUris = _namespacePolicyResolver.GetPoliciesForNs(ns).AllowRedirectUris;
		if (!allowRedirectUris)
		{
			return null;
		}

		IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();

		// we only attempt to upload to the last store
		// assuming that any other store will pull from it when they lack content once the upload has finished
		IBlobStore store = _blobStores.Last();
		{
			string storeName = store.GetType().Name;
			using TelemetrySpan scope = _tracer.StartActiveSpan("put_blob_with_redirect")
					.SetAttribute("operation.name", "put_blob_with_redirect")
					.SetAttribute("resource.name", identifier.ToString())
					.SetAttribute("store", storeName)
			;

			using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope($"blob.put-redirect.{storeName}", $"PUT(redirect) to store: '{storeName}'");

			Uri? redirectUri = await store.PutObjectWithRedirectAsync(ns, identifier, cancellationToken);
			if (redirectUri != null)
			{
				// assumes that the blob is inserted successfully so we add this event now
				await _replicationLog.InsertAddBlobEventAsync(ns, identifier, DateTime.UtcNow.ToReplicationBucket(), bucketHint);

				return redirectUri;
			}
		}

		// no store found that supports redirect
		return null;
	}

	public async Task<BlobId> PutObjectKnownHashAsync(NamespaceId ns, IBufferedPayload content, BlobId identifier, BucketId? bucketHint, bool? bypassCache = null, CancellationToken cancellationToken = default)
	{
		using TelemetrySpan scope = _tracer.StartActiveSpan("put_blob")
			.SetAttribute("operation.name", "put_blob")
			.SetAttribute("resource.name", identifier.ToString())
			.SetAttribute("Content-Length", content.Length.ToString())
			;

		BlobId objectStoreIdentifier = await PutObjectToStoresAsync(ns, content, identifier, bucketHint, bypassCache, cancellationToken);
		await _blobIndex.AddBlobToIndexAsync(ns, identifier, (ulong)content.Length, cancellationToken: cancellationToken);

		return objectStoreIdentifier;
	}

	private async Task<BlobId> PutObjectToStoresAsync(NamespaceId ns, IBufferedPayload bufferedPayload, BlobId identifier, BucketId? bucketHint, bool? bypassCache, CancellationToken cancellationToken = default)
	{
		IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();

		foreach (IBlobStore store in _blobStores)
		{
			// store is considered a cache layer if its filesystem or memory and it is not the last store in the list
			bool isCacheLayer = store is FileSystemStore or MemoryBlobStore && _blobStores.Last() != store;
			if (bypassCache.HasValue && bypassCache.Value && isCacheLayer)
			{
				continue;
			}
			using TelemetrySpan scope = _tracer.StartActiveSpan("put_blob_to_store")
				.SetAttribute("operation.name", "put_blob_to_store")
				.SetAttribute("resource.name", identifier.ToString())
				.SetAttribute("store", store.GetType().ToString())
				;
			string storeName = store.GetType().Name;

			using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope($"blob.put.{storeName}", $"PUT to store: '{storeName}'");

			await using Stream s = bufferedPayload.GetStream();
			await store.PutObjectAsync(ns, s, identifier, cancellationToken);
		}
		Task insertToReplicationLogTask = _replicationLog.InsertAddBlobEventAsync(ns, identifier, DateTime.UtcNow.ToReplicationBucket(), bucketHint: bucketHint);

		NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
		if (policy.PopulateFallbackNamespaceOnUpload && policy.FallbackNamespace.HasValue)
		{
			await PutObjectToStoresAsync(policy.FallbackNamespace.Value, bufferedPayload, identifier, bucketHint, bypassCache, cancellationToken);
			await _blobIndex.AddBlobToIndexAsync(policy.FallbackNamespace.Value, identifier, (ulong)bufferedPayload.Length, cancellationToken: cancellationToken);
		}

		await insertToReplicationLogTask;
		return identifier;
	}

	private async Task<BlobId> PutObjectToStoresAsync(NamespaceId ns, byte[] payload, BlobId identifier, BucketId? bucketHint = null, bool? bypassCache = null, CancellationToken cancellationToken = default)
	{
		using MemoryStream ms = new MemoryStream(payload);
		using IBufferedPayload bufferedPayload = await _bufferedPayloadFactory.CreateFromStreamAsync(ms, ms.Length, "put-object-store", cancellationToken);
		return await PutObjectToStoresAsync(ns, bufferedPayload, identifier, bucketHint, bypassCache, cancellationToken);
	}

	public async Task<BlobContents> GetObjectAsync(NamespaceId ns, BlobId blob, List<string>? storageLayers = null, bool supportsRedirectUri = false, bool allowOndemandReplication = true, BucketId? bucketHint = null, bool bypassCache = false, CancellationToken cancellationToken = default)
	{
		// when bypassing cache we only use the last layer in the storage hierarchy
		if (bypassCache)
		{
			storageLayers = new List<string> { _blobStores.Select(store => store.GetType().Name).Last() };
		}

		try
		{
			return await GetObjectFromStoresAsync(ns, blob, storageLayers, supportsRedirectUri, cancellationToken);
		}
		catch (BlobNotFoundException)
		{
			NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);

			if (policy.FallbackNamespace != null)
			{
				ClaimsPrincipal? user = _httpContextAccessor.HttpContext?.User;
				HttpRequest? request = _httpContextAccessor.HttpContext?.Request;

				if (user == null || request == null)
				{
					_logger.LogWarning("Unable to fallback to namespace {Namespace} due to not finding required http context values", policy.FallbackNamespace.Value);
					throw;
				}

				if (_requestHelper == null)
				{
					_logger.LogWarning("Unable to fallback to namespace {Namespace} due to request helper missing", policy.FallbackNamespace.Value);
					throw;
				}

				AccessScope accessScope = bucketHint == null ? new AccessScope(policy.FallbackNamespace) : new AccessScope(policy.FallbackNamespace, bucketHint.Value); 
				ActionResult? result = await _requestHelper.HasAccessToScopeAsync(user, request, accessScope, new[] { JupiterAclAction.ReadObject });
				if (result != null)
				{
					_logger.LogInformation("Authorization error when attempting to fallback to namespace {FallbackNamespace}. This may be confusing for users that as they had access to original namespace {Namespace}", policy.FallbackNamespace.Value, ns);
					throw new AuthorizationException(result, "Failed to authenticate for fallback namespace");
				}

				try
				{
					IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();

					// read the content from the fallback namespace
					BlobContents fallbackContent = await GetObjectFromStoresAsync(policy.FallbackNamespace.Value, blob, storageLayers, cancellationToken: cancellationToken);

					// populate the primary namespace with the content
					using TelemetrySpan _ = _tracer.StartActiveSpan("HierarchicalStore.Populate").SetAttribute("operation.name", "HierarchicalStore.Populate").SetAttribute("resource.name", blob.ToString());
					using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope($"blob.populate", "Populating caches with blob contents");

					await using MemoryStream tempStream = new MemoryStream();
					await fallbackContent.Stream.CopyToAsync(tempStream, cancellationToken);
					byte[] data = tempStream.ToArray();

					await PutObjectAsync(ns, data, blob, bucketHint: null, bypassCache:false, cancellationToken);
					return await GetObjectAsync(ns, blob, storageLayers, supportsRedirectUri, allowOndemandReplication, bucketHint, bypassCache, cancellationToken);
				}
				catch (BlobNotFoundException)
				{
					// if we fail to find the blob in the fallback namespace we can carry on as we will rethrow the blob not found exception later (if the replication also fails)
				}
			}

			if (ShouldFetchBlobOnDemand(ns) && allowOndemandReplication)
			{
				try
				{
					return await ReplicateObjectAsync(ns, blob, supportsRedirectUri: supportsRedirectUri, bucketHint: bucketHint, cancellationToken: cancellationToken);
				}
				catch (BlobNotFoundException)
				{
					// if the blob is not found we can ignore it as we will just rethrow it later anyway
				}
			}

			// if the primary namespace failed check to see if we should use a fallback policy which has replication enabled
			if (policy.FallbackNamespace != null && ShouldFetchBlobOnDemand(policy.FallbackNamespace.Value) && allowOndemandReplication)
			{
				try
				{
					return await ReplicateObjectAsync(policy.FallbackNamespace.Value, blob, supportsRedirectUri: supportsRedirectUri, bucketHint: bucketHint, cancellationToken: cancellationToken);
				}
				catch (BlobNotFoundException)
				{
					// if the blob is not found we can ignore it as we will just rethrow it later anyway
				}
			}

			// we might have attempted to fetch the object and failed, or had no fallback options, either way the blob can not be found so we should rethrow
			throw;
		}
	}

	private async Task<BlobContents> GetObjectFromStoresAsync(NamespaceId ns, BlobId blob, List<string>? storageLayers = null, bool supportsRedirectUri = false, CancellationToken cancellationToken = default)
	{
		bool seenBlobNotFound = false;
		bool seenNamespaceNotFound = false;
		int numStoreMisses = 0;
		BlobContents? blobContents = null;
		IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();

		foreach (IBlobStore store in _blobStores)
		{
			string blobStoreName = store.GetType().Name;

			// check which storage layers to skip if we have a explicit list of storage layers to use
			if (storageLayers != null && storageLayers.Count != 0)
			{
				bool found = false;
				foreach (string storageLayer in storageLayers)
				{
					if (string.Equals(storageLayer, blobStoreName, StringComparison.OrdinalIgnoreCase))
					{
						found = true;
						break;
					}
				}

				if (!found)
				{
					continue;
				}
			}

			using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.GetObject")
				.SetAttribute("operation.name", "HierarchicalStore.GetObject")
				.SetAttribute("resource.name", blob.ToString())
				.SetAttribute("BlobStore", store.GetType().ToString())
				.SetAttribute("ObjectFound", false.ToString())
				;

			string storeName = store.GetType().Name;
			using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope($"blob.get.{storeName}", $"Blob GET from: '{storeName}'");
			_storeGetAttemptsCounter?.Add(1, new KeyValuePair<string, object?>("store", storeName));
			try
			{
				blobContents = await store.GetObjectAsync(ns, blob, supportsRedirectUri: supportsRedirectUri, cancellationToken: cancellationToken);
				scope.SetAttribute("ObjectFound", true.ToString());
				_storeGetHitsCounter?.Add(1, new KeyValuePair<string, object?>("store", storeName));
				break;
			}
			catch (BlobNotFoundException)
			{
				seenBlobNotFound = true;
				numStoreMisses++;
			}
			catch (NamespaceNotFoundException)
			{
				seenNamespaceNotFound = true;
			}
		}

		if (seenBlobNotFound && blobContents == null)
		{
			throw new BlobNotFoundException(ns, blob);
		}

		if (seenNamespaceNotFound && blobContents == null)
		{
			throw new NamespaceNotFoundException(ns);
		}

		// if we applied filters to the storage layers resulting in no blob found we consider it a miss
		if (storageLayers != null && storageLayers.Count != 0 && blobContents == null)
		{
			throw new BlobNotFoundException(ns, blob);
		}

		if (blobContents == null)
		{
			// Should not happen but exists to safeguard against the null pointer
			throw new Exception("blobContents is null");
		}

		// if we had a miss populate the earlier stores unless we are using a redirect uri, at which point we do not have the contents available to us
		if (numStoreMisses >= 1 && blobContents.RedirectUri == null)
		{
			using TelemetrySpan _ = _tracer.StartActiveSpan("HierarchicalStore.Populate").SetAttribute("operation.name", "HierarchicalStore.Populate");
			using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope($"blob.populate", "Populating caches with blob contents");

			{
				if (numStoreMisses == 1 && blobContents.BufferedPayload is FilesystemBufferedPayload payload && _blobStores[0] is FileSystemStore fileSystemStore)
				{
					IBlobStore blobStore = _blobStores[0];
					using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.PopulateStore")
						.SetAttribute("operation.name", "HierarchicalStore.PopulateStore")
						.SetAttribute("Mode", "MoveBufferedPayload")
						.SetAttribute("BlobStore", blobStore.GetType().Name);

					// blob is buffered on filesystem and needs to be written into the filesystem cache only this allows us to move the file instead of copying
					await fileSystemStore.MoveBufferedPayloadAsync(ns, blob, payload, cancellationToken: cancellationToken);

					return await blobStore.GetObjectAsync(ns, blob, cancellationToken: cancellationToken);
				}
			}

			// {
			// 	if (numStoreMisses == 1 && _blobStores[0] is FileSystemStore fileSystemStore)
			// 	{
			// 		// if we are only missing data from the filesystem store, buffer to it and then send using the data in the filesystem so that we can use sendfile for this data
			// 		using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.PopulateStore")
			// 			.SetAttribute("operation.name", "HierarchicalStore.PopulateStore")
			// 			.SetAttribute("Mode", "UseFilesystemPayload")
			// 			.SetAttribute("BlobStore", nameof(FileSystemStore));
			//
			// 		await using Stream s = blobContents.Stream;
			// 		await fileSystemStore.PutObjectAsync(ns, s, blob, cancellationToken);
			//
			// 		return await fileSystemStore.GetObjectAsync(ns, blob, flags: LastAccessTrackingFlags.DoTracking, cancellationToken: cancellationToken);
			// 	}
			// }
			
			bool streamCanSeek = blobContents.Stream.CanSeek;
			
			// no using statement as the blob contents will take ownership of this buffered payload and dispose it when the contents it is disposed
			IBufferedPayload? bufferedPayload = null;
			if (!streamCanSeek)
			{
				bufferedPayload = await _bufferedPayloadFactory.CreateFromStreamAsync(blobContents.Stream, blobContents.Length, "populate-store", cancellationToken);
			}

			// Don't populate the last store, as that is where we got the hit
			for (int i = 0; i < numStoreMisses; i++)
			{
				IBlobStore blobStore = _blobStores[i];
				using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.PopulateStore")
					.SetAttribute("operation.name", "HierarchicalStore.PopulateStore")
					.SetAttribute("BlobStore", blobStore.GetType().Name);

				Stream? s = null;
				try
				{
					s = streamCanSeek ? blobContents.Stream : bufferedPayload!.GetStream();

					// Populate each store traversed that did not have the content found lower in the hierarchy
					await blobStore.PutObjectAsync(ns, s, blob, cancellationToken);
				}
				finally
				{
					if (s != null)
					{
						if (streamCanSeek)
						{
							// it is a seekable stream so we just reset this
							s.Seek(0, SeekOrigin.Begin);
						}
						else
						{
							await s.DisposeAsync();
						}
					}
				}
			}

			if (streamCanSeek)
			{
				return blobContents;
			}
			else
			{
				// dispose the old blob contents before returning a new one
				await blobContents.DisposeAsync();
				return new BlobContents(bufferedPayload!);
			}
		}

		return blobContents;
	}

	public async Task<Uri?> GetObjectWithRedirectAsync(NamespaceId ns, BlobId blob, List<string>? storageLayers = null, CancellationToken cancellationToken = default)
	{
		bool seenBlobNotFound = false;
		bool seenNamespaceNotFound = false;
		int numStoreMisses = 0;
		IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();

		Uri? redirectUri = null;
		foreach (IBlobStore store in _blobStores)
		{
			string blobStoreName = store.GetType().Name;

			// check which storage layers to skip if we have a explicit list of storage layers to use
			if (storageLayers != null && storageLayers.Count != 0)
			{
				bool found = false;
				foreach (string storageLayer in storageLayers)
				{
					if (string.Equals(storageLayer, blobStoreName, StringComparison.OrdinalIgnoreCase))
					{
						found = true;
						break;
					}
				}

				if (!found)
				{
					continue;
				}
			}

			using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.GetObjectRedirect")
				.SetAttribute("operation.name", "HierarchicalStore.GetObject")
				.SetAttribute("resource.name", blob.ToString())
				.SetAttribute("BlobStore", store.GetType().ToString())
				.SetAttribute("ObjectFound", false.ToString())
				;

			string storeName = store.GetType().Name;
			using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope($"blob.get-redirect.{storeName}", $"Blob GET from: '{storeName}'");

			try
			{
				redirectUri = await store.GetObjectByRedirectAsync(ns, blob, cancellationToken);
				scope.SetAttribute("ObjectFound", true.ToString());
				break;
			}
			catch (BlobNotFoundException)
			{
				seenBlobNotFound = true;
				numStoreMisses++;
			}
			catch (NamespaceNotFoundException)
			{
				seenNamespaceNotFound = true;
			}
		}

		if (seenBlobNotFound && redirectUri == null)
		{
			throw new BlobNotFoundException(ns, blob);
		}

		if (seenNamespaceNotFound && redirectUri == null)
		{
			throw new NamespaceNotFoundException(ns);
		}

		// if we applied filters to the storage layers resulting in no blob found we consider it a miss
		if (storageLayers != null && storageLayers.Count != 0 && redirectUri == null)
		{
			throw new BlobNotFoundException(ns, blob);
		}

		return redirectUri;
	}

	public async Task<BlobMetadata> GetObjectMetadataAsync(NamespaceId ns, BlobId blobId, CancellationToken cancellationToken)
	{
		bool seenBlobNotFound = false;
		bool seenNamespaceNotFound = false;
		IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();

		foreach (IBlobStore store in _blobStores)
		{
			string storeName = store.GetType().Name;

			using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.GetObjectMetadata")
					.SetAttribute("operation.name", "HierarchicalStore.GetObjectMetadata")
					.SetAttribute("resource.name", blobId.ToString())
					.SetAttribute("BlobStore", storeName)
					.SetAttribute("ObjectFound", false.ToString())
				;

			using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope($"blob.get-metadata.{storeName}", $"Blob GET Metadata from: '{storeName}'");

			try
			{
				BlobMetadata metadata = await store.GetObjectMetadataAsync(ns, blobId, cancellationToken);
				scope.SetAttribute("ObjectFound", true.ToString());
				return metadata;
			}
			catch (BlobNotFoundException)
			{
				seenBlobNotFound = true;
			}
			catch (NamespaceNotFoundException)
			{
				seenNamespaceNotFound = true;
			}
		}

		if (seenBlobNotFound)
		{
			throw new BlobNotFoundException(ns, blobId);
		}

		if (seenNamespaceNotFound)
		{
			throw new NamespaceNotFoundException(ns);
		}

		// the only way we get here is we can not find the blob in any store, thus it does not exist (should have triggered the blob not found above)
		throw new BlobNotFoundException(ns, blobId);
	}

	public async Task<BlobContents> ReplicateObjectAsync(NamespaceId ns, BlobId blob, bool supportsRedirectUri = false, bool force = false, BucketId? bucketHint = null, CancellationToken cancellationToken = default)
	{
		if (!force && !ShouldFetchBlobOnDemand(ns))
		{
			throw new NotSupportedException($"Replication is not allowed in namespace {ns}");
		}
		IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();
		using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.Replicate").SetAttribute("operation.name", "HierarchicalStore.Replicate").SetAttribute("resource.name", blob.ToString());

		List<string> regions;
		try
		{
			regions = await _blobIndex.GetBlobRegionsAsync(ns, blob, cancellationToken);
		}
		catch (BlobNotFoundException)
		{
			regions = new List<string>();
		}

		if (!regions.Any())
		{
			scope.SetAttribute("found", false);
			throw new BlobReplicationException(ns, blob, "Blob not found in any region");
		}
		scope.SetAttribute("found", true);

		_logger.LogInformation("On-demand replicating blob {Blob} in Namespace {Namespace}", blob, ns);
		List<(int, string)> possiblePeers = new List<(int, string)>(_peerStatusService.GetPeersByLatency(regions));

		bool replicated = false;
		foreach ((int latency, string? region) in possiblePeers)
		{
			PeerStatus? peerStatus = _peerStatusService.GetPeerStatus(region);
			if (peerStatus == null)
			{
				throw new Exception($"Failed to find peer {region}");
			}

			_logger.LogInformation("Attempting to replicate blob {Blob} in Namespace {Namespace} from {Region}", blob, ns, region);

			using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope($"blob.replicate.{region}", "Replicating blob from remote instances");

			PeerEndpoints peerEndpoint = peerStatus.Endpoints.First();
			using HttpClient httpClient = _httpClientFactory.CreateClient();
			string url = peerEndpoint.Url.ToString();
			if (!url.EndsWith("/", StringComparison.InvariantCultureIgnoreCase))
			{
				url += "/";
			}
			using TelemetrySpan replicateScope = _tracer.StartActiveSpan("HierarchicalStore.Replicate.From").SetAttribute("operation.name", "HierarchicalStore.Replicate.From").SetAttribute("resource.name", $"{region}-{blob}");

			using HttpRequestMessage blobRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"{url}api/v1/blobs/{ns}/{blob}?allowOndemandReplication=false"), cancellationToken);
			HttpResponseMessage blobResponse = await httpClient.SendAsync(blobRequest, HttpCompletionOption.ResponseHeadersRead, cancellationToken);

			if (blobResponse.StatusCode == HttpStatusCode.NotFound)
			{
				replicateScope.SetAttribute("found", false);

				// try the next region
				continue;
			}

			if (blobResponse.StatusCode != HttpStatusCode.OK)
			{
				_logger.LogWarning("Failed to replicate {Blob} in {Namespace} from region {Region} due to bad http status code {StatusCode}.", blob, ns, region, blobResponse.StatusCode);
				BlobReplicationException e = new BlobReplicationException(ns, blob, $"Failed to replicate {blob} in {ns} from region {region} due to bad http status code {blobResponse.StatusCode}.");
				replicateScope.RecordException(e);
				throw e;
			}
			replicateScope.SetAttribute("found", true);

			await using Stream s = await blobResponse.Content.ReadAsStreamAsync(cancellationToken);
			using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFilesystemBufferedPayloadAsync(s, "replicate-blob", cancellationToken);
			await PutObjectAsync(ns, payload, blob, bucketHint: bucketHint, cancellationToken: cancellationToken);
			replicated = true;
			break;
		}

		scope.SetAttribute("replicated", replicated);

		if (!replicated)
		{
			_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to it not existing in any region", blob, ns);

			throw new BlobReplicationException(ns, blob, $"Failed to replicate {blob} in {ns} due to it not existing in any region");
		}
		return await GetObjectAsync(ns, blob, supportsRedirectUri: supportsRedirectUri, bucketHint: bucketHint, cancellationToken: cancellationToken);
	}

	public bool ShouldFetchBlobOnDemand(NamespaceId ns)
	{
		return _settings.CurrentValue.EnableOnDemandReplication && _namespacePolicyResolver.GetPoliciesForNs(ns).OnDemandReplication;
	}

	public bool IsMultipartUploadSupported(NamespaceId ns)
	{
		foreach (IBlobStore store in _blobStores)
		{
			if (store is IMultipartBlobStore)
			{
				return true;
			}
		}

		return false;
	}

	public async Task<(string?, string?)> StartMultipartUploadAsync(NamespaceId ns, CancellationToken cancellationToken = default)
	{
		foreach (IBlobStore store in _blobStores)
		{
			if (store is IMultipartBlobStore multipartBlobStore)
			{
				string blobStoreName = store.GetType().Name;

				using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.StartMultipartUpload")
						.SetAttribute("operation.name", "HierarchicalStore.StartMultipartUpload")
						.SetAttribute("BlobStore", blobStoreName)
					;

				string blobName = CbObjectId.NewObjectId().ToString();
				scope.SetAttribute("resource.name", blobName);

				return (await multipartBlobStore.StartMultipartUploadAsync(ns, blobName, cancellationToken), blobName);
			}
		}

		return (null, null);
	}

	public async Task CompleteMultipartUploadAsync(NamespaceId ns, string blobName, string uploadId, List<string> partIds, CancellationToken cancellationToken = default)
	{
		foreach (IBlobStore store in _blobStores)
		{
			if (store is IMultipartBlobStore multipartBlobStore)
			{
				string blobStoreName = store.GetType().Name;

				using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.CompleteMultipartUploadAsync")
						.SetAttribute("operation.name", "HierarchicalStore.CompleteMultipartUploadAsync")
						.SetAttribute("resource.name", blobName)
						.SetAttribute("BlobStore", blobStoreName)
					;

				await multipartBlobStore.CompleteMultipartUploadAsync(ns, blobName, uploadId, partIds, cancellationToken);
				return;
			}
		}

		throw new NotImplementedException("Multipart uploads not supported by any backend");
	}

	public async Task PutMultipartUploadAsync(NamespaceId ns, string blobName, string uploadId, string partIdentifier, byte[] blobData, CancellationToken cancellationToken = default)
	{
		foreach (IBlobStore store in _blobStores)
		{
			if (store is IMultipartBlobStore multipartBlobStore)
			{
				string blobStoreName = store.GetType().Name;

				using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.PutMultipartUploadAsync")
						.SetAttribute("operation.name", "HierarchicalStore.PutMultipartUploadAsync")
						.SetAttribute("resource.name", blobName)
						.SetAttribute("BlobStore", blobStoreName)
					;

				await multipartBlobStore.PutMultipartPartAsync(ns, blobName, uploadId, partIdentifier, blobData, cancellationToken);
				return;
			}
		}

		throw new NotImplementedException("Multipart uploads not supported by any backend");
	}

	public async Task<Uri?> MaybePutMultipartUploadWithRedirectAsync(NamespaceId ns, string blobName, string uploadId, string partIdentifier, CancellationToken cancellationToken = default)
	{
		foreach (IBlobStore store in _blobStores)
		{
			if (store is IMultipartBlobStore multipartBlobStore)
			{
				string blobStoreName = store.GetType().Name;

				using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.PutMultipartUploadAsync")
					.SetAttribute("operation.name", "HierarchicalStore.PutMultipartUploadAsync")
					.SetAttribute("resource.name", blobName)
					.SetAttribute("BlobStore", blobStoreName);

				return await multipartBlobStore.GetWriteRedirectForPartAsync(ns, blobName, uploadId, partIdentifier, cancellationToken);
			}
		}

		return null;
	}

	public async Task<(ContentId, BlobId)> VerifyMultipartUpload(NamespaceId ns, BlobId id, string blobName, bool isCompressed, CancellationToken cancellationToken = default)
	{
		IMultipartBlobStore? blobStore = null;
		foreach (IBlobStore store in _blobStores)
		{
			if (store is IMultipartBlobStore multipartBlobStore)
			{
				blobStore = multipartBlobStore;
			}
		}

		if (blobStore == null)
		{
			throw new NotImplementedException("Multipart upload not supported by any backend");
		}

		string blobStoreName = blobStore.GetType().Name;

		using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.VerifyMultipartUpload")
			.SetAttribute("operation.name", "HierarchicalStore.VerifyMultipartUpload")
			.SetAttribute("resource.name", id.ToString())
			.SetAttribute("BlobStore", blobStoreName);

		BlobContents? blobContents = await blobStore.GetMultipartObjectByNameAsync(ns, blobName, cancellationToken);

		if (blobContents == null)
		{
			throw new Exception($"Failed to find multipart blob {blobName}");
		}

		using IBufferedPayload bufferedPayload = await _bufferedPayloadFactory.CreateFromStreamAsync(blobContents.Stream, blobContents.Length, "verify-multipart", cancellationToken);
		ContentId cid;
		BlobId blobId;
		if (isCompressed)
		{
			(cid, blobId) = await this.PutCompressedObjectMetadataAsync(ns, bufferedPayload, ContentId.FromBlobIdentifier(id), _provider, cancellationToken: cancellationToken);
			await _blobIndex.AddBlobToIndexAsync(ns, blobId, rawSize: (ulong)bufferedPayload.Length, cancellationToken: cancellationToken);
		}
		else
		{
			ContentHash hash = await VerifyContentMatchesHashAsync(blobContents.Stream, id, cancellationToken: cancellationToken);
			blobId = BlobId.FromContentHash(hash);
			cid = ContentId.FromContentHash(hash);

			await _blobIndex.AddBlobToIndexAsync(ns, blobId, rawSize: (ulong)bufferedPayload.Length, cancellationToken: cancellationToken);
		}

		await blobStore.RenameMultipartBlobAsync(ns, blobName, blobId, cancellationToken);
		
		return (cid, blobId);

	}

	public List<MultipartByteRange> GetMultipartRanges(NamespaceId ns, string uploadId, string blobName, ulong blobLength, CancellationToken cancellationToken = default)
	{
		IMultipartBlobStore? blobStore = null;
		foreach (IBlobStore store in _blobStores)
		{
			if (store is IMultipartBlobStore multipartBlobStore)
			{
				blobStore = multipartBlobStore;
			}
		}

		if (blobStore == null)
		{
			throw new NotImplementedException("Multipart upload not supported by any backend");
		}

		string blobStoreName = blobStore.GetType().Name;

		using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.GetMultipartPartIds")
			.SetAttribute("operation.name", "HierarchicalStore.GetMultipartPartIds")
			.SetAttribute("resource.name", blobName)
			.SetAttribute("BlobStore", blobStoreName);

		return blobStore.GetMultipartRanges(ns, uploadId, blobLength, cancellationToken);
	}

	public MultipartLimits? GetMultipartLimits(NamespaceId ns)
	{
		IMultipartBlobStore? blobStore = null;
		foreach (IBlobStore store in _blobStores)
		{
			if (store is IMultipartBlobStore multipartBlobStore)
			{
				blobStore = multipartBlobStore;
			}
		}

		if (blobStore == null)
		{
			return null;
		}

		return blobStore.GetMultipartLimits(ns);
	}

	public async Task CopyBlobAsync(NamespaceId ns, NamespaceId targetNamespace, BlobId blobId, BucketId? bucketHint = null, CancellationToken cancellationToken = default)
	{
		// we only copy the data in the root store as the cache layers do not really need this copied
		IBlobStore rootStore = _blobStores.Last();
		ulong rawSize = await rootStore.CopyBlobAsync(ns, targetNamespace, blobId, cancellationToken);

		await _blobIndex.AddBlobToIndexAsync(targetNamespace, blobId, rawSize: rawSize, cancellationToken: cancellationToken);
		await _replicationLog.InsertAddBlobEventAsync(targetNamespace, blobId, DateTime.UtcNow.ToReplicationBucket(), bucketHint: bucketHint);
	}

	private async Task<HttpRequestMessage> BuildHttpRequestAsync(HttpMethod httpMethod, Uri uri, CancellationToken cancellationToken)
	{
		string? token = await _serviceCredentials.GetTokenAsync(cancellationToken);
		HttpRequestMessage request = new HttpRequestMessage(httpMethod, uri);
		if (!string.IsNullOrEmpty(token))
		{
			request.Headers.Add("Authorization", $"{_serviceCredentials.GetAuthenticationScheme()} {token}");
		}

		return request;
	}

	public async Task<bool> ExistsAsync(NamespaceId ns, BlobId blob, List<string>? storageLayers = null, bool ignoreRemoteBlobs = false, CancellationToken cancellationToken = default)
	{
		bool exists = await ExistsInStoresAsync(ns, blob, storageLayers, cancellationToken);
		if (exists)
		{
			return exists;
		}

		NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
		if (policy.FallbackNamespace != null)
		{
			return await ExistsInStoresAsync(policy.FallbackNamespace.Value, blob, storageLayers, cancellationToken);
		}

		if (ShouldFetchBlobOnDemand(ns) && !ignoreRemoteBlobs)
		{
			return await ExistsInRemoteAsync(ns, blob, cancellationToken);
		}

		return false;
	}

	private async Task<bool> ExistsInStoresAsync(NamespaceId ns, BlobId blob, List<string>? storageLayers = null, CancellationToken cancellationToken = default)
	{
		bool useBlobIndex = _namespacePolicyResolver.GetPoliciesForNs(ns).UseBlobIndexForExists;
		if (useBlobIndex)
		{
			using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.ObjectExists")
				.SetAttribute("operation.name", "HierarchicalStore.ObjectExists")
				.SetAttribute("resource.name", blob.ToString())
				.SetAttribute("BlobStore", "BlobIndex")
				;
			bool exists = await _blobIndex.BlobExistsInRegionAsync(ns, blob, cancellationToken: cancellationToken);
			if (exists)
			{
				scope.SetAttribute("ObjectFound", true.ToString());
				return true;
			}

			scope.SetAttribute("ObjectFound", false.ToString());
			return false;
		}
		else
		{
			foreach (IBlobStore store in _blobStores)
			{
				string blobStoreName = store.GetType().Name;
				// check which storage layers to skip if we have an explicit list of storage layers to use
				if (storageLayers != null && storageLayers.Count != 0)
				{
					bool found = false;
					foreach (string storageLayer in storageLayers)
					{
						if (string.Equals(storageLayer, blobStoreName, StringComparison.OrdinalIgnoreCase))
						{
							found = true;
							break;
						}
					}

					if (!found)
					{
						continue;
					}
				}

				using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.ObjectExists")
					.SetAttribute("operation.name", "HierarchicalStore.ObjectExists")
					.SetAttribute("resource.name", blob.ToString())
					.SetAttribute("BlobStore", blobStoreName)
					;
				if (await store.ExistsAsync(ns, blob, cancellationToken: cancellationToken))
				{
					scope.SetAttribute("ObjectFound", true.ToString());
					return true;
				}
				scope.SetAttribute("ObjectFound", false.ToString());
			}

			return false;
		}
	}

	public async Task<bool> ExistsInRemoteAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken)
	{
		IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();
		using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.ExistsRemote")
			.SetAttribute("operation.name", "HierarchicalStore.ExistsRemote")
			.SetAttribute("resource.name", blob.ToString());

		using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope("blob.exists-remote", "Verify if blob exists in remotes");

		List<string> regions;
		try
		{
			regions = await _blobIndex.GetBlobRegionsAsync(ns, blob, cancellationToken);
		}
		catch (BlobNotFoundException)
		{
			regions = new List<string>();
		}

		// we do not actually verify that the blob exists remotely as that would take a lot of time
		// instead we simply check if there are any regions were the blob exists that is not our current region

		// if it exists in more then one region, we are sure it exists somewhere that is not here
		if (regions.Count > 1)
		{
			scope.SetAttribute("ObjectFound", true.ToString());
			return true;
		}

		if (regions.Any(region => !string.Equals(region, _currentSite, StringComparison.OrdinalIgnoreCase)))
		{
			scope.SetAttribute("ObjectFound", true.ToString());
			return true;
		}

		scope.SetAttribute("ObjectFound", false.ToString());
		return false;
	}

	public async Task<bool> ExistsInRootStoreAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default)
	{
		IBlobStore store = _blobStores.Last();

		using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.ObjectExistsInRoot")
			.SetAttribute("operation.name", "HierarchicalStore.ObjectExistsInRoot")
			.SetAttribute("resource.name", blob.ToString())
			.SetAttribute("BlobStore", store.GetType().Name)
			;
		if (await store.ExistsAsync(ns, blob, cancellationToken: cancellationToken))
		{
			scope.SetAttribute("ObjectFound", true.ToString());
			return true;
		}
		scope.SetAttribute("ObjectFound", false.ToString());
		return false;
	}

	public async Task DeleteObjectAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default)
	{
		bool blobNotFound = false;
		bool deletedAtLeastOnce = false;

		// remove the object from the tracking first, if this times out we do not want to end up with a inconsistent blob index
		// if the blob store delete fails on the other hand we will still run a delete again during GC (as the blob is still orphaned at that point)
		// this assumes that blob gc is based on scanning the root blob store
		await _blobIndex.RemoveBlobFromAllRegionsAsync(ns, blob, cancellationToken: cancellationToken);

		if (_settings.CurrentValue.RemoveReferencesOnBlobDeletion)
		{
			await _blobIndex.RemoveReferencesAsync(ns, blob, null, cancellationToken);
		}

		foreach (IBlobStore store in _blobStores)
		{
			try
			{
				using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.DeleteObject")
					.SetAttribute("operation.name", "HierarchicalStore.DeleteObject")
					.SetAttribute("resource.name", blob.ToString())
					.SetAttribute("BlobStore", store.GetType().Name)
					;

				await store.DeleteObjectAsync(ns, blob, cancellationToken);
				deletedAtLeastOnce = true;
			}
			catch (NamespaceNotFoundException)
			{
				// Ignore
			}
			catch (BlobNotFoundException)
			{
				blobNotFound = true;
			}
		}

		if (deletedAtLeastOnce)
		{
			return;
		}

		if (blobNotFound)
		{
			throw new BlobNotFoundException(ns, blob);
		}

		throw new NamespaceNotFoundException(ns);
	}

	public async Task DeleteObjectAsync(List<NamespaceId> namespaces, BlobId blob, CancellationToken cancellationToken = default)
	{
		// remove the object from the tracking first, if this times out we do not want to end up with a inconsistent blob index
		// if the blob store delete fails on the other hand we will still run a delete again during GC (as the blob is still orphaned at that point)
		// this assumes that blob gc is based on scanning the root blob store

		await Parallel.ForEachAsync(namespaces, cancellationToken, async (ns, token) =>
		{
			await _blobIndex.RemoveBlobFromAllRegionsAsync(ns, blob, cancellationToken: token);
			if (_settings.CurrentValue.RemoveReferencesOnBlobDeletion)
			{
				await _blobIndex.RemoveReferencesAsync(ns, blob, null, token);
			}
		});

		// let each blob store figure out how it could effectively clear a storage pool
		foreach (IBlobStore store in _blobStores)
		{
			using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.DeleteObjectFromStoragePool")
					.SetAttribute("operation.name", "HierarchicalStore.DeleteObjectFromStoragePool")
					.SetAttribute("resource.name", blob.ToString())
					.SetAttribute("BlobStore", store.GetType().Name)
				;

			await store.DeleteObjectAsync(namespaces, blob, cancellationToken);
		}
	}

	public async Task DeleteNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default)
	{
		bool deletedAtLeastOnce = false;
		foreach (IBlobStore store in _blobStores)
		{
			using TelemetrySpan scope = _tracer.StartActiveSpan("HierarchicalStore.DeleteNamespace")
				.SetAttribute("operation.name", "HierarchicalStore.DeleteNamespace")
				.SetAttribute("resource.name", ns.ToString())
				.SetAttribute("BlobStore", store.GetType().Name)
				;
			try
			{
				await store.DeleteNamespaceAsync(ns, cancellationToken);
				deletedAtLeastOnce = true;
			}
			catch (NamespaceNotFoundException)
			{
				// Ignore
			}
		}

		if (deletedAtLeastOnce)
		{
			return;
		}

		throw new NamespaceNotFoundException(ns);
	}

	public IAsyncEnumerable<(BlobId, DateTime)> ListObjectsAsync(NamespaceId ns, CancellationToken cancellationToken)
	{
		// as this is a hierarchy of blob stores the last blob store should contain the superset of all stores
		return _blobStores.Last().ListObjectsAsync(ns, cancellationToken);
	}

	public async Task<BlobId[]> FilterOutKnownBlobsAsync(NamespaceId ns, IEnumerable<BlobId> blobs, CancellationToken cancellationToken)
	{
		List<(BlobId, Task<bool>)> existTasks = new();
		foreach (BlobId blob in blobs)
		{
			existTasks.Add((blob, ExistsAsync(ns, blob, cancellationToken: cancellationToken)));
		}

		List<BlobId> missingBlobs = new();
		foreach ((BlobId blob, Task<bool> existsTask) in existTasks)
		{
			bool exists = await existsTask;

			if (!exists)
			{
				missingBlobs.Add(blob);
			}
		}

		return missingBlobs.ToArray();
	}

	public async Task<BlobId[]> FilterOutKnownBlobsAsync(NamespaceId ns, IAsyncEnumerable<BlobId> blobs, CancellationToken cancellationToken)
	{
		ConcurrentBag<BlobId> missingBlobs = new ConcurrentBag<BlobId>();

		try
		{
			await Parallel.ForEachAsync(blobs, cancellationToken, async (identifier, ctx) =>
			{
				bool exists = await ExistsAsync(ns, identifier, cancellationToken: ctx);

				if (!exists)
				{
					missingBlobs.Add(identifier);
				}
			});
		}
		catch (AggregateException e)
		{
			if (e.InnerException is PartialReferenceResolveException)
			{
				throw e.InnerException;
			}

			throw;
		}

		return missingBlobs.ToArray();
	}

	public async Task<BlobContents> GetObjectsAsync(NamespaceId ns, BlobId[] blobs, CancellationToken cancellationToken)
	{
		using TelemetrySpan _ = _tracer.StartActiveSpan("blob.combine").SetAttribute("operation.name", "blob.combine");
		Task<BlobContents>[] tasks = new Task<BlobContents>[blobs.Length];
		for (int i = 0; i < blobs.Length; i++)
		{
			tasks[i] = GetObjectAsync(ns, blobs[i], cancellationToken: cancellationToken);
		}

		MemoryStream ms = new MemoryStream();
		foreach (Task<BlobContents> task in tasks)
		{
			BlobContents blob = await task;
			await using Stream s = blob.Stream;
			await s.CopyToAsync(ms, cancellationToken);
		}

		ms.Seek(0, SeekOrigin.Begin);

		return new BlobContents(ms, ms.Length);
	}
}
