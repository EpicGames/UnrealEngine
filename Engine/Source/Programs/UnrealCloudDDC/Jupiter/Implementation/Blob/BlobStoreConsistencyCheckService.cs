// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Implementation.Blob;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class ConsistencyState
	{
	}

	// ReSharper disable once ClassNeverInstantiated.Global
	public class BlobStoreConsistencyCheckService : PollingService<ConsistencyState>
	{
		private readonly IOptionsMonitor<ConsistencyCheckSettings> _settings;
		private readonly IOptionsMonitor<UnrealCloudDDCSettings> _unrealCloudDDCSettings;
		private readonly IServiceProvider _provider;
		private readonly ILeaderElection _leaderElection;
		private readonly IReferencesStore _referencesStore;
		private readonly IBlobIndex _blobIndex;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;
		private readonly INamespacePolicyResolver _policyResolver;

		protected override bool ShouldStartPolling()
		{
			return _settings.CurrentValue.EnableBlobStoreChecks;
		}

		public BlobStoreConsistencyCheckService(IOptionsMonitor<ConsistencyCheckSettings> settings, IOptionsMonitor<UnrealCloudDDCSettings> unrealCloudDDCSettings, IServiceProvider provider, ILeaderElection leaderElection, IReferencesStore referencesStore, IBlobIndex blobIndex, Tracer tracer, ILogger<BlobStoreConsistencyCheckService> logger, INamespacePolicyResolver policyResolver) : base(serviceName: nameof(BlobStoreConsistencyCheckService), TimeSpan.FromSeconds(settings.CurrentValue.ConsistencyCheckPollFrequencySeconds), new ConsistencyState(), logger)
		{
			_settings = settings;
			_unrealCloudDDCSettings = unrealCloudDDCSettings;
			_provider = provider;
			_leaderElection = leaderElection;
			_referencesStore = referencesStore;
			_blobIndex = blobIndex;
			_tracer = tracer;
			_logger = logger;
			_policyResolver = policyResolver;
		}

		public override async Task<bool> OnPollAsync(ConsistencyState state, CancellationToken cancellationToken)
		{
			if (!_settings.CurrentValue.EnableBlobStoreChecks)
			{
				_logger.LogInformation("Skipped running blob store consistency check as it is disabled");
				return false;
			}

			await RunConsistencyCheckAsync(cancellationToken);

			return true;
		}

		private async Task RunConsistencyCheckAsync(CancellationToken cancellationToken)
		{
			foreach (IBlobStore blobStore in BlobService.GetBlobStores(_provider, _unrealCloudDDCSettings).Where(RunConsistencyCheckOnBlobStore))
			{
				string blobStoreName = blobStore.GetType().Name;

				bool isRootStore = blobStore is AmazonS3Store or AzureBlobStore;
				bool requiresLeader = blobStore is not FileSystemStore;

				if (!_settings.CurrentValue.RunBlobStoreConsistencyCheckOnRootStore && isRootStore)
				{
					continue;
				}

				if (requiresLeader && !_leaderElection.IsThisInstanceLeader())
				{
					_logger.LogInformation("Skipped running blob store consistency check Blob Store {BlobStore} because this instance was not the leader", blobStoreName);
					continue;
				}

				List<NamespaceId> namespaces = await _referencesStore.GetNamespacesAsync(cancellationToken).ToListAsync(cancellationToken);

				// technically this does not need to be run per namespace but per storage pool
				foreach (NamespaceId ns in namespaces)
				{
					ulong countOfBlobsChecked = 0;
					ulong countOfIncorrectBlobsFound = 0;

					// consistency checks do not run on none content address storage as we have no way of verifying the identifier
					if (!_policyResolver.GetPoliciesForNs(ns).UseContentAddressedStorage)
					{
						continue;
					}

					await foreach ((BlobId blob, DateTime lastModified) in blobStore.ListObjectsAsync(ns, cancellationToken))
					{
						using TelemetrySpan scope = _tracer.StartActiveSpan("consistency_check.blob_store")
							.SetAttribute("operation.name", "consistency_check.blob_store")
							.SetAttribute("resource.name", $"{ns}.{blob}")
							.SetAttribute("BlobStore", blobStoreName);

						if (countOfBlobsChecked % 100 == 0)
						{
							_logger.LogInformation("Consistency check running on Blob Store {BlobStore}, count of blobs processed so far: {CountOfBlobs}", blobStoreName, countOfBlobsChecked);
						}

						Interlocked.Increment(ref countOfBlobsChecked);

						BlobContents contents = await blobStore.GetObjectAsync(ns, blob, LastAccessTrackingFlags.SkipTracking, cancellationToken: cancellationToken);
						await using Stream s = contents.Stream;

						bool inconsistencyFound = false;
						BlobId newHash = await BlobId.FromStreamAsync(s, cancellationToken);
						if (!blob.Equals(newHash))
						{
							_logger.LogError("Mismatching hash for {Blob} in {Namespace} stored in {BlobStore}, new hash has {NewHash}. Deleting incorrect blob.", blob, ns, blobStoreName, newHash);

							Interlocked.Increment(ref countOfIncorrectBlobsFound);
							await blobStore.DeleteObjectAsync(ns, blob, cancellationToken);

							if (isRootStore)
							{
								// update blob index tracking to indicate that we no longer have this blob in this region
								await _blobIndex.RemoveBlobFromRegionAsync(ns, blob, cancellationToken: cancellationToken);
							}
						}

						scope.SetAttribute("deleted", inconsistencyFound.ToString());
					}

					_logger.LogInformation("Blob Store {BlobStore}: Consistency check finished for {Namespace}, found {CountOfIncorrectBlobs} incorrect blobs. Processed {CountOfBlobs} blobs.", blobStoreName, ns, countOfIncorrectBlobsFound, countOfBlobsChecked);
				}
			}
		}

		private bool RunConsistencyCheckOnBlobStore(IBlobStore blobStore)
		{
			switch (blobStore)
			{
				case FileSystemStore:
				case AzureBlobStore:
				case AmazonS3Store:
					return true;
				case MemoryBlobStore:
				case RelayBlobStore:
					return false;
				default:
					throw new NotImplementedException();
			}
		}

		protected override Task OnStopping(ConsistencyState state)
		{
			return Task.CompletedTask;
		}
	}
}
