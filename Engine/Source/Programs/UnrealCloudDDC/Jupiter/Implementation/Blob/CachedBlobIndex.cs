// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation.Blob;

public class CachedBlobIndex : IBlobIndex
{
	private readonly FileSystemStore _fileSystemStore;

	public CachedBlobIndex(FileSystemStore fileSystemStore)
	{
		_fileSystemStore = fileSystemStore;
	}

	public async Task AddBlobToIndexAsync(NamespaceId ns, BlobId id, ulong rawSize, string? region = null,
		CancellationToken cancellationToken = default)
	{
		// We do not actually track any blob information when running in cached mode
		await Task.CompletedTask;
	}

	public async Task RemoveBlobFromRegionAsync(NamespaceId ns, BlobId id, string? region = null, CancellationToken cancellationToken = default)
	{
		// We do not actually track any blob information when running in cached mode
		await Task.CompletedTask;
	}

	public async Task RemoveBlobFromAllRegionsAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken = default)
	{
		// We do not actually track any blob information when running in cached mode
		await Task.CompletedTask;
	}

	public IAsyncEnumerable<(NamespaceId, BaseBlobReference)> GetAllBlobReferencesAsync(CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public IAsyncEnumerable<BaseBlobReference> GetBlobReferencesAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public async Task<bool> BlobExistsInRegionAsync(NamespaceId ns, BlobId blobIdentifier, string? region = null, CancellationToken cancellationToken = default)
	{
		return await _fileSystemStore.ExistsAsync(ns, blobIdentifier, forceCheck: false, cancellationToken: cancellationToken);
	}

	public async Task AddRefToBlobsAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId[] blobs, CancellationToken cancellationToken)
	{
		// We do not actually track any blob information when running in cached mode
		await Task.CompletedTask;
	}

	public IAsyncEnumerable<(NamespaceId, BlobId)> GetAllBlobsAsync(CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public Task RemoveReferencesAsync(NamespaceId ns, BlobId id, List<BaseBlobReference>? referencesToRemove, CancellationToken cancellationToken)
	{
		// We do not actually track any blob information when running in cached mode
		return Task.CompletedTask;
	}

	public Task<List<string>> GetBlobRegionsAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public Task AddBlobReferencesAsync(NamespaceId ns, BlobId sourceBlob, BlobId targetBlob, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public Task AddBlobToBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobId, long blobSize, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public Task RemoveBlobFromBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, List<BlobId> blobIds, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public Task<BucketStats> CalculateBucketStatisticsAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}
}
