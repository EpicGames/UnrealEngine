// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Jupiter.Common;
using Jupiter.DataStore;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation.Blob
{
	public class DataStoreBlobIndex : IBlobIndex
	{
		private readonly INamespacePolicyResolver _policyResolver;
		private readonly IOptionsMonitor<JupiterSettings> _jupiterSettings;
		private readonly DataStoreTable<DataStoreBlobRecord> _blobTable;

		public DataStoreBlobIndex(IDataStoreTableService tableService, INamespacePolicyResolver policyResolver, IOptionsMonitor<JupiterSettings> jupiterSettings)
		{
			_policyResolver = policyResolver;
			_jupiterSettings = jupiterSettings;
			_blobTable = tableService.GetTableByType<DataStoreBlobRecord>();
		}
		public async Task AddBlobToIndexAsync(NamespaceId ns, BlobId id, ulong rawSize, string? region = null, CancellationToken cancellationToken = default)
		{
			await _blobTable.InsertAsync(GetKeyspace(ns), CalculatePrefix(id), DataStoreBlobRecord.GetPartitionKey(id), new DataStoreBlobRecord(ContentId.FromBlobIdentifier(id), rawSize, rawSize), true, cancellationToken);
		}

		private KeyspaceId GetKeyspace(NamespaceId ns)
		{
			return KeyspaceId.FromStoragePool(_policyResolver.GetPoliciesForNs(ns).StoragePool);
		}

		internal static string CalculatePrefix(BlobId id)
		{
			return StringUtils.FormatAsHexLowerString(id.HashData.AsSpan()[0..1]);
		}

		public Task RemoveBlobFromRegionAsync(NamespaceId ns, BlobId id, string? region = null, CancellationToken cancellationToken = default)
		{
			return Task.CompletedTask;
		}

		public async Task RemoveBlobFromAllRegionsAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken = default)
		{
			await _blobTable.DeleteAsync(GetKeyspace(ns), CalculatePrefix(id), DataStoreBlobRecord.GetPartitionKey(id), cancellationToken);
		}

		public async Task<bool> BlobExistsInRegionAsync(NamespaceId ns, BlobId id, string? region = null, CancellationToken cancellationToken = default)
		{
			DataStorePartitionValue? value = await _blobTable.FindAsync(GetKeyspace(ns), CalculatePrefix(id), DataStoreBlobRecord.GetPartitionKey(id), cancellationToken);
			return value != null;
		}

		public IAsyncEnumerable<(NamespaceId, BlobId)> GetAllBlobsAsync(CancellationToken cancellationToken = default)
		{
			throw new System.NotImplementedException();
		}

		public IAsyncEnumerable<(NamespaceId, BaseBlobReference)> GetAllBlobReferencesAsync(CancellationToken cancellationToken = default)
		{
			throw new System.NotImplementedException();
		}

		public IAsyncEnumerable<BaseBlobReference> GetBlobReferencesAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken = default)
		{
			throw new System.NotImplementedException();
		}

		public Task AddRefToBlobsAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId[] blobs, CancellationToken cancellationToken = default)
		{
			return Task.CompletedTask;
		}

		public Task RemoveReferencesAsync(NamespaceId ns, BlobId id, List<BaseBlobReference>? referencesToRemove, CancellationToken cancellationToken = default)
		{
			return Task.CompletedTask;
		}

		public async Task<List<string>> GetBlobRegionsAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default)
		{
			if (await BlobExistsInRegionAsync(ns, blob, cancellationToken: cancellationToken))
			{
				return [_jupiterSettings.CurrentValue.CurrentSite];
			}

			return [];
		}

		public async Task AddBlobReferencesAsync(NamespaceId ns, BlobId blob, BlobId blobThatReferences, CancellationToken cancellationToken = default)
		{
			throw new System.NotImplementedException();
		}

		public async Task AddBlobToBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobId, long blobSize, CancellationToken cancellationToken = default)
		{
			throw new System.NotImplementedException();
		}

		public async Task RemoveBlobFromBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, List<BlobId> blobIds, CancellationToken cancellationToken = default)
		{
			throw new System.NotImplementedException();
		}

		public async Task<BucketStats> CalculateBucketStatisticsAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken = default)
		{
			throw new System.NotImplementedException();
		}
	}

	internal record struct DataStoreBlobRecord : IDataStoreTableObject<DataStoreBlobRecord>
	{
		public IoHash ContentId { get; set; }

		public ulong UncompressedSize { get; set; }

		public ulong CompressedSize { get; set; }

		public DataStoreBlobRecord(ContentId contentId, ulong uncompressedSize, ulong compressedSize)
		{
			ContentId = contentId.AsIoHash();
			UncompressedSize = uncompressedSize;
			CompressedSize = compressedSize;
		}

		public DataStoreBlobRecord(IoHash contentId, ulong uncompressedSize, ulong compressedSize)
		{
			ContentId = contentId;
			UncompressedSize = uncompressedSize;
			CompressedSize = compressedSize;
		}

		public static DataStorePartitionKey GetPartitionKey(BlobId blobId)
		{
			return new DataStorePartitionKey(blobId.HashData);
		}

		public static DataStorePartitionKey? MaybeGetPartitionKey(BlobId? blobId)
		{
			if (blobId == null)
			{
				return null;
			}
			return new DataStorePartitionKey(blobId.HashData);
		}

		public static byte[] Serialize(DataStoreBlobRecord self)
		{
			byte[] payload = new byte[20 + 8 + 8];
			MemoryWriter writer = new MemoryWriter(payload);

			writer.WriteFixedLengthBytes(self.ContentId.ToByteArray().AsSpan());
			writer.WriteUInt64(self.UncompressedSize);
			writer.WriteUInt64(self.CompressedSize);

			return payload;
		}

		public static void Deserialize(byte[] payload, out DataStoreBlobRecord record)
		{
			MemoryReader reader = new MemoryReader(payload);
			IoHash hash = new IoHash(reader.ReadFixedLengthBytes(20).Span);
			ulong uncompressedSize = reader.ReadUInt64();
			ulong compressedSize = reader.ReadUInt64();

			record = new DataStoreBlobRecord(hash, uncompressedSize, compressedSize);
		}

		public static bool HasSortKey()
		{
			return false;
		}

		public static byte[] GetSortKey(DataStoreBlobRecord self)
		{
			return null!;
		}
	}
}
