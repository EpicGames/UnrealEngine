// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class FileSystemStore : IBlobStore, IBlobCleanup
	{
		private readonly IServiceProvider _provider;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly IOptionsMonitor<FilesystemSettings> _settings;
		private readonly Tracer _tracer;
		private readonly ILogger<FileSystemStore> _logger;
		private readonly ConcurrentDictionary<NamespaceId, FileStorageBackend> _backends = new ConcurrentDictionary<NamespaceId, FileStorageBackend>();

		public FileSystemStore(IOptionsMonitor<FilesystemSettings> settings, Tracer tracer, ILogger<FileSystemStore> logger, IServiceProvider provider, INamespacePolicyResolver namespacePolicyResolver)
		{
			_settings = settings;
			_tracer = tracer;
			_logger = logger;
			_provider = provider;
			_namespacePolicyResolver = namespacePolicyResolver;
		}

		private FileStorageBackend GetBackend(NamespaceId ns)
		{
			return _backends.GetOrAdd(ns, x => ActivatorUtilities.CreateInstance<FileStorageBackend>(_provider, GetFilesystemPath(x), x));
		}

		private string GetRootDir()
		{
			return PathUtil.ResolvePath(_settings.CurrentValue.RootDir);
		}

		public static string GetFilesystemPath(BlobId blob)
		{
			const int CountOfCharactersPerDirectory = 2;
			string objectName = blob.ToString();
			string firstPart = objectName.Substring(0, CountOfCharactersPerDirectory);
			string secondPart = objectName.Substring(CountOfCharactersPerDirectory, CountOfCharactersPerDirectory);
			string fileName = objectName;

			return Path.Combine(firstPart, secondPart, fileName);
		}

		public static FileInfo GetFilesystemPath(string rootDir, NamespaceId ns, BlobId blob)
		{
			return new FileInfo(Path.Combine(rootDir, ns.ToString(), GetFilesystemPath(blob)));
		}

		public DirectoryReference GetFilesystemPath(NamespaceId ns)
		{
			return DirectoryReference.Combine(new DirectoryReference(GetRootDir()), ns.ToString());
		}

		public Task<Uri?> GetObjectByRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not supported
			return Task.FromResult<Uri?>(null);
		}

		public Task<BlobMetadata> GetObjectMetadataAsync(NamespaceId ns, BlobId blobId, CancellationToken cancellationToken = default)
		{
			try
			{
				string path = GetFilesystemPath(blobId);
				return GetBackend(ns).GetMetadata(path);
			}
			catch (FileNotFoundException)
			{
				throw new BlobNotFoundException(ns, blobId);
			}
		}

		public Task<ulong> CopyBlobAsync(NamespaceId ns, NamespaceId targetNamespace, BlobId blobId, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException();
		}

		public Task<Uri?> PutObjectWithRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not supported
			return Task.FromResult<Uri?>(null);
		}
		public async Task<BlobId> PutObjectAsync(NamespaceId ns, ReadOnlyMemory<byte> content, BlobId blobIdentifier, CancellationToken cancellationToken = default)
		{
			using EpicGames.Core.ReadOnlyMemoryStream stream = new EpicGames.Core.ReadOnlyMemoryStream(content);
			return await PutObjectAsync(ns, stream, blobIdentifier, cancellationToken);
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, Stream content, BlobId blobIdentifier, CancellationToken cancellationToken = default)
		{
			string path = GetFilesystemPath(blobIdentifier);
			await GetBackend(ns).WriteAsync(path, content, cancellationToken);
			return blobIdentifier;
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, byte[] content, BlobId blobIdentifier, CancellationToken cancellationToken = default)
		{
			using MemoryStream stream = new MemoryStream(content);
			return await PutObjectAsync(ns, stream, blobIdentifier, cancellationToken);
		}

		public async Task<BlobContents> GetObjectAsync(NamespaceId ns, BlobId blob, LastAccessTrackingFlags flags, bool supportsRedirectUri = false, CancellationToken cancellationToken = default)
		{
			string path = GetFilesystemPath(blob);

			BlobContents? contents = await GetBackend(ns).TryReadAsync(path, flags, cancellationToken);
			if (contents == null)
			{
				throw new BlobNotFoundException(ns, blob);
			}

			return contents;
		}

		public async Task<bool> ExistsAsync(NamespaceId ns, BlobId blob, bool forceCheck, CancellationToken cancellationToken = default)
		{
			string path = GetFilesystemPath(blob);
			return await GetBackend(ns).ExistsAsync(path, cancellationToken);
		}

		public async Task DeleteObjectAsync(NamespaceId ns, BlobId objectName, CancellationToken cancellationToken = default)
		{
			string path = GetFilesystemPath(objectName);
			await GetBackend(ns).DeleteAsync(path, cancellationToken);
		}

		public async Task DeleteObjectAsync(IEnumerable<NamespaceId> namespaces, BlobId objectName, CancellationToken cancellationToken = default)
		{
			string path = GetFilesystemPath(objectName);
			foreach (NamespaceId ns in namespaces)
			{
				await GetBackend(ns).DeleteAsync(path, cancellationToken);
			}
		}
		
		public Task DeleteNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			DirectoryInfo namespaceDirectory = GetFilesystemPath(ns).ToDirectoryInfo();
			if (namespaceDirectory.Exists)
			{
				namespaceDirectory.Delete(true);
			}

			return Task.CompletedTask;
		}

		public async IAsyncEnumerable<(BlobId, DateTime)> ListObjectsAsync(NamespaceId ns, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			IStorageBackend backend = GetBackend(ns);
			await foreach ((string path, DateTime time) in backend.ListAsync(cancellationToken))
			{
				string name = path.Substring(path.LastIndexOf('/') + 1);
				yield return (new BlobId(name), time);
			}
		}

		public bool ShouldRun()
		{
			return true;
		}

		public async Task<ulong> CleanupAsync(CancellationToken cancellationToken = default)
		{
			return await CleanupInternalAsync(cancellationToken);
		}

		/// <summary>
		/// Clean the store from expired files
		/// 
		/// Uses the configuration to remove the least recently accessed (modified) blobs
		/// </summary>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns></returns>
		public async Task<ulong> CleanupInternalAsync(CancellationToken cancellationToken = default)
		{
			using TelemetrySpan scope = _tracer.StartActiveSpan("gc.filesystem")
				.SetAttribute("operation.name", "gc.filesystem");

			bool perNamespaceGC = _settings.CurrentValue.PerNamespaceGC;
			ulong countOfBlobsRemoved = 0;

			if (perNamespaceGC)
			{
				await foreach (NamespaceId ns in ListNamespaces().WithCancellation(cancellationToken))
				{
					countOfBlobsRemoved += await RunGCAsync(ns, cancellationToken);
				}
			}
			else
			{
				// legacy global GC
				countOfBlobsRemoved += await RunGCAsync(ns: null, cancellationToken);
			}

			return countOfBlobsRemoved;
		}

		private async Task<ulong> RunGCAsync(NamespaceId? ns, CancellationToken cancellationToken = default)
		{
			ulong countOfBlobsRemoved = 0;
			(long triggerSize, long targetSize) = GetTriggerAndTargetSize(ns);

			// Perform a maximum of 5 clean up runs
			for (int i = 0; i < 5; i++)
			{
				long size = await CalculateDiskSpaceUsedAsync(ns);
				string nsString = ns != null ? ns.ToString()! : "All";
				// first check to see if we should trigger at all, this happens for each run but only really matters for the first attempt
				if (size < triggerSize)
				{
					_logger.LogInformation("Filesystem cleanup not running. Disksize used: '{UsedDiskSize}'. Namespace: '{Namespace}'. Trigger size was {TriggerSize}", size, nsString, triggerSize);
					return countOfBlobsRemoved;
				}

				// then check if we have reached the target size, if not we should continue running
				if (size <= targetSize)
				{
					_logger.LogInformation("Filesystem cleanup reached target size. Disksize used: '{UsedDiskSize}'. Namespace: '{Namespace}'. Target size was {TargetSize}", size, nsString, targetSize);

					return countOfBlobsRemoved;
				}

				_logger.LogInformation("Filesystem cleanup running. Disksize used: '{UsedDiskSize}'.Namespace: '{Namespace}'. Trigger size was {TriggerSize}", size, nsString, triggerSize);

				// define progressively shorter windows of how long we keep data around for, based on their last write time
				DateTime[] cutoffPeriods = new DateTime[]
				{
					DateTime.Now.AddDays(-14),
					DateTime.Now.AddDays(-7),
					DateTime.Now.AddDays(-3),
					DateTime.Now.AddDays(-1),
					DateTime.Now.AddHours(-12),
					DateTime.Now /* This is a bit extreme as it will just throw out any object that exists right now, but if we get this far we need to really remove something */
				};

				bool hadFiles = false;
				long totalBytesDeleted = 0;

				foreach (DateTime cutoff in cutoffPeriods)
				{
					IEnumerable<FileInfo> fileInfos = GetObjectsOlderThen(cutoff, ns);

					foreach (FileInfo fi in fileInfos)
					{
						hadFiles = true;
						try
						{
							totalBytesDeleted += fi.Length;
							fi.Delete();
							++countOfBlobsRemoved;

							long currentSize = size - totalBytesDeleted;
							if (currentSize <= targetSize || cancellationToken.IsCancellationRequested)
							{
								return countOfBlobsRemoved;
							}
						}
						catch (FileNotFoundException)
						{
							// if the file was gced while running we can just ignore it
						}
						catch (DirectoryNotFoundException)
						{
							// if the directory was deleted then we can ignore it to, end result is the blob is gone which is what matters
						}
					}
				}

				if (!hadFiles)
				{
					return countOfBlobsRemoved;
				}
			}

			return countOfBlobsRemoved;
		}

		private (long triggerSize, long targetSize) GetTriggerAndTargetSize(NamespaceId? ns)
		{
			if (ns == null)
			{
				ulong maxSizeBytes = _settings.CurrentValue.MaxSizeBytes;
				long triggerSize = (long)(maxSizeBytes * _settings.CurrentValue.TriggerThresholdPercentage);
				long targetSize = (long)(maxSizeBytes * _settings.CurrentValue.TargetThresholdPercentage); // Target to shrink to if triggered

				return (triggerSize, targetSize);
			}
			else
			{
				NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns.Value);

				ulong? maxSizeBytesPerNs = policy.MaxFilesystemStorageBytes;
				ulong maxSizeBytes = maxSizeBytesPerNs ?? _settings.CurrentValue.MaxSizeBytes;

				long triggerSize = (long)(maxSizeBytes * _settings.CurrentValue.TriggerThresholdPercentage);
				long targetSize = (long)(maxSizeBytes * _settings.CurrentValue.TargetThresholdPercentage); // Target to shrink to if triggered
				return (triggerSize, targetSize);
			}
		}

		public IEnumerable<FileInfo> GetObjectsOlderThen(DateTime cutoff, NamespaceId? ns = null)
		{
			string path = ns != null ? Path.Combine(GetRootDir(), ns.ToString()!) : GetRootDir();
			DirectoryInfo di = new DirectoryInfo(path);
			if (!di.Exists)
			{
				return Array.Empty<FileInfo>();
			}
			return di.EnumerateFiles("*", SearchOption.AllDirectories).Where(x => x.LastWriteTime < cutoff);
		}

		/// <summary>
		/// Get least recently accessed objects
		/// Assumes files on disk have the their last access timestamp up-to-date
		/// </summary>
		/// <param name="ns">Namespace, if set to null all namespaces will be scanned</param>
		/// <param name="maxResults">Max results to return. Note that the entire namespace will be scanned no matter what.</param>
		/// <param name="maxCountOfObjectsScanned">Max count of objects scanned before we stop.</param>
		/// <returns>Enumerable of least recently accessed objects as FileInfos</returns>
		public IEnumerable<FileInfo> GetLeastRecentlyAccessedObjects(NamespaceId? ns = null, int maxResults = 10_000, int maxCountOfObjectsScanned = 40_000_000)
		{
			// TODO: The maxCountOfObjectsScanned is not a ideal solution, we should likely find a solution were we do not have to read all objects into memory like this but rather can scan over them to determine a reasonable last write time cutoff
			string path = ns != null ? Path.Combine(GetRootDir(), ns.ToString()!) : GetRootDir();
			DirectoryInfo di = new DirectoryInfo(path);
			if (!di.Exists)
			{
				return Array.Empty<FileInfo>();
			}

			return di.EnumerateFiles("*", SearchOption.AllDirectories).Take(maxCountOfObjectsScanned).OrderBy(x => x.LastWriteTime).Take(maxResults);
		}

		/// <summary>
		/// Calculate the total size of blobs on disk for given namespace
		/// </summary>
		/// <param name="ns">Namespace, if set to null the total size of all namespaces will be returned</param>
		/// <returns>Total size of blobs in bytes</returns>
		public async Task<long> CalculateDiskSpaceUsedAsync(NamespaceId? ns = null)
		{
			using TelemetrySpan scope = _tracer.StartActiveSpan("gc.filesystem.calc_disc_usage")
				.SetAttribute("operation.name", "gc.filesystem.calc_disc_usage");

			string path = ns != null ? Path.Combine(GetRootDir(), ns.ToString()!) : GetRootDir();
			DirectoryInfo di = new DirectoryInfo(path);
			if (!di.Exists)
			{
				return 0;
			}

			return await Task.Run(() => di.EnumerateFiles("*", SearchOption.AllDirectories).Sum(x =>
			{
				try
				{
					return x.Length;
				}
				catch (FileNotFoundException)
				{
					// if the file has been gced we just ignore it
					return 0;
				}
			}));
		}

		public IAsyncEnumerable<NamespaceId> ListNamespaces()
		{
			DirectoryInfo di = new DirectoryInfo(GetRootDir());
			if (!di.Exists)
			{
				return AsyncEnumerable.Empty<NamespaceId>();
			}

			return di.GetDirectories().Select(x => new NamespaceId(x.Name)).ToAsyncEnumerable();
		}

		public async Task MoveBufferedPayloadAsync(NamespaceId ns, BlobId blobId, FilesystemBufferedPayload bufferedPayload, CancellationToken cancellationToken)
		{
			string path = GetFilesystemPath(blobId);
			await GetBackend(ns).MoveBufferedPayloadAsync(path, bufferedPayload, cancellationToken);
		}
	}

	public class FileStorageBackend : IStorageBackend
	{
		private readonly ILogger _logger;
		private readonly DirectoryReference _baseDir;
		private readonly NamespaceId _namespaceId;
		private readonly IOptionsMonitor<FilesystemSettings> _settings;

		private const int DefaultBufferSize = 4096;

		public FileStorageBackend(DirectoryReference baseDir, NamespaceId ns, ILogger<FileStorageBackend> logger, IOptionsMonitor<FilesystemSettings> settings)
		{
			_logger = logger;
			_baseDir = baseDir;
			_namespaceId = ns;
			_settings = settings;
		}

		private string GetRootDir()
		{
			return PathUtil.ResolvePath(_settings.CurrentValue.RootDir);
		}

		public static FileInfo GetFilesystemPath(string rootDir, NamespaceId ns, BlobId blob)
		{
			const int CountOfCharactersPerDirectory = 2;
			string objectName = blob.ToString();
			string firstPart = objectName.Substring(0, CountOfCharactersPerDirectory);
			string secondPart = objectName.Substring(CountOfCharactersPerDirectory, CountOfCharactersPerDirectory);
			string fileName = objectName;

			return new FileInfo(Path.Combine(rootDir, ns.ToString(), firstPart, secondPart, fileName));
		}

		public FileInfo GetFilesystemPath(string path)
		{
			return FileReference.Combine(_baseDir, path).ToFileInfo();
		}

		public DirectoryInfo GetFilesystemPath(NamespaceId ns)
		{
			return new DirectoryInfo(Path.Combine(GetRootDir(), ns.ToString()));
		}

		static readonly string s_processSuffix = Guid.NewGuid().ToString();
		static int s_uniqueId = 0;

		public async Task WriteAsync(string path, Stream content, CancellationToken cancellationToken = default)
		{
			FileInfo filePath = GetFilesystemPath(path);
			filePath.Directory?.Create();

			if (!File.Exists(filePath.FullName))
			{
				int uniqueId = Interlocked.Increment(ref s_uniqueId);

				string tempFilePath = $"{filePath.FullName}.{s_processSuffix}.{uniqueId}";

				await using (FileStream fs = new FileStream(tempFilePath, FileMode.Create, FileAccess.Write, FileShare.Read, DefaultBufferSize, FileOptions.Asynchronous | FileOptions.SequentialScan))
				{
					await content.CopyToAsync(fs, cancellationToken);
				}

				try
				{
					File.Move(tempFilePath, filePath.FullName, true);
				}
				catch (IOException) when (File.Exists(filePath.FullName))
				{
				}
				catch (UnauthorizedAccessException) when (File.Exists(filePath.FullName))
				{
				}

				filePath.Refresh();

				if (filePath.Length == 0)
				{
					_logger.LogWarning("0 byte file written as {Path} {Method}", path, "Stream");
				}
			}

			UpdateLastWriteTime(filePath.FullName, DateTime.UnixEpoch);
		}

		/// <summary>
		/// Update the last modified/write time that is used for determining when file was last accessed
		/// 
		/// Using access time is tricky as many file systems disable that for performance reasons.
		/// A new blob written to disk should be set with the oldest possible write time.
		/// This will ensure sorting of least recently accessed files during garbage collection works as intended.
		/// The write time update will happen async without any waiting to prevent blocking the critical path
		/// as it's best-effort only.
		/// </summary>
		/// <param name="filePath"></param>
		/// <param name="lastAccessed">Time the file was last accessed</param>
		private static void UpdateLastWriteTime(string filePath, DateTime lastAccessed)
		{
			try
			{
				File.SetLastWriteTimeUtc(filePath, lastAccessed);
			}
			catch (FileNotFoundException)
			{
				// it is okay if the file does not exist anymore, that just means it got gced
			}
		}

		public Task<BlobContents?> TryReadAsync(string path, LastAccessTrackingFlags flags, CancellationToken cancellationToken = default)
		{
			FileInfo filePath = GetFilesystemPath(path);

			if (!File.Exists(filePath.FullName))
			{
				return Task.FromResult<BlobContents?>(null);
			}

			if (flags == LastAccessTrackingFlags.DoTracking)
			{
				UpdateLastWriteTime(filePath.FullName, DateTime.UtcNow);
			}
			FileStream fs = new FileStream(filePath.FullName, FileMode.Open, FileAccess.Read, FileShare.Read, DefaultBufferSize, FileOptions.Asynchronous | FileOptions.SequentialScan);

			return Task.FromResult<BlobContents?>(new BlobContents(fs, fs.Length, localPath: $"{_namespaceId}/{path}", absolutePath: filePath.FullName));
		}

		public Task<bool> ExistsAsync(string path, CancellationToken cancellationToken = default)
		{
			FileInfo filePath = GetFilesystemPath(path);

			return Task.FromResult(File.Exists(filePath.FullName));
		}

		public Task DeleteAsync(string path, CancellationToken cancellationToken = default)
		{
			FileInfo filePath = GetFilesystemPath(path);
			if (File.Exists(filePath.FullName))
			{
				filePath.Delete();
			}
			return Task.CompletedTask;
		}

		public IAsyncEnumerable<(string, DateTime)> ListAsync(CancellationToken cancellationToken = default)
		{
			return DoListOldObjects().ToAsyncEnumerable();
		}

		private IEnumerable<(string, DateTime)> DoListOldObjects()
		{
			DirectoryInfo di = _baseDir.ToDirectoryInfo();
			if (!di.Exists)
			{
				yield break;
			}

			foreach (FileInfo file in di.EnumerateFiles("*", SearchOption.AllDirectories))
			{
				string path = new FileReference(file).MakeRelativeTo(_baseDir).Replace(Path.DirectorySeparatorChar, '/');
				yield return (path, file.LastWriteTime);
			}
		}

		public Task<BlobMetadata> GetMetadata(string path)
		{
			FileInfo fileInfo = GetFilesystemPath(path);
			return Task.FromResult(new BlobMetadata(fileInfo.Length, fileInfo.CreationTime));
		}

		public Task MoveBufferedPayloadAsync(string path, FilesystemBufferedPayload bufferedPayload, CancellationToken cancellationToken)
		{
			FileInfo fileInfo = GetFilesystemPath(path);
			if (cancellationToken.IsCancellationRequested)
			{
				return Task.CompletedTask;
			}
			
			fileInfo.Directory?.Create();

			try
			{
				File.Move(bufferedPayload.TempFile.FullName, fileInfo.FullName, true);
			}
			catch (IOException) when (File.Exists(fileInfo.FullName))
			{
			}

			UpdateLastWriteTime(fileInfo.FullName, DateTime.UnixEpoch);

			return Task.CompletedTask;
		}
	}
}
