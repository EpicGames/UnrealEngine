// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation.Blob
{
	public class BucketStats
	{
		public NamespaceId Namespace { get; set; }
		public BucketId Bucket { get; set; }
		public long CountOfRefs { get; set; }
		public long CountOfBlobs { get; set; }
		public long TotalSize { get; set; }
		public double AvgSize { get; set; }
		public long LargestBlob { get; set; }
		public long SmallestBlobFound { get; set; }
	}

	public interface IBlobIndex
	{
		Task AddBlobToIndexAsync(NamespaceId ns, BlobId id, ulong rawSize, string? region = null, CancellationToken cancellationToken = default);

		Task RemoveBlobFromRegionAsync(NamespaceId ns, BlobId id, string? region = null, CancellationToken cancellationToken = default);
		Task RemoveBlobFromAllRegionsAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken = default);
		Task<bool> BlobExistsInRegionAsync(NamespaceId ns, BlobId blobIdentifier, string? region = null, CancellationToken cancellationToken = default);
		IAsyncEnumerable<(NamespaceId, BlobId)> GetAllBlobsAsync(CancellationToken cancellationToken = default);

		IAsyncEnumerable<(NamespaceId, BaseBlobReference)> GetAllBlobReferencesAsync(CancellationToken cancellationToken = default);
		IAsyncEnumerable<BaseBlobReference> GetBlobReferencesAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken = default);
		Task AddRefToBlobsAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId[] blobs, CancellationToken cancellationToken = default);

		Task RemoveReferencesAsync(NamespaceId ns, BlobId id, List<BaseBlobReference>? referencesToRemove, CancellationToken cancellationToken = default);
		Task<List<string>> GetBlobRegionsAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default);
		Task AddBlobReferencesAsync(NamespaceId ns, BlobId blob, BlobId blobThatReferences, CancellationToken cancellationToken = default);

		Task AddBlobToBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobId, long blobSize, CancellationToken cancellationToken = default);
		Task RemoveBlobFromBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, List<BlobId> blobIds, CancellationToken cancellationToken = default);
		Task<BucketStats> CalculateBucketStatisticsAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken = default);
	}

	public abstract class BaseBlobReference : IEquatable<BaseBlobReference>
	{
		protected BaseBlobReference(BlobId sourceBlob)
		{
			SourceBlob = sourceBlob;
		}

		public BlobId SourceBlob { get; init; }

		public override bool Equals(object? o)
		{
			if (o is BaseBlobReference baseBlob)
			{
				return Equals(baseBlob);
			}

			return false;
		}

		public override int GetHashCode()
		{
			return SourceBlob.GetHashCode();
		}

		public virtual bool Equals(BaseBlobReference? other)
		{
			if (other != null && other.SourceBlob.Equals(SourceBlob))
			{
				return true;
			}

			return false;
		}
	}

	public class RefBlobReference : BaseBlobReference
	{
		public RefBlobReference(BlobId sourceBlob, BucketId bucket, RefId key) : base(sourceBlob)
		{
			Bucket = bucket;
			Key = key;
		}

		public BucketId Bucket { get; set; }
		public RefId Key { get; set; }
		public override bool Equals(BaseBlobReference? other)
		{
			if (other is RefBlobReference otherRefBlob)
			{
				return otherRefBlob.Key.Equals(Key) && otherRefBlob.Bucket.Equals(Bucket) && otherRefBlob.SourceBlob.Equals(SourceBlob);
			}

			return false;
		}
	}

	public class BlobToBlobReference : BaseBlobReference
	{
		public BlobToBlobReference(BlobId sourceBlob, BlobId blob) : base(sourceBlob)
		{
			Blob = blob;
		}

		public BlobId Blob { get; set; }
		public override bool Equals(BaseBlobReference? other)
		{
			if (other is BlobToBlobReference otherBlobToBlob)
			{
				return otherBlobToBlob.Blob.Equals(Blob) && otherBlobToBlob.SourceBlob.Equals(SourceBlob);
			}

			return false;
		}
	}
}
