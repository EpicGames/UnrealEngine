// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Net.Mime;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Core;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Microsoft.Extensions.DependencyInjection;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation;

public interface IBlobService
{
	Task<ContentHash> VerifyContentMatchesHashAsync(Stream content, ContentHash identifier, CancellationToken cancellationToken = default);
	Task<BlobId> PutObjectKnownHashAsync(NamespaceId ns, IBufferedPayload content, BlobId identifier, BucketId? bucketHint = null, bool? bypassCache = null, CancellationToken cancellationToken = default);
	Task<BlobId> PutObjectAsync(NamespaceId ns, IBufferedPayload payload, BlobId identifier, BucketId? bucketHint = null, bool? bypassCache = null, CancellationToken cancellationToken = default);
	Task<BlobId> PutObjectAsync(NamespaceId ns, byte[] payload, BlobId identifier, BucketId? bucketHint = null, bool? bypassCache = null, CancellationToken cancellationToken = default);
	Task<Uri?> MaybePutObjectWithRedirectAsync(NamespaceId ns, BlobId identifier, BucketId? bucketHint = null, CancellationToken cancellationToken = default);

	Task<BlobContents> GetObjectAsync(NamespaceId ns, BlobId blob, List<string>? storageLayers = null, bool supportsRedirectUri = false, bool allowOndemandReplication = true, BucketId? bucketHint = null, bool bypassCache = false, CancellationToken cancellationToken = default);

	Task<Uri?> GetObjectWithRedirectAsync(NamespaceId ns, BlobId blobIdentifier, List<string>? storageLayers = null, CancellationToken cancellationToken = default);

	Task<BlobMetadata> GetObjectMetadataAsync(NamespaceId ns, BlobId blobId, CancellationToken cancellationToken = default);

	Task<BlobContents> ReplicateObjectAsync(NamespaceId ns, BlobId blob, bool supportsRedirectUri = false, bool force = false, BucketId? bucketHint = null, CancellationToken cancellationToken = default);

	Task<bool> ExistsAsync(NamespaceId ns, BlobId blob, List<string>? storageLayers = null, bool ignoreRemoteBlobs = false, CancellationToken cancellationToken = default);

	/// <summary>
	/// Checks that the blob exists in the root store, the store which is last in the list and thus is intended to have every blob in it
	/// </summary>
	/// <param name="ns">The namespace</param>
	/// <param name="blob">The identifier of the blob</param>
	/// <param name="cancellationToken"></param>
	/// <returns></returns>
	Task<bool> ExistsInRootStoreAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default);

	// Delete a object
	Task DeleteObjectAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default);

	/// <summary>
	/// Delete the object from multiple namespaces at once, extra efficient if they are all in the same storage pool
	/// </summary>
	/// <param name="namespaces">The namespaces to delete the blob from</param>
	/// <param name="blob">The identifier of the blob to delete</param>
	/// <param name="cancellationToken"></param>
	Task DeleteObjectAsync(List<NamespaceId> namespaces, BlobId blob, CancellationToken cancellationToken = default);

	// delete the whole namespace
	Task DeleteNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default);

	IAsyncEnumerable<(BlobId, DateTime)> ListObjectsAsync(NamespaceId ns, CancellationToken cancellationToken = default);
	Task<BlobId[]> FilterOutKnownBlobsAsync(NamespaceId ns, IEnumerable<BlobId> blobs, CancellationToken cancellationToken = default);
	Task<BlobId[]> FilterOutKnownBlobsAsync(NamespaceId ns, IAsyncEnumerable<BlobId> blobs, CancellationToken cancellationToken = default);
	Task<BlobContents> GetObjectsAsync(NamespaceId ns, BlobId[] refRequestBlobReferences, CancellationToken cancellationToken = default);

	bool ShouldFetchBlobOnDemand(NamespaceId ns);

	bool IsMultipartUploadSupported(NamespaceId ns);
	Task<(string?, string?)> StartMultipartUploadAsync(NamespaceId ns, CancellationToken cancellationToken = default);
	Task CompleteMultipartUploadAsync(NamespaceId ns, string blobName, string uploadId, List<string> partIds, CancellationToken cancellationToken = default);
	Task PutMultipartUploadAsync(NamespaceId ns, string blobName, string uploadId, string partIdentifier, byte[] blobData, CancellationToken cancellationToken = default);
	Task<Uri?> MaybePutMultipartUploadWithRedirectAsync(NamespaceId ns, string blobName, string uploadId, string partId, CancellationToken cancellationToken = default);
	Task<(ContentId, BlobId)> VerifyMultipartUpload(NamespaceId ns, BlobId blobId, string blobName, bool isCompressed, CancellationToken cancellationToken = default);
	List<MultipartByteRange> GetMultipartRanges(NamespaceId ns, string blobName, string uploadId, ulong blobLength, CancellationToken cancellationToken = default);
	MultipartLimits? GetMultipartLimits(NamespaceId ns);
	Task CopyBlobAsync(NamespaceId ns, NamespaceId targetNamespace, BlobId blobId, BucketId? bucketHint = null, CancellationToken cancellationToken = default);
}

public class BlobMetadata
{
	public BlobMetadata(long length, DateTime creationTime)
	{
		Length = length;
		CreationTime = creationTime;
	}

	public long Length { get; set; }
	public DateTime CreationTime { get; set; }
}

public static class BlobServiceExtensions
{
	public static async Task<(ContentId, BlobId)> PutCompressedObjectMetadataAsync(this IBlobService blobService, NamespaceId ns, IBufferedPayload payload, ContentId? id, IServiceProvider provider, CancellationToken cancellationToken)
	{
		IContentIdStore contentIdStore = provider.GetService<IContentIdStore>()!;
		CompressedBufferUtils compressedBufferUtils = provider.GetService<CompressedBufferUtils>()!;
		Tracer tracer = provider.GetService<Tracer>()!;

		await using Stream payloadStream = payload.GetStream();
		// decompress the content and generate an identifier from it to verify the identifier we got
		(IBufferedPayload, IoHash) decompressResult = await compressedBufferUtils.DecompressContentAsync(payloadStream, (ulong)payload.Length, cancellationToken);
		using IBufferedPayload bufferedPayload = decompressResult.Item1;
		IoHash rawHash = decompressResult.Item2;
		await using Stream decompressedStream = bufferedPayload.GetStream();

		ulong decompressedSize = (ulong)decompressedStream.Length;
		ContentId identifierDecompressedPayload;
		if (id != null)
		{
			identifierDecompressedPayload = ContentId.FromContentHash(await blobService.VerifyContentMatchesHashAsync(decompressedStream, id, cancellationToken));
		}
		else
		{
			ContentHash blobHash;
			{
				using TelemetrySpan _ = tracer.StartActiveSpan("web.hash").SetAttribute("operation.name", "web.hash");
				blobHash = await BlobId.FromStreamAsync(decompressedStream, cancellationToken);
			}

			identifierDecompressedPayload = ContentId.FromContentHash(blobHash);
		}

		ContentId rawHashCid = ContentId.FromIoHash(rawHash);
		if (!identifierDecompressedPayload.Equals(rawHashCid))
		{
			throw new HashMismatchException(identifierDecompressedPayload, rawHashCid);
		}
		BlobId identifierCompressedPayload;
		{
			using TelemetrySpan _ = tracer.StartActiveSpan("web.hash").SetAttribute("operation.name", "web.hash");
			await using Stream hashStream = payload.GetStream();
			identifierCompressedPayload = await BlobId.FromStreamAsync(hashStream, cancellationToken);
		}

		// commit the mapping from the decompressed hash to the compressed hash, we run this in parallel with the blob store submit
		// TODO: let users specify weight of the blob compared to previously submitted content ids
		ulong compressedSize = (ulong)payload.Length;
		await contentIdStore.PutAsync(ns, identifierDecompressedPayload, new BlobId[] {identifierCompressedPayload}, compressedSize, decompressedSize, cancellationToken);

		return (identifierDecompressedPayload, identifierCompressedPayload);
	}

	public static async Task<ContentId> PutCompressedObjectAsync(this IBlobService blobService, NamespaceId ns, IBufferedPayload payload, ContentId? id, IServiceProvider provider, CancellationToken cancellationToken, BucketId? bucketHint = null, bool? bypassCache = null)
	{
		(ContentId cid, BlobId identifierCompressedPayload) = await PutCompressedObjectMetadataAsync(blobService, ns, payload, id, provider, cancellationToken);
		// we still commit the compressed buffer to the object store using the hash of the compressed content
		{
			await blobService.PutObjectKnownHashAsync(ns, payload, identifierCompressedPayload, bucketHint, bypassCache: bypassCache, cancellationToken: cancellationToken);
		}

		return cid;
	}

	public static async Task<(BlobContents, string, BlobId?)> GetCompressedObjectAsync(this IBlobService blobService, NamespaceId ns, ContentId contentId, IServiceProvider provider, bool supportsRedirectUri = false, CancellationToken cancellationToken = default)
	{
		IContentIdStore contentIdStore = provider.GetService<IContentIdStore>()!;
		Tracer tracer = provider.GetService<Tracer>()!;

		BlobId[]? chunks = await contentIdStore.ResolveAsync(ns, contentId, mustBeContentId: false, cancellationToken);
		if (chunks == null || chunks.Length == 0)
		{
			throw new ContentIdResolveException(contentId);
		}

		// single chunk, we just return that chunk
		if (chunks.Length == 1)
		{
			BlobId blobToReturn = chunks[0];
			string mimeType = CustomMediaTypeNames.UnrealCompressedBuffer;
			if (contentId.Equals(blobToReturn))
			{
				// this was actually the unmapped blob, meaning its not a compressed buffer
				mimeType = MediaTypeNames.Application.Octet;
			}

			return (await blobService.GetObjectAsync(ns, blobToReturn, supportsRedirectUri: supportsRedirectUri, cancellationToken: cancellationToken), mimeType, blobToReturn);
		}

		// chunked content, combine the chunks into a single stream
		using TelemetrySpan _ = tracer.StartActiveSpan("blob.combine").SetAttribute("operation.name", "blob.combine");
		Task<BlobContents>[] tasks = new Task<BlobContents>[chunks.Length];
		for (int i = 0; i < chunks.Length; i++)
		{
			// even if it was requested to support redirect, since we need to combine the chunks using redirects is not possible
			tasks[i] = blobService.GetObjectAsync(ns, chunks[i], supportsRedirectUri: false, cancellationToken: cancellationToken);
		}

		MemoryStream ms = new MemoryStream();
		foreach (Task<BlobContents> task in tasks)
		{
			BlobContents blob = await task;
			await using Stream s = blob.Stream;
			await s.CopyToAsync(ms, cancellationToken);
		}

		ms.Seek(0, SeekOrigin.Begin);

		// TODO: A chunked blob does not store the hash of the full object so we are unable to return it, only used for a header so okay for now but should be fixed
		// chunking could not have happened for a non compressed buffer so assume it is compressed
		return (new BlobContents(ms, ms.Length), CustomMediaTypeNames.UnrealCompressedBuffer, null);
	}
}

public class HashMismatchException : Exception
{
	public ContentHash SuppliedHash { get; }
	public ContentHash ContentHash { get; }

	public HashMismatchException(ContentHash suppliedHash, ContentHash contentHash) : base($"ID was not a hash of the content uploaded. Supplied hash was: {suppliedHash} but hash of content was {contentHash}")
	{
		SuppliedHash = suppliedHash;
		ContentHash = contentHash;
	}
}