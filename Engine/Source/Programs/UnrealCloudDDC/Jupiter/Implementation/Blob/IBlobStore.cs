// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation
{
	public enum LastAccessTrackingFlags
	{
		DoTracking = 0,
		SkipTracking = 1,
	}

	public interface IBlobStore
	{
		Task<BlobId> PutObjectAsync(NamespaceId ns, byte[] blob, BlobId identifier, CancellationToken cancellationToken = default);
		Task<BlobId> PutObjectAsync(NamespaceId ns, ReadOnlyMemory<byte> blob, BlobId identifier, CancellationToken cancellationToken = default);
		Task<BlobId> PutObjectAsync(NamespaceId ns, Stream content, BlobId identifier, CancellationToken cancellationToken = default);

		Task<BlobContents> GetObjectAsync(NamespaceId ns, BlobId blob, LastAccessTrackingFlags flags = LastAccessTrackingFlags.DoTracking, bool supportsRedirectUri = false, CancellationToken cancellationToken = default);
		Task<bool> ExistsAsync(NamespaceId ns, BlobId blob, bool forceCheck = false, CancellationToken cancellationToken = default);

		// Delete a object
		Task DeleteObjectAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default);

		// Delete a object from multiple namespaces at once
		Task DeleteObjectAsync(IEnumerable<NamespaceId> ns, BlobId blob, CancellationToken cancellationToken = default);

		// delete the whole namespace
		Task DeleteNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default);

		IAsyncEnumerable<(BlobId, DateTime)> ListObjectsAsync(NamespaceId ns, CancellationToken cancellationToken = default);
		Task<Uri?> PutObjectWithRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default);
		Task<Uri?> GetObjectByRedirectAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default);
		Task<BlobMetadata> GetObjectMetadataAsync(NamespaceId ns, BlobId blobId, CancellationToken cancellationToken = default);

		Task<ulong> CopyBlobAsync(NamespaceId ns, NamespaceId targetNamespace, BlobId blobId, CancellationToken cancellationToken = default);
	}

	public interface IMultipartBlobStore
	{
		Task<string> StartMultipartUploadAsync(NamespaceId ns, string blobName, CancellationToken cancellationToken = default);
		Task CompleteMultipartUploadAsync(NamespaceId ns, string path, string uploadId, List<string> partIds, CancellationToken cancellationToken = default);
		Task PutMultipartPartAsync(NamespaceId ns, string blobName, string uploadId, string partIdentifier, byte[] blob, CancellationToken cancellationToken = default);
		Task<Uri?> GetWriteRedirectForPartAsync(NamespaceId ns, string blobName, string uploadId, string partIdentifier, CancellationToken cancellationToken = default);
		Task<BlobContents?> GetMultipartObjectByNameAsync(NamespaceId ns, string blobName, CancellationToken cancellationToken = default);
		Task RenameMultipartBlobAsync(NamespaceId ns, string blobName, BlobId blobId, CancellationToken cancellationToken = default);
		List<MultipartByteRange> GetMultipartRanges(NamespaceId ns, string uploadId, ulong blobLength, CancellationToken cancellationToken = default);
		MultipartLimits GetMultipartLimits(NamespaceId ns, CancellationToken cancellationToken = default);
	}

	public class MissingMultipartPartsException : Exception
	{
		public MissingMultipartPartsException(List<string> missingParts)
		{
			MissingParts = missingParts;
		}

		public List<string> MissingParts { get; init; }
	}

	public record MultipartLimits
	{
		public uint MinChunkSize { get; set; }
		public uint IdealChunkSize { get; set; }
		public int MaxCountOfChunks { get; set; }
	}
	public record MultipartByteRange
	{
		public ulong FirstByte { get; set; }
		public ulong LastByte { get; set; }
		public string PartId { get; set; } = null!;
	}

	public interface IStorageBackend
	{
		Task WriteAsync(string path, Stream content, CancellationToken cancellationToken = default);
		Task<BlobContents?> TryReadAsync(string path, LastAccessTrackingFlags flags = LastAccessTrackingFlags.DoTracking, CancellationToken cancellationToken = default);
		Task<bool> ExistsAsync(string path, CancellationToken cancellationToken = default);
		Task DeleteAsync(string path, CancellationToken cancellationToken = default);
		IAsyncEnumerable<(string, DateTime)> ListAsync(CancellationToken cancellationToken = default);
	}

	public class BlobNotFoundException : Exception
	{
		public NamespaceId Ns { get; }
		public BlobId Blob { get; }

		public BlobNotFoundException(NamespaceId ns, BlobId blob) : base($"No Blob in Namespace {ns} with id {blob}")
		{
			Ns = ns;
			Blob = blob;
		}

		public BlobNotFoundException(NamespaceId ns, BlobId blob, string message) : base(message)
		{
			Ns = ns;
			Blob = blob;
		}
	}

	public class BlobReplicationException : BlobNotFoundException
	{
		public BlobReplicationException(NamespaceId ns, BlobId blob, string message) : base(ns, blob, message)
		{
		}
	}

	public class BlobToLargeException : Exception
	{
		public BlobId Blob { get; }

		public BlobToLargeException(BlobId blob) : base($"Blob {blob} was to large to cache")
		{
			Blob = blob;
		}
	}

	public class ResourceHasToManyRequestsException : Exception
	{
		public ResourceHasToManyRequestsException(Exception originalException) : base($"To many requests to resource", originalException)
		{
		}
	}
}
