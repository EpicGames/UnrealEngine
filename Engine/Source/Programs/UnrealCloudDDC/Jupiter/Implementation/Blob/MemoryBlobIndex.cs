// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation.Blob;

public class MemoryBlobIndex : IBlobIndex
{
	private class MemoryBlobInfo
	{
		public HashSet<string> Regions { get; init; } = new HashSet<string>();
		public NamespaceId Namespace { get; init; }
		public BlobId BlobIdentifier { get; init; } = null!;
		public List<BaseBlobReference> References { get; init; } = new List<BaseBlobReference>();
	}

	private class MemoryBucketInfo
	{
		public List<(RefId, BlobId, long)> Entries { get; init; } = new List<(RefId, BlobId, long)>();

		public void AddEntry(RefId refId, BlobId blobId, long value)
		{
			int index = Entries.FindIndex(tuple => tuple.Item1.Equals(refId) && tuple.Item2.Equals(blobId));
			if (index == -1)
			{
				// no previous value found, add this entry
				Entries.Add((refId, blobId, value));
			}
			else
			{
				// replace existing value
				Entries[index] = (refId, blobId, value);
			}
		}

		public void RemoveEntriesForRef(RefId refId)
		{
			Entries.RemoveAll(tuple => tuple.Item1.Equals(refId));
		}
	}

	private readonly ConcurrentDictionary<NamespaceId, ConcurrentDictionary<BlobId, MemoryBlobInfo>> _index = new();
	private readonly ConcurrentDictionary<NamespaceId, ConcurrentDictionary<BucketId, MemoryBucketInfo>> _bucketIndex = new();
	private readonly IOptionsMonitor<JupiterSettings> _jupiterSettings;

	public MemoryBlobIndex(IOptionsMonitor<JupiterSettings> settings)
	{
		_jupiterSettings = settings;
	}

	private ConcurrentDictionary<BlobId, MemoryBlobInfo> GetNamespaceContainer(NamespaceId ns)
	{
		return _index.GetOrAdd(ns, id => new ConcurrentDictionary<BlobId, MemoryBlobInfo>());
	}

	public Task AddBlobToIndexAsync(NamespaceId ns, BlobId id, ulong rawSize, string? region = null, CancellationToken cancellationToken = default)
	{
		region ??= _jupiterSettings.CurrentValue.CurrentSite;
		ConcurrentDictionary<BlobId, MemoryBlobInfo> index = GetNamespaceContainer(ns);
		index[id] = NewBlobInfo(ns, id, region);
		return Task.CompletedTask;
	}

	private Task<MemoryBlobInfo?> GetBlobInfo(NamespaceId ns, BlobId id)
	{
		ConcurrentDictionary<BlobId, MemoryBlobInfo> index = GetNamespaceContainer(ns);

		if (!index.TryGetValue(id, out MemoryBlobInfo? blobInfo))
		{
			return Task.FromResult<MemoryBlobInfo?>(null);
		}

		return Task.FromResult<MemoryBlobInfo?>(blobInfo);
	}

	public Task RemoveBlobFromRegionAsync(NamespaceId ns, BlobId id, string? region = null, CancellationToken cancellationToken = default)
	{
		region ??= _jupiterSettings.CurrentValue.CurrentSite;
		ConcurrentDictionary<BlobId, MemoryBlobInfo> index = GetNamespaceContainer(ns);

		index.AddOrUpdate(id, _ =>
		{
			MemoryBlobInfo info = NewBlobInfo(ns, id, region);
			return info;
		}, (_, info) =>
		{
			info.Regions.Remove(region);
			return info;
		});
		return Task.CompletedTask;
	}

	public Task RemoveBlobFromAllRegionsAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken = default)
	{
		ConcurrentDictionary<BlobId, MemoryBlobInfo> index = GetNamespaceContainer(ns);

		index.Remove(id, out MemoryBlobInfo? _);
		return Task.CompletedTask;
	}

	public async Task<bool> BlobExistsInRegionAsync(NamespaceId ns, BlobId blobIdentifier, string? region = null, CancellationToken cancellationToken = default)
	{
		string expectedRegion = region ?? _jupiterSettings.CurrentValue.CurrentSite;
		MemoryBlobInfo? blobInfo = await GetBlobInfo(ns, blobIdentifier);
		return blobInfo?.Regions.Contains(expectedRegion) ?? false;
	}

	public async IAsyncEnumerable<(NamespaceId, BaseBlobReference)> GetAllBlobReferencesAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
	{
		await Task.CompletedTask;

		foreach (KeyValuePair<NamespaceId, ConcurrentDictionary<BlobId, MemoryBlobInfo>> namespacePair in _index)
		{
			foreach (KeyValuePair<BlobId, MemoryBlobInfo> blobPair in namespacePair.Value)
			{
				foreach (BaseBlobReference reference in blobPair.Value.References)
				{
					yield return (namespacePair.Key, reference);
				}
			}
		}
	}

	public async IAsyncEnumerable<BaseBlobReference> GetBlobReferencesAsync(NamespaceId ns, BlobId id, [EnumeratorCancellation] CancellationToken cancellationToken)
	{
		MemoryBlobInfo? blobInfo = await GetBlobInfo(ns, id);

		if (blobInfo != null)
		{
			foreach (BaseBlobReference reference in blobInfo.References)
			{
				yield return reference;
			}
		}
	}

	public Task AddRefToBlobsAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId[] blobs, CancellationToken cancellationToken)
	{
		foreach (BlobId id in blobs)
		{
			ConcurrentDictionary<BlobId, MemoryBlobInfo> index = GetNamespaceContainer(ns);

			index.AddOrUpdate(id, _ =>
			{
				MemoryBlobInfo info = NewBlobInfo(ns, id, _jupiterSettings.CurrentValue.CurrentSite);
				info.References!.Add(new RefBlobReference(id, bucket, key));
				return info;
			}, (_, info) =>
			{
				info.References!.Add(new RefBlobReference(id, bucket, key));
				return info;
			});
		}

		return Task.CompletedTask;
	}

	public async IAsyncEnumerable<(NamespaceId, BlobId)> GetAllBlobsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
	{
		await Task.CompletedTask;

		foreach (KeyValuePair<NamespaceId, ConcurrentDictionary<BlobId, MemoryBlobInfo>> pair in _index)
		{
			foreach ((BlobId? _, MemoryBlobInfo? blobInfo) in pair.Value)
			{
				yield return (blobInfo.Namespace, blobInfo.BlobIdentifier);
			}
		}
	}

	public Task RemoveReferencesAsync(NamespaceId ns, BlobId id, List<BaseBlobReference>? referencesToRemove, CancellationToken cancellationToken)
	{
		ConcurrentDictionary<BlobId, MemoryBlobInfo> index = GetNamespaceContainer(ns);

		if (index.TryGetValue(id, out MemoryBlobInfo? blobInfo))
		{
			if (referencesToRemove == null)
			{
				blobInfo.References.Clear();
			}
			else
			{
				foreach (BaseBlobReference r in referencesToRemove)
				{
					blobInfo.References.Remove(r);
				}
			}
		}

		return Task.CompletedTask;
	}

	public async Task<List<string>> GetBlobRegionsAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken)
	{
		MemoryBlobInfo? blobInfo = await GetBlobInfo(ns, blob);

		if (blobInfo != null)
		{
			return blobInfo.Regions.ToList();
		}

		throw new BlobNotFoundException(ns, blob);
	}

	public async Task AddBlobReferencesAsync(NamespaceId ns, BlobId sourceBlob, BlobId targetBlob, CancellationToken cancellationToken)
	{
		MemoryBlobInfo? blobInfo = await GetBlobInfo(ns, sourceBlob);

		if (blobInfo == null)
		{
			throw new BlobNotFoundException(ns, sourceBlob);
		}

		blobInfo.References.Add(new BlobToBlobReference(sourceBlob, targetBlob));
	}

	public Task AddBlobToBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobId, long blobSize, CancellationToken cancellationToken)
	{
		ConcurrentDictionary<BucketId, MemoryBucketInfo> bucketDict = _bucketIndex.GetOrAdd(ns, id => new ConcurrentDictionary<BucketId, MemoryBucketInfo>());
		MemoryBucketInfo bucketInfo = bucketDict.GetOrAdd(bucket, id => new MemoryBucketInfo());
		bucketInfo.AddEntry(key, blobId, blobSize);

		return Task.CompletedTask;
	}

	public Task RemoveBlobFromBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, List<BlobId> blobIds, CancellationToken cancellationToken)
	{
		if (_bucketIndex.TryGetValue(ns, out ConcurrentDictionary<BucketId, MemoryBucketInfo>? bucketDict))
		{
			if (bucketDict.TryGetValue(bucket, out MemoryBucketInfo? bucketInfo))
			{
				bucketInfo.RemoveEntriesForRef(key);
			}
		}

		return Task.CompletedTask;
	}

	public Task<BucketStats> CalculateBucketStatisticsAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
	{
		HashSet<RefId> foundRefs = new HashSet<RefId>();
		int countOfBlobs = 0;
		long totalBlobSize = 0;
		long smallestBlobFound = 0;
		long largestBlobFound = 0;

		if (_bucketIndex.TryGetValue(ns, out ConcurrentDictionary<BucketId, MemoryBucketInfo>? bucketDict))
		{
			if (bucketDict.TryGetValue(bucket, out MemoryBucketInfo? bucketInfo))
			{
				foreach ((RefId refId, BlobId? blobId, long size) in bucketInfo.Entries)
				{
					foundRefs.Add(refId);
					countOfBlobs++;
					totalBlobSize += size;

					smallestBlobFound = Math.Min(size, smallestBlobFound);
					largestBlobFound = Math.Max(size, largestBlobFound);
				}
			}
		}

		return Task.FromResult(new BucketStats()
		{
			Namespace = ns,
			Bucket = bucket,
			CountOfRefs = foundRefs.Count,
			CountOfBlobs = countOfBlobs,
			SmallestBlobFound = smallestBlobFound,
			LargestBlob = largestBlobFound,
			TotalSize = totalBlobSize,
			AvgSize = totalBlobSize / (double)countOfBlobs
		});
	}

	private static MemoryBlobInfo NewBlobInfo(NamespaceId ns, BlobId blob, string region)
	{
		MemoryBlobInfo info = new MemoryBlobInfo
		{
			Regions = new HashSet<string> { region },
			Namespace = ns,
			BlobIdentifier = blob,
			References = new List<BaseBlobReference>()
		};
		return info;
	}
}
