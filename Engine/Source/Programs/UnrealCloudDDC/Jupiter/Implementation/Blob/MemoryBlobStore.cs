// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Utils;

namespace Jupiter.Implementation
{
	internal class MemoryBlobStore : IBlobStore
	{
		private readonly ConcurrentDictionary<NamespaceId, MemoryStorageBackend> _backends = new ConcurrentDictionary<NamespaceId, MemoryStorageBackend>();

		/// <summary>
		/// Throw an exception when putting an object that already exists
		///
		/// Primarily used for testing.
		/// </summary>
		private readonly bool _throwOnOverwrite;

		public MemoryBlobStore(bool throwOnOverwrite = false)
		{
			_throwOnOverwrite = throwOnOverwrite;
		}

		internal MemoryStorageBackend GetBackend(NamespaceId ns)
		{
			return _backends.GetOrAdd(ns, x => new MemoryStorageBackend(_throwOnOverwrite));
		}

		private static string GetPath(BlobId blob) => blob.ToString();

		public Task<BlobId> PutObjectAsync(NamespaceId ns, byte[] blob, BlobId identifier, CancellationToken cancellationToken = default)
		{
			using MemoryStream stream = new MemoryStream(blob);
			return PutObjectAsync(ns, stream, identifier, cancellationToken);
		}
		public Task<Uri?> GetObjectByRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not supported
			return Task.FromResult<Uri?>(null);
		}

		public async Task<BlobMetadata> GetObjectMetadataAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			BlobMetadata? metadata = await GetBackend(ns).GetObjectMetadataAsync(GetPath(identifier));
			if (metadata == null)
			{
				throw new BlobNotFoundException(ns, identifier);
			}

			return metadata;
		}

		public async Task<ulong> CopyBlobAsync(NamespaceId ns, NamespaceId targetNamespace, BlobId blobId, CancellationToken cancellationToken = default)
		{
			if (await GetBackend(targetNamespace).ExistsAsync(GetPath(blobId), cancellationToken))
			{
				// blob already exists, do not copy
				return (ulong)(await GetBackend(targetNamespace).GetObjectMetadataAsync(GetPath(blobId)))!.Length;
			}

			BlobContents? contents = await GetBackend(ns).TryReadAsync(GetPath(blobId), cancellationToken: cancellationToken);
			if (contents == null)
			{
				throw new BlobNotFoundException(ns, blobId);
			}

			await PutObjectAsync(targetNamespace, contents.Stream, blobId, cancellationToken);

			return (ulong)contents.Stream.Length;
		}

		public Task<Uri?> PutObjectWithRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not supported
			return Task.FromResult<Uri?>(null);
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, ReadOnlyMemory<byte> blob, BlobId identifier, CancellationToken cancellationToken = default)
		{
			return await PutObjectAsync(ns, blob: blob.ToArray(), identifier, cancellationToken);
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, Stream blob, BlobId identifier, CancellationToken cancellationToken = default)
		{
			IStorageBackend backend = GetBackend(ns);
			await backend.WriteAsync(GetPath(identifier), blob, cancellationToken);
			return identifier;
		}

		public async Task<BlobContents> GetObjectAsync(NamespaceId ns, BlobId blob, LastAccessTrackingFlags flags = LastAccessTrackingFlags.DoTracking, bool supportsRedirectUri = false, CancellationToken cancellationToken = default)
		{
			BlobContents? contents = await GetBackend(ns).TryReadAsync(GetPath(blob), flags, cancellationToken);
			if (contents == null)
			{
				throw new BlobNotFoundException(ns, blob);
			}
			return contents;
		}

		public Task DeleteObjectAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default) => GetBackend(ns).DeleteAsync(GetPath(blob), cancellationToken);

		public Task DeleteObjectAsync(IEnumerable<NamespaceId> namespaces, BlobId blob, CancellationToken cancellationToken = default)
		{
			foreach (NamespaceId ns in namespaces)
			{
				DeleteObjectAsync(ns, blob, cancellationToken);
			}

			return Task.CompletedTask;
		}

		public Task<bool> ExistsAsync(NamespaceId ns, BlobId blob, bool forceCheck = false, CancellationToken cancellationToken = default) => GetBackend(ns).ExistsAsync(GetPath(blob), cancellationToken);

		public Task DeleteNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			if (!_backends.TryRemove(ns, out _))
			{
				throw new NamespaceNotFoundException(ns);
			}
			return Task.CompletedTask;
		}

		public async IAsyncEnumerable<(BlobId, DateTime)> ListObjectsAsync(NamespaceId ns, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			IStorageBackend backend = GetBackend(ns);
			await foreach ((string path, DateTime time) in backend.ListAsync(cancellationToken))
			{
				yield return (new BlobId(path), time);
			}
		}

		internal IEnumerable<BlobId> GetIdentifiers(NamespaceId ns)
		{
			return GetBackend(ns).GetIdentifiers().Select(x => new BlobId(x));
		}

		// only for unit tests to update the last modified time
		internal void SetLastModifiedTime(NamespaceId ns, BlobId blob, DateTime modifiedTime)
		{
			GetBackend(ns).SetLastModifiedTime(GetPath(blob), modifiedTime);
		}
	}

	internal class MemoryStorageBackend : IStorageBackend
	{
		private class BlobContainer
		{
			public string Path { get; }
			public byte[] Contents { get; }
			public DateTime LastModified { get; set; }

			public BlobContainer(string path, byte[] contents)
			{
				Path = path;
				Contents = contents;
				LastModified = DateTime.Now;
			}
		}

		private readonly ConcurrentDictionary<string, BlobContainer> _blobs = new ConcurrentDictionary<string, BlobContainer>(StringComparer.Ordinal);

		/// <summary>
		/// Throw an exception when putting an object that already exists
		///
		/// Primarily used for testing.
		/// </summary>
		private readonly bool throwOnOverwrite;

		public MemoryStorageBackend(bool throwOnOverwrite = false)
		{
			this.throwOnOverwrite = throwOnOverwrite;
		}

		public async Task WriteAsync(string path, Stream stream, CancellationToken cancellationToken = default)
		{
			byte[] blob = await stream.ToByteArrayAsync(cancellationToken);

			// we do not split the blob into smaller parts when storing in memory, this is only for test purposes
			// so there is no need to add that complexity

			if (throwOnOverwrite && _blobs.ContainsKey(path))
			{
				throw new Exception($"Blob {path} already exists");
			}
			_blobs.TryAdd(path, new BlobContainer(path, blob.ToArray()));
		}

		public Task<BlobContents?> TryReadAsync(string path, LastAccessTrackingFlags flags = LastAccessTrackingFlags.DoTracking, CancellationToken cancellationToken = default)
		{
			if (!_blobs.TryGetValue(path, value: out BlobContainer? blobContainer))
			{
				return Task.FromResult<BlobContents?>(null);
			}

			byte[] content = blobContainer.Contents;
			return Task.FromResult<BlobContents?>(new BlobContents(new MemoryStream(content), content.LongLength));
		}

		public Task DeleteAsync(string path, CancellationToken cancellationToken = default)
		{
			_blobs.TryRemove(path, out _);
			return Task.CompletedTask;
		}

		public Task<bool> ExistsAsync(string path, CancellationToken cancellationToken = default)
		{
			return Task.FromResult(_blobs.ContainsKey(path));
		}

		public async IAsyncEnumerable<(string, DateTime)> ListAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			await Task.Yield();
			foreach (BlobContainer blobContainer in _blobs.Values)
			{
				yield return (blobContainer.Path, blobContainer.LastModified);
			}
		}

		public IEnumerable<string> GetIdentifiers() => _blobs.Keys;

		// only for unit tests to update the last modified time
		public void SetLastModifiedTime(string path, DateTime modifiedTime)
		{
			if (_blobs.TryGetValue(path, value: out BlobContainer? blobContainer))
			{
				blobContainer.LastModified = modifiedTime;
			}
		}

		public Task<BlobMetadata?> GetObjectMetadataAsync(string path)
		{
			if (!_blobs.TryGetValue(path, value: out BlobContainer? blobContainer))
			{
				return Task.FromResult<BlobMetadata?>(null);
			}

			byte[] content = blobContainer.Contents;
			return Task.FromResult<BlobMetadata?>(new BlobMetadata(content.LongLength, blobContainer.LastModified));
		}
	}
}
