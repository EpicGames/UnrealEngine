// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Options;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Bson.Serialization.Options;
using MongoDB.Driver;

namespace Jupiter.Implementation.Blob;

public class MongoBlobIndex : MongoStore, IBlobIndex
{
	private readonly IOptionsMonitor<JupiterSettings> _jupiterSettings;

	public MongoBlobIndex(IOptionsMonitor<JupiterSettings> jupiterSettings, IOptionsMonitor<MongoSettings> settings) : base(settings)
	{
		_jupiterSettings = jupiterSettings;

		CreateCollectionIfNotExistsAsync<MongoBlobIndexModelV0>().Wait();

		IndexKeysDefinitionBuilder<MongoBlobIndexModelV0> indexKeysDefinitionBuilder = Builders<MongoBlobIndexModelV0>.IndexKeys;
		CreateIndexModel<MongoBlobIndexModelV0> indexModel = new CreateIndexModel<MongoBlobIndexModelV0>(
			indexKeysDefinitionBuilder.Combine(
				indexKeysDefinitionBuilder.Ascending(m => m.Ns),
				indexKeysDefinitionBuilder.Ascending(m => m.BlobId)
			)
			, new CreateIndexOptions()
			{
				Name = "CompoundIndex"
			});

		AddIndexFor<MongoBlobIndexModelV0>().CreateMany(new[] {
			indexModel,
		});
	}

	private async Task<MongoBlobIndexModelV0?> GetBlobInfoAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken)
	{
		IMongoCollection<MongoBlobIndexModelV0> collection = GetCollection<MongoBlobIndexModelV0>();
		IAsyncCursor<MongoBlobIndexModelV0>? cursor = await collection.FindAsync(m => m.Ns == ns.ToString() && m.BlobId == id.ToString(), cancellationToken: cancellationToken);
		MongoBlobIndexModelV0? model = await cursor.FirstOrDefaultAsync(cancellationToken);
		if (model == null)
		{
			return null;
		}

		return model;
	}

	public async Task AddBlobToIndexAsync(NamespaceId ns, BlobId id, ulong rawSize, string? region = null,
		CancellationToken cancellationToken = default)
	{
		region ??= _jupiterSettings.CurrentValue.CurrentSite;
		IMongoCollection<MongoBlobIndexModelV0> collection = GetCollection<MongoBlobIndexModelV0>();
		MongoBlobIndexModelV0 model = new MongoBlobIndexModelV0(ns, id);
		model.Regions.Add(region);

		FilterDefinition<MongoBlobIndexModelV0> filter = Builders<MongoBlobIndexModelV0>.Filter.Where(m => m.Ns == ns.ToString() && m.BlobId == id.ToString());
		await collection.FindOneAndReplaceAsync(filter, model, new FindOneAndReplaceOptions<MongoBlobIndexModelV0, MongoBlobIndexModelV0>
		{
			IsUpsert = true
		}, cancellationToken);
	}

	public async Task RemoveBlobFromRegionAsync(NamespaceId ns, BlobId id, string? region = null, CancellationToken cancellationToken = default)
	{
		region ??= _jupiterSettings.CurrentValue.CurrentSite;

		MongoBlobIndexModelV0? model = await GetBlobInfoAsync(ns, id, cancellationToken);
		if (model == null)
		{
			throw new BlobNotFoundException(ns, id);
		}
		IMongoCollection<MongoBlobIndexModelV0> collection = GetCollection<MongoBlobIndexModelV0>();
		model.Regions.Remove(region);
		FilterDefinition<MongoBlobIndexModelV0> filter = Builders<MongoBlobIndexModelV0>.Filter.Where(m => m.Ns == ns.ToString() && m.BlobId == id.ToString());
		await collection.FindOneAndReplaceAsync(filter, model, new FindOneAndReplaceOptions<MongoBlobIndexModelV0, MongoBlobIndexModelV0>
		{
			IsUpsert = true
		}, cancellationToken);

	}

	public async Task RemoveBlobFromAllRegionsAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken = default)
	{
		IMongoCollection<MongoBlobIndexModelV0> collection = GetCollection<MongoBlobIndexModelV0>();
		FilterDefinition<MongoBlobIndexModelV0> filter = Builders<MongoBlobIndexModelV0>.Filter.Where(m => m.Ns == ns.ToString() && m.BlobId == id.ToString());
		await collection.DeleteOneAsync(filter, cancellationToken);
	}

	public async IAsyncEnumerable<(NamespaceId, BaseBlobReference)> GetAllBlobReferencesAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
	{
		IMongoCollection<MongoBlobIndexModelV0> collection = GetCollection<MongoBlobIndexModelV0>();
		IAsyncCursor<MongoBlobIndexModelV0>? cursor = await collection.FindAsync(FilterDefinition<MongoBlobIndexModelV0>.Empty, cancellationToken: cancellationToken);

		while (await cursor.MoveNextAsync(cancellationToken))
		{
			foreach (MongoBlobIndexModelV0 model in cursor.Current)
			{
				NamespaceId ns = new NamespaceId(model.Ns);
				BlobId id = new BlobId(model.BlobId);
				foreach (Dictionary<string, string> reference in model.References)
				{
					if (reference.ContainsKey("bucket"))
					{
						string bucket = reference["bucket"];
						string key = reference["key"];
						yield return (ns, new RefBlobReference(id, new BucketId(bucket), new RefId(key)));
					}
					else if (reference.ContainsKey("blob_id"))
					{
						string blobId = reference["blob_id"];
						yield return (ns, new BlobToBlobReference(id, new BlobId(blobId)));
					}
					else
					{
						throw new NotImplementedException();
					}
				}
			}
		}
	}

	public async IAsyncEnumerable<BaseBlobReference> GetBlobReferencesAsync(NamespaceId ns, BlobId id, [EnumeratorCancellation] CancellationToken cancellationToken)
	{
		MongoBlobIndexModelV0? blobInfo = await GetBlobInfoAsync(ns, id, cancellationToken);
		if (blobInfo == null)
		{
			yield break;
		}

		foreach (Dictionary<string, string> reference in blobInfo.References)
		{
			if (reference.ContainsKey("bucket"))
			{
				string bucket = reference["bucket"];
				string key = reference["key"];
				yield return new RefBlobReference(id, new BucketId(bucket), new RefId(key));
			}
			else if (reference.ContainsKey("blob_id"))
			{
				string blobId = reference["blob_id"];
				yield return new BlobToBlobReference(id, new BlobId(blobId));
			}
			else
			{
				throw new NotImplementedException();
			}
		}
	}

	public async Task<bool> BlobExistsInRegionAsync(NamespaceId ns, BlobId blobIdentifier, string? region = null, CancellationToken cancellationToken = default)
	{
		MongoBlobIndexModelV0? blobInfo = await GetBlobInfoAsync(ns, blobIdentifier, cancellationToken);
		return blobInfo?.Regions.Contains(_jupiterSettings.CurrentValue.CurrentSite) ?? false;
	}

	public async Task AddRefToBlobsAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId[] blobs, CancellationToken cancellationToken)
	{
		IMongoCollection<MongoBlobIndexModelV0> collection = GetCollection<MongoBlobIndexModelV0>();

		string nsAsString = ns.ToString();
		Task[] refUpdateTasks = new Task[blobs.Length];
		for (int i = 0; i < blobs.Length; i++)
		{
			BlobId id = blobs[i];
			refUpdateTasks[i] = Task.Run(async () =>
			{
				UpdateDefinition<MongoBlobIndexModelV0> update = Builders<MongoBlobIndexModelV0>.Update.AddToSet(m => m.References, new Dictionary<string, string> { { "bucket", bucket.ToString() }, { "key", key.ToString() } });
				FilterDefinition<MongoBlobIndexModelV0> filter = Builders<MongoBlobIndexModelV0>.Filter.Where(m => m.Ns == nsAsString && m.BlobId == id.ToString());

				await collection.FindOneAndUpdateAsync(filter, update, cancellationToken: cancellationToken);
			}, cancellationToken);
		}

		await Task.WhenAll(refUpdateTasks);
	}

	public async IAsyncEnumerable<(NamespaceId, BlobId)> GetAllBlobsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
	{
		IMongoCollection<MongoBlobIndexModelV0> collection = GetCollection<MongoBlobIndexModelV0>();
		IAsyncCursor<MongoBlobIndexModelV0>? cursor = await collection.FindAsync(FilterDefinition<MongoBlobIndexModelV0>.Empty, cancellationToken: cancellationToken);

		while (await cursor.MoveNextAsync(cancellationToken))
		{
			foreach (MongoBlobIndexModelV0 model in cursor.Current)
			{
				yield return (new NamespaceId(model.Ns), new BlobId(model.BlobId));
			}
		}
	}

	public async Task RemoveReferencesAsync(NamespaceId ns, BlobId id, List<BaseBlobReference>? referencesToRemove, CancellationToken cancellationToken)
	{
		IMongoCollection<MongoBlobIndexModelV0> collection = GetCollection<MongoBlobIndexModelV0>();

		string nsAsString = ns.ToString();
		if (referencesToRemove == null)
		{
			FilterDefinition<MongoBlobIndexModelV0> filter = Builders<MongoBlobIndexModelV0>.Filter.Where(m => m.Ns == nsAsString && m.BlobId == id.ToString());
			await collection.DeleteOneAsync(filter, cancellationToken);
		}
		else
		{
			List<Dictionary<string, string>> refs = referencesToRemove.Select(reference =>
			{
				if (reference is RefBlobReference refBlobReference)
				{
					return new Dictionary<string, string>
					{
						{ "bucket", refBlobReference.Bucket.ToString() }, { "key", refBlobReference.Key.ToString() }
					};
				}
				else if (reference is BlobToBlobReference blobToBlobReference)
				{
					return new Dictionary<string, string>
					{
						{ "blob_id", blobToBlobReference.Blob.ToString()}
					};
				}
				else
				{
					throw new NotImplementedException();
				}
			}).ToList();
			UpdateDefinition<MongoBlobIndexModelV0> update = Builders<MongoBlobIndexModelV0>.Update.PullAll(m => m.References, refs);
			FilterDefinition<MongoBlobIndexModelV0> filter = Builders<MongoBlobIndexModelV0>.Filter.Where(m => m.Ns == nsAsString && m.BlobId == id.ToString());

			await collection.FindOneAndUpdateAsync(filter, update, cancellationToken: cancellationToken);
		}
	}

	public async Task<List<string>> GetBlobRegionsAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken)
	{
		MongoBlobIndexModelV0? blobInfo = await GetBlobInfoAsync(ns, blob, cancellationToken);
		if (blobInfo == null)
		{
			throw new BlobNotFoundException(ns, blob);
		}
		return blobInfo.Regions.ToList();
	}

	public async Task AddBlobReferencesAsync(NamespaceId ns, BlobId sourceBlob, BlobId targetBlob, CancellationToken cancellationToken)
	{
		IMongoCollection<MongoBlobIndexModelV0> collection = GetCollection<MongoBlobIndexModelV0>();

		string nsAsString = ns.ToString();

		UpdateDefinition<MongoBlobIndexModelV0> update = Builders<MongoBlobIndexModelV0>.Update.AddToSet(m => m.References, new Dictionary<string, string> { { "blob_id", targetBlob.ToString() } });
		FilterDefinition<MongoBlobIndexModelV0> filter = Builders<MongoBlobIndexModelV0>.Filter.Where(m => m.Ns == nsAsString && m.BlobId == sourceBlob.ToString());

		await collection.FindOneAndUpdateAsync(filter, update, cancellationToken: cancellationToken);
	}

	public async Task AddBlobToBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobId, long blobSize, CancellationToken cancellationToken)
	{
		IMongoCollection<MongoBucketBlobV0> collection = GetCollection<MongoBucketBlobV0>();

		FilterDefinition<MongoBucketBlobV0> filter = Builders<MongoBucketBlobV0>.Filter.Where(m => m.Ns == ns.ToString() && m.BucketId == bucket.ToString() && m.RefId == key.ToString() && m.BlobId == blobId.ToString());
		await collection.ReplaceOneAsync(filter, new MongoBucketBlobV0(ns, bucket, key, blobId, blobSize), new ReplaceOptions() { IsUpsert = true }, cancellationToken);
	}

	public async Task RemoveBlobFromBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, List<BlobId> blobIds, CancellationToken cancellationToken)
	{
		IMongoCollection<MongoBucketBlobV0> collection = GetCollection<MongoBucketBlobV0>();

		FilterDefinition<MongoBucketBlobV0> filter = Builders<MongoBucketBlobV0>.Filter.Where(m => m.Ns == ns.ToString() && m.BucketId == bucket.ToString() && m.RefId == key.ToString());
		await collection.DeleteManyAsync(filter, cancellationToken);
	}

	public async Task<BucketStats> CalculateBucketStatisticsAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
	{
		IMongoCollection<MongoBucketBlobV0> collection = GetCollection<MongoBucketBlobV0>();
		FilterDefinition<MongoBucketBlobV0> filter = Builders<MongoBucketBlobV0>.Filter.Where(m => m.Ns == ns.ToString() && m.BucketId == bucket.ToString());

		var blobStats = await collection.Aggregate()
			.Match(filter)
			.Group(
				a => a.BucketId,
				r => new
				{
					TotalSize = r.Sum(a => a.Size),
					SmallestBlob = r.Min(a => a.Size),
					LargestBlob = r.Max(a => a.Size),
					CountOfBlobs = r.Count(),
				}).ToListAsync(cancellationToken);

		var blobStat = blobStats.FirstOrDefault();

		long countOfRefs = collection
			.Aggregate()
			.Match(filter)
			.Group(m => m.RefId,
				grouping => new { DoesNotMatter = grouping.Key })
			.Count()
			.First(cancellationToken)
			.Count;

		return new BucketStats()
		{
			Namespace = ns,
			Bucket = bucket,
			CountOfRefs = countOfRefs,
			CountOfBlobs = blobStat?.CountOfBlobs ?? 0,
			SmallestBlobFound = (long)(blobStat?.SmallestBlob ?? 0),
			LargestBlob = (long)(blobStat?.LargestBlob ?? 0),
			TotalSize = blobStat?.TotalSize ?? 0,
			AvgSize = blobStat == null ? 0 : blobStat.TotalSize / (double)blobStat.CountOfBlobs
		};
	}
}

[BsonDiscriminator("blob-index.v0")]
[BsonIgnoreExtraElements]
[MongoCollectionName("BlobIndex")]
class MongoBlobIndexModelV0
{
	[BsonConstructor]
	public MongoBlobIndexModelV0(string ns, string blobId, List<string> regions, List<Dictionary<string, string>> references)
	{
		Ns = ns;
		BlobId = blobId;
		Regions = regions;
		References = references;
	}

	public MongoBlobIndexModelV0(NamespaceId ns, BlobId blobId)
	{
		Ns = ns.ToString();
		BlobId = blobId.ToString();
	}

	[BsonRequired]
	public string Ns { get; set; }

	[BsonRequired]
	public string BlobId { get; set; }

	public List<string> Regions { get; set; } = new List<string>();

	[BsonDictionaryOptions(DictionaryRepresentation.Document)]
	public List<Dictionary<string, string>> References { get; set; } = new List<Dictionary<string, string>>();

}

[BsonDiscriminator("bucket-blob.v0")]
[BsonIgnoreExtraElements]
[MongoCollectionName("BucketBlob")]
class MongoBucketBlobV0
{
	[BsonConstructor]
	public MongoBucketBlobV0(string ns, string bucketId, string refId, string blobId, long size)
	{
		Ns = ns;
		BucketId = bucketId;
		RefId = refId;
		BlobId = blobId;
		Size = size;
	}

	public MongoBucketBlobV0(NamespaceId ns, BucketId bucketId, RefId refId, BlobId blobId, long size)
	{
		Ns = ns.ToString();
		BucketId = bucketId.ToString();
		RefId = refId.ToString();
		BlobId = blobId.ToString();
		Size = size;
	}

	[BsonRequired]
	public string Ns { get; set; }

	[BsonRequired]
	public string BucketId { get; set; }

	[BsonRequired]
	public string RefId { get; set; }

	[BsonRequired]
	public string BlobId { get; set; }

	[BsonRequired]
	public long Size { get; set; }
}