// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Reflection;

namespace Jupiter.Implementation;

public static class PathUtil
{
	public static string ResolvePath(string path)
	{
		return Environment.ExpandEnvironmentVariables(path)
			.Replace("$(ExecutableLocation)", Path.GetDirectoryName(Assembly.GetEntryAssembly()!.Location), StringComparison.OrdinalIgnoreCase);
	}
}
