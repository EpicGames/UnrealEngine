// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Mime;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Implementation.LeaderElection;
using k8s;
using k8s.Models;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class PeerBlobStore : IBlobStore
	{
		private readonly IPeerServiceDiscovery _serviceDiscovery;
		private readonly IHttpClientFactory _httpClientFactory;
		private readonly IServiceCredentials _serviceCredentials;
		private readonly ILogger _logger;

		public PeerBlobStore(IPeerServiceDiscovery serviceDiscovery, IHttpClientFactory httpClientFactory, IServiceCredentials serviceCredentials, ILogger<PeerBlobStore> logger)
		{
			_serviceDiscovery = serviceDiscovery;
			_httpClientFactory = httpClientFactory;
			_serviceCredentials = serviceCredentials;
			_logger = logger;
		}

		private async Task<HttpRequestMessage> BuildHttpRequestAsync(HttpMethod method, Uri uri, CancellationToken cancellationToken = default)
		{
			string? token = await _serviceCredentials.GetTokenAsync(cancellationToken);
			HttpRequestMessage request = new HttpRequestMessage(method, uri);
			if (!string.IsNullOrEmpty(token))
			{
				request.Headers.Add("Authorization", $"{_serviceCredentials.GetAuthenticationScheme()} {token}");
			}

			return request;
		}

		private async Task<BlobContents?> DoGetObjectAsync(string instance, NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default)
		{
			try
			{
				string filesystemLayerName = nameof(FileSystemStore);
				using HttpRequestMessage getObjectRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/blobs/{ns}/{blob}?storageLayers={filesystemLayerName}", UriKind.Relative), cancellationToken);
				getObjectRequest.Headers.Add("Accept", MediaTypeNames.Application.Octet);
				HttpResponseMessage response = await GetHttpClient(instance).SendAsync(getObjectRequest, cancellationToken);
				if (response.StatusCode == HttpStatusCode.NotFound)
				{
					return null;
				}

				response.EnsureSuccessStatusCode();

				long? contentLength = response.Content.Headers.ContentLength;
				if (contentLength == null)
				{
					_logger.LogWarning("Content length missing in response from peer blob store. This is not supported, ignoring response");
					return null;
				}

				return new BlobContents(await response.Content.ReadAsStreamAsync(cancellationToken), contentLength.Value);
			}
			catch (Exception e)
			{
				_logger.LogWarning(e,
					"Exception when attempting to fetch blob {Blob} in namespace {Namespace} from instance {Instance}",
					blob, ns, instance);
			}

			return null;
		}

		public async Task<BlobContents> GetObjectAsync(NamespaceId ns, BlobId blob, LastAccessTrackingFlags flags = LastAccessTrackingFlags.DoTracking, bool supportsRedirectUri = false, CancellationToken cancellationToken = default)
		{
			List<Task<BlobContents?>> tasks = new();

			await foreach (string instance in _serviceDiscovery.FindOtherInstances(cancellationToken))
			{
				Task<BlobContents?> task = DoGetObjectAsync(instance, ns, blob, cancellationToken);
				tasks.Add(task);
			}

			while (tasks.Count != 0)
			{
				Task<BlobContents?> finishedTask = await Task.WhenAny(tasks);
				BlobContents? result = await finishedTask;
				if (result != null)
				{
					return result;
				}

				tasks.Remove(finishedTask);
			}

			throw new BlobNotFoundException(ns, blob);
		}

		private async Task<bool?> DoExistsAsync(string instance, NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default)
		{
			try
			{
				string filesystemLayerName = nameof(FileSystemStore);
				using HttpRequestMessage headObjectRequest = await BuildHttpRequestAsync(HttpMethod.Head, new Uri($"api/v1/blobs/{ns}/{blob}?storageLayers={filesystemLayerName}", UriKind.Relative), cancellationToken);
				HttpResponseMessage response = await GetHttpClient(instance).SendAsync(headObjectRequest, cancellationToken);
				if (response.StatusCode == HttpStatusCode.NotFound)
				{
					return false;
				}

				response.EnsureSuccessStatusCode();

				return true;
			}
			catch (Exception e)
			{
				_logger.LogWarning(e,
					"Exception when attempting to fetch blob {Blob} in namespace {Namespace} from instance {Instance}",
					blob, ns, instance);
			}

			return null;
		}

		public async Task<bool> ExistsAsync(NamespaceId ns, BlobId blob, bool forceCheck = false, CancellationToken cancellationToken = default)
		{
			List<Task<bool?>> tasks = new();

			await foreach (string instance in _serviceDiscovery.FindOtherInstances(cancellationToken))
			{
				Task<bool?> task = DoExistsAsync(instance, ns, blob, cancellationToken);
				tasks.Add(task);
			}

			while (tasks.Count != 0)
			{
				Task<bool?> finishedTask = await Task.WhenAny(tasks);
				bool? result = await finishedTask;
				if (result != null)
				{
					return result.Value;
				}

				tasks.Remove(finishedTask);
			}

			return false;
		}

		private HttpClient GetHttpClient(string instance)
		{
			HttpClient httpClient = _httpClientFactory.CreateClient(instance);
			httpClient.BaseAddress = new Uri($"http://{instance}");

			// for these connections to be useful they need to be fast - so timeout quickly if we can not establish the connection
			httpClient.Timeout = TimeSpan.FromSeconds(1.0);
			return httpClient;
		}

		public Task<Uri?> GetObjectByRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not supported
			return Task.FromResult<Uri?>(null);
		}

		public Task<BlobMetadata> GetObjectMetadataAsync(NamespaceId ns, BlobId blobId, CancellationToken cancellationToken = default)
		{
			// Do not call into other instances to determine metadata as we lack a endpoint for it
			throw new BlobNotFoundException(ns, blobId);
		}

		public Task<ulong> CopyBlobAsync(NamespaceId ns, NamespaceId targetNamespace, BlobId blobId, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException();
		}

		public Task<Uri?> PutObjectWithRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not supported
			return Task.FromResult<Uri?>(null);
		}

		public Task<BlobId> PutObjectAsync(NamespaceId ns, byte[] blob, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not applicable
			return Task.FromResult(identifier);
		}

		public Task<BlobId> PutObjectAsync(NamespaceId ns, ReadOnlyMemory<byte> blob, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not applicable
			return Task.FromResult(identifier);
		}

		public Task<BlobId> PutObjectAsync(NamespaceId ns, Stream content, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not applicable
			return Task.FromResult(identifier);
		}

		public Task DeleteObjectAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default)
		{
			// not applicable
			return Task.CompletedTask;
		}

		public Task DeleteObjectAsync(IEnumerable<NamespaceId> namespaces, BlobId blob, CancellationToken cancellationToken = default)
		{
			// not applicable
			return Task.CompletedTask;
		}

		public Task DeleteNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			// not applicable
			return Task.CompletedTask;
		}

		public IAsyncEnumerable<(BlobId, DateTime)> ListObjectsAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			// not applicable
			return AsyncEnumerable.Empty<(BlobId, DateTime)>();
		}
	}

	public interface IPeerServiceDiscovery
	{
		public IAsyncEnumerable<string> FindOtherInstances(CancellationToken cancellationToken = default);
	}

	public class StaticPeerServiceDiscoverySettings
	{
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<string> Peers { get; set; } = new List<string>();
	}

	public sealed class StaticPeerServiceDiscovery : IPeerServiceDiscovery
	{
		private readonly IOptionsMonitor<StaticPeerServiceDiscoverySettings> _settings;

		public StaticPeerServiceDiscovery(IOptionsMonitor<StaticPeerServiceDiscoverySettings> settings)
		{
			_settings = settings;
		}

		public async IAsyncEnumerable<string> FindOtherInstances([EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			await Task.CompletedTask;
			foreach (string peer in _settings.CurrentValue.Peers)
			{
				yield return peer;
			}
		}
	}

	public sealed class KubernetesPeerServiceDiscovery : IPeerServiceDiscovery, IDisposable
	{
		private readonly IOptionsMonitor<KubernetesLeaderElectionSettings> _leaderSettings;
		private readonly Kubernetes _client;
		private DateTime _podEnumerationValidUntil = DateTime.Now;
		private List<string>? _lastPodEnumeration;

		public KubernetesPeerServiceDiscovery(IOptionsMonitor<KubernetesLeaderElectionSettings> leaderSettings)
		{
			_leaderSettings = leaderSettings;
			KubernetesClientConfiguration config = KubernetesClientConfiguration.InClusterConfig();
			_client = new Kubernetes(config);
		}

		public async IAsyncEnumerable<string> FindOtherInstances([EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			if (_lastPodEnumeration != null && _podEnumerationValidUntil > DateTime.Now)
			{
				foreach (string pod in _lastPodEnumeration)
				{
					yield return pod;
				}
				yield break;
			}

			_lastPodEnumeration = await EnumeratePodsAsync(cancellationToken).ToListAsync(cancellationToken);
			_podEnumerationValidUntil = DateTime.Now.AddMinutes(5);

			foreach (string pod in _lastPodEnumeration)
			{
				yield return pod;
			}
		}

		private async IAsyncEnumerable<string> EnumeratePodsAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			string currentPodName = _leaderSettings.CurrentValue.PodName;
			V1PodList podList = await _client.ListNamespacedPodAsync(_leaderSettings.CurrentValue.Namespace, labelSelector: _leaderSettings.CurrentValue.PeerPodLabelSelector, cancellationToken: cancellationToken);

			foreach (V1Pod pod in podList.Items)
			{
				if (pod.Status.Phase != "Running")
				{
					continue;
				}

				// skip the current pod
				if (pod.Name() == currentPodName)
				{
					continue;
				}
				string ip = pod.Status.PodIP;
				if (string.IsNullOrEmpty(ip))
				{
					continue;
				}

				// we typically expose the container on port 80
				yield return $"{ip}:80";
			}
		}

		private void Dispose(bool disposing)
		{
			if (disposing)
			{
				_client.Dispose();
			}
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}
	}
}
