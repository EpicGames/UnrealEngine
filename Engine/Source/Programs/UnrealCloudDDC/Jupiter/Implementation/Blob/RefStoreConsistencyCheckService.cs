// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Implementation.Blob;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class RefConsistencyState
	{
	}

	// ReSharper disable once ClassNeverInstantiated.Global
	public class RefStoreConsistencyCheckService : PollingService<ConsistencyState>
	{
		private readonly IOptionsMonitor<ConsistencyCheckSettings> _settings;
		private readonly ILeaderElection _leaderElection;
		private readonly IRefService _refService;
		private readonly IReferencesStore _referencesStore;
		private readonly IPeerStatusService _peerStatusService;
		private readonly IBlobIndex _blobIndex;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;

		protected override bool ShouldStartPolling()
		{
			return _settings.CurrentValue.EnableRefStoreChecks;
		}

		public RefStoreConsistencyCheckService(IOptionsMonitor<ConsistencyCheckSettings> settings, ILeaderElection leaderElection, IRefService refService, IBlobIndex blobIndex, Tracer tracer, ILogger<RefStoreConsistencyCheckService> logger, IReferencesStore referencesStore, IPeerStatusService peerStatusService) : base(serviceName: nameof(RefStoreConsistencyCheckService), TimeSpan.FromSeconds(settings.CurrentValue.ConsistencyCheckPollFrequencySeconds), new ConsistencyState(), logger)
		{
			_settings = settings;
			_leaderElection = leaderElection;
			_refService = refService;
			_referencesStore = referencesStore;
			_peerStatusService = peerStatusService;
			_blobIndex = blobIndex;
			_tracer = tracer;
			_logger = logger;
		}

		public override async Task<bool> OnPollAsync(ConsistencyState state, CancellationToken cancellationToken = default)
		{
			if (!_settings.CurrentValue.EnableRefStoreChecks)
			{
				_logger.LogInformation("Skipped running ref store consistency check as it is disabled");
				return false;
			}

			try
			{
				await RunConsistencyCheckAsync(cancellationToken);
			}
			catch (RemoteServiceException e)
			{
				_logger.LogWarning(e, "Scylla communication issue occured while running consistency check");
			}

			return true;
		}

		public async Task RunConsistencyCheckAsync(CancellationToken cancellationToken = default)
		{
			if (!_leaderElection.IsThisInstanceLeader())
			{
				_logger.LogInformation("Skipped running ref store consistency check because this instance was not the leader");
				return;
			}

			ulong countOfRefsChecked = 0;
			ulong countOfMissingLastAccessTime = 0;
			ulong countOfRegionalInconsistentRefs = 0;
			await Parallel.ForEachAsync(_referencesStore.GetRecordsWithoutAccessTimeAsync(cancellationToken), cancellationToken,
				async (record, token) =>
				{
					(NamespaceId ns, BucketId bucket, RefId refId) = record;
					using TelemetrySpan scope = _tracer.StartActiveSpan("consistency_check.ref_store")
					.SetAttribute("operation.name", "consistency_check.ref_store")
					.SetAttribute("resource.name", $"{ns}.{bucket}.{refId}");

					if (_settings.CurrentValue.CheckRefStoreLastAccessTimeConsistency)
					{
						DateTime? lastAccessTime = await _referencesStore.GetLastAccessTimeAsync(ns, bucket, refId, token);
						if (!lastAccessTime.HasValue)
						{
							// if there is no last access time record we add one so that the two tables are consistent, this will make the ref record be considered by the GC and thus cleanup anything that might be very old (but its added as a new record so it will take until the configured cleanup time has passed)
							await _referencesStore.UpdateLastAccessTimeAsync(ns, bucket, refId, DateTime.Now, token);

							Interlocked.Increment(ref countOfMissingLastAccessTime);
						}
					}
					if (_settings.CurrentValue.CheckRefStoreRegionalConsistency)
					{
						if (_settings.CurrentValue.RegionalConsistencyCheckNamespaces.Contains(ns.ToString()))
						{
							bool missing = false;
							try
							{
								missing = await VerifyRegionalConsistencyAsync(ns, bucket, refId);
							}
							catch (Exception e)
							{
								_logger.LogWarning(e, "Unknown exception with message {Message} when checking ref store consistency. Ignoring.", e.Message);
							}

							if (missing)
							{
								Interlocked.Increment(ref countOfRegionalInconsistentRefs);
							}
						}
					}
					if (_settings.CurrentValue.CheckRefStorePartialObjects)
					{
						// check if any ref can be found by listing but not when explicitly looked up, indicates it doesn't contain all the expected fields and thus is a partial record that should be removed to fix bugs with how TTL was set when is_finalized was updated
						try
						{
							await _referencesStore.GetAsync(ns, bucket, refId, IReferencesStore.FieldFlags.None, IReferencesStore.OperationFlags.BypassCache, token);
						}
						catch (RefNotFoundException)
						{
							_logger.LogWarning("Partial record {Namespace} {Bucket} {Name} found. Deleting.", ns, bucket, refId);
							await _referencesStore.DeleteAsync(ns, bucket, refId, token);
						}
					}
					Interlocked.Increment(ref countOfRefsChecked);
				});

			if (_settings.CurrentValue.CheckRefStoreLastAccessTimeConsistency)
			{
				_logger.LogInformation("Consistency check finished for ref store, found {CountOfMissingLastAccessTime} refs that were lacking last access time. Processed {CountOfRefs} refs.", countOfMissingLastAccessTime, countOfRefsChecked);
			}

			if (_settings.CurrentValue.CheckRefStoreRegionalConsistency)
			{
				_logger.LogInformation("Consistency check finished for ref store, found {CountOfInconsistentRefs} refs that were inconsistent in at least one region. Processed {CountOfRefs} refs.", countOfRegionalInconsistentRefs, countOfRefsChecked);
			}
		}

		private async Task<bool> VerifyRegionalConsistencyAsync(NamespaceId ns, BucketId bucket, RefId refId)
		{
			string[] fields = new string[] { "namespace", "bucket", "ref", "last_access" };
			(RefRecord refRecord, BlobContents? _) = await _refService.GetAsync(ns, bucket, refId, fields: fields, doLastAccessTracking: false);
			// last access in this case is going to be the creation date
			// TODO: Not sure if we really need to exclude new refs like this. Attempting without this for now.
			/*if (refRecord.LastAccess > DateTime.Now.AddHours(-2))
			{
				// if the ref was created in the last two hours we ignore it as replication may not have had time to run yet
				return false;
			}*/

			List<BlobId> blobs = await _refService.GetReferencedBlobsAsync(ns, bucket, refId, ignoreMissingBlobs: true);
			Dictionary<string, Dictionary<string, bool>> blobStatePerRegion = new Dictionary<string, Dictionary<string, bool>>();

			bool blobMissing = false;
			await Parallel.ForEachAsync(blobs, async (blobId, cancellationToken) =>
			{
				Dictionary<string, bool> blobState = new Dictionary<string, bool>();

				foreach (string region in _peerStatusService.GetRegions())
				{
					bool exists = await _blobIndex.BlobExistsInRegionAsync(ns, blobId, region, CancellationToken.None);
					blobState.TryAdd(region, exists);
					if (!exists)
					{
						blobMissing = true;
					}
				}

				lock (blobStatePerRegion)
				{
					blobStatePerRegion[blobId.ToString()] = blobState;
				}
			});

			if (!blobMissing)
			{
				return false;
			}

			int countOfMissingBlobs = 0;
			HashSet<string> regionsWithMissingBlobs = new HashSet<string>();
			foreach ((string _, Dictionary<string, bool> regions) in blobStatePerRegion)
			{
				bool blobWasMissingInRegion = false;
				foreach (KeyValuePair<string, bool> regionState in regions.Where(pair => pair.Value == false))
				{
					if (!regionState.Value)
					{
						regionsWithMissingBlobs.Add(regionState.Key);
						blobWasMissingInRegion = true;
					}
				}

				if (blobWasMissingInRegion)
				{
					countOfMissingBlobs++;
				}
			}
			_logger.LogWarning("Regional inconsistency for ref {Namespace} {Bucket} {RefId} missing a total of {CountOfBlobs} blob(s) in these regions: {Regions} . Ref was created at {CreationTime}", ns, bucket, refId, countOfMissingBlobs, regionsWithMissingBlobs, refRecord.LastAccess);

			return blobMissing;
		}

		protected override Task OnStopping(ConsistencyState state)
		{
			return Task.CompletedTask;
		}
	}
}
