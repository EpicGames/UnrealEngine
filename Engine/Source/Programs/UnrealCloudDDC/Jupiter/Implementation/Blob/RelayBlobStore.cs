// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Net.Mime;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class RelayBlobStore : RelayStore, IBlobStore
	{
		public RelayBlobStore(IOptionsMonitor<UpstreamRelaySettings> settings, IHttpClientFactory httpClientFactory, IServiceCredentials serviceCredentials) : base(settings, httpClientFactory, serviceCredentials)
		{
		}

		public Task<Uri?> GetObjectByRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// not supported
			return Task.FromResult<Uri?>(null);
		}

		public Task<BlobMetadata> GetObjectMetadataAsync(NamespaceId ns, BlobId blobId, CancellationToken cancellationToken = default)
		{
			// do not call into other instances to find metadata as we lack a endpoint for that
			throw new BlobNotFoundException(ns, blobId);
		}

		public Task<ulong> CopyBlobAsync(NamespaceId ns, NamespaceId targetNamespace, BlobId blobId, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException();
		}

		public Task<Uri?> PutObjectWithRedirectAsync(NamespaceId ns, BlobId identifier, CancellationToken cancellationToken = default)
		{
			// TODO: It could be useful to support relaying the presigned url
			// not supported
			return Task.FromResult<Uri?>(null);
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, byte[] blob, BlobId identifier, CancellationToken cancellationToken = default)
		{
			using HttpRequestMessage putObjectRequest = await BuildHttpRequestAsync(HttpMethod.Put, new Uri($"api/v1/blobs/{ns}/{identifier}", UriKind.Relative));

			putObjectRequest.Content = new ByteArrayContent(blob);
			putObjectRequest.Content.Headers.ContentType = new MediaTypeHeaderValue(MediaTypeNames.Application.Octet);
			putObjectRequest.Content.Headers.Add(CommonHeaders.HashHeaderName, identifier.ToString());

			HttpResponseMessage response = await HttpClient.SendAsync(putObjectRequest, cancellationToken);

			response.EnsureSuccessStatusCode();

			return identifier;
		}

		public Task<BlobId> PutObjectAsync(NamespaceId ns, ReadOnlyMemory<byte> blob, BlobId identifier, CancellationToken cancellationToken = default)
		{
			return PutObjectAsync(ns, blob.ToArray(), identifier, cancellationToken);
		}

		public async Task<BlobId> PutObjectAsync(NamespaceId ns, Stream content, BlobId identifier, CancellationToken cancellationToken = default)
		{
			using HttpRequestMessage putObjectRequest = await BuildHttpRequestAsync(HttpMethod.Put, new Uri($"api/v1/blobs/{ns}/{identifier}", UriKind.Relative));
			putObjectRequest.Content = new StreamContent(content);
			putObjectRequest.Content.Headers.ContentType = new MediaTypeHeaderValue(MediaTypeNames.Application.Octet);
			putObjectRequest.Content.Headers.Add(CommonHeaders.HashHeaderName, identifier.ToString());
			HttpResponseMessage response = await HttpClient.SendAsync(putObjectRequest, cancellationToken);

			response.EnsureSuccessStatusCode();

			return identifier;
		}

		public async Task<BlobContents> GetObjectAsync(NamespaceId ns, BlobId blob, LastAccessTrackingFlags flags, bool supportsRedirectUri = false, CancellationToken cancellationToken = default)
		{
			using HttpRequestMessage getObjectRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/blobs/{ns}/{blob}", UriKind.Relative));
			getObjectRequest.Headers.Add("Accept", MediaTypeNames.Application.Octet);
			HttpResponseMessage response = await HttpClient.SendAsync(getObjectRequest, cancellationToken);
			if (response.StatusCode == HttpStatusCode.NotFound)
			{
				throw new BlobNotFoundException(ns, blob);
			}

			response.EnsureSuccessStatusCode();

			long? contentLength = response.Content.Headers.ContentLength;
			if (contentLength == null)
			{
				throw new Exception($"Content length missing in response from upstream blob store. This is not supported");
			}

			return new BlobContents(await response.Content.ReadAsStreamAsync(cancellationToken), contentLength.Value);
		}

		public async Task<bool> ExistsAsync(NamespaceId ns, BlobId blob, bool forceCheck, CancellationToken cancellationToken = default)
		{
			using HttpRequestMessage headObjectRequest = await BuildHttpRequestAsync(HttpMethod.Head, new Uri($"api/v1/blobs/{ns}/{blob}", UriKind.Relative));
			HttpResponseMessage response = await HttpClient.SendAsync(headObjectRequest, cancellationToken);
			if (response.StatusCode == HttpStatusCode.NotFound)
			{
				return false;
			}

			response.EnsureSuccessStatusCode();

			return true;
		}

		public Task DeleteObjectAsync(IEnumerable<NamespaceId> namespaces, BlobId blob, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException("DeleteObjectAsync from multiple namespaces is not supported on the relay blob store");
		}

		public Task DeleteObjectAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException("DeleteObjects is not supported on the relay blob store");
		}

		public Task DeleteNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException("DeleteNamespace is not supported on the relay blob store");
		}

		public IAsyncEnumerable<(BlobId, DateTime)> ListObjectsAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException("ListObjects is not supported on the relay blob store");
		}
	}
}
