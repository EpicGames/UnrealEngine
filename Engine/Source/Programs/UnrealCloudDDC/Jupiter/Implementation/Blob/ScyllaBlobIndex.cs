// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Cassandra;
using Cassandra.Mapping;
using Jupiter.Common;
using Jupiter.Common.Utils;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation.Blob;

public class ScyllaBlobIndex : IBlobIndex
{
	private readonly IScyllaSessionManager _scyllaSessionManager;
	private readonly IOptionsMonitor<JupiterSettings> _jupiterSettings;
	private readonly IOptionsMonitor<ScyllaSettings> _scyllaSettings;
	private readonly INamespacePolicyResolver _namespacePolicyResolver;
	private readonly Tracer _tracer;
	private readonly ISession _session;
	private readonly Mapper _mapper;
	private readonly PreparedStatement? _getBucketStatsStatement;
	private readonly PreparedStatement? _getBucketStatsRefsStatement;

	private readonly PreparedStatement _getBlobReferenceStatement;

	public ScyllaBlobIndex(IScyllaSessionManager scyllaSessionManager, IOptionsMonitor<JupiterSettings> jupiterSettings, IOptionsMonitor<ScyllaSettings> scyllaSettings, INamespacePolicyResolver namespacePolicyResolver, Tracer tracer)
	{
		_scyllaSessionManager = scyllaSessionManager;
		_jupiterSettings = jupiterSettings;
		_scyllaSettings = scyllaSettings;
		_namespacePolicyResolver = namespacePolicyResolver;
		_tracer = tracer;
		_session = scyllaSessionManager.GetSessionForReplicatedKeyspace();
		_mapper = new Mapper(_session);

		if (!scyllaSettings.CurrentValue.AvoidSchemaChanges)
		{
			string blobType = scyllaSessionManager.IsCassandra ? "blob" : "frozen<blob_identifier>";
			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS blob_index (
				namespace text,
				blob_id {blobType},
				regions set<text>,
				references set<frozen<object_reference>>,
				PRIMARY KEY ((namespace, blob_id))
			);"
			));

			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS blob_index_v2 (
				namespace text,
				blob_id blob,
				region text,
				PRIMARY KEY ((namespace, blob_id), region)
			);"
			));

			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS blob_incoming_references (
				namespace text,
				blob_id blob,
				reference_id blob,
				reference_type smallint,
				bucket_id text,
				PRIMARY KEY ((namespace, blob_id), reference_id)
			);"
			));

			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS bucket_referenced_blobs (
				namespace text,
				bucket_id text,
				hash_prefix text,
				blob_id blob,
				size bigint,
				PRIMARY KEY ((namespace, bucket_id, hash_prefix), blob_id)
			);"
			));

			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS bucket_referenced_ref (
				namespace text,
				bucket_id text,
				hash_prefix text,
				reference_id text,
				PRIMARY KEY ((namespace, bucket_id, hash_prefix), reference_id)
			);"
			));
		}

		string cqlOptions = _scyllaSessionManager.IsScylla ? "BYPASS CACHE" : "";
		_getBlobReferenceStatement = _session.Prepare($"SELECT namespace, blob_id, reference_id, reference_type, bucket_id FROM blob_incoming_references WHERE token(namespace, blob_id) >= ? AND token(namespace, blob_id) <= ?  {cqlOptions}");

		if (scyllaSessionManager.IsScylla)
		{
			_getBucketStatsStatement = _session.Prepare("select count(blob_id), min(size), max(size), sum(size) from bucket_referenced_blobs WHERE namespace = ? AND bucket_id = ?  AND hash_prefix = ?");
			_getBucketStatsRefsStatement = _session.Prepare("select count(reference_id) from bucket_referenced_ref WHERE namespace = ? AND bucket_id = ?  AND hash_prefix = ?");
		}
	}

	public async Task AddBlobToIndexAsync(NamespaceId ns, BlobId id, ulong rawSize, string? region = null,
		CancellationToken cancellationToken = default)
	{
		region ??= _jupiterSettings.CurrentValue.CurrentSite;
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.insert_blob_index").SetAttribute("resource.name", $"{ns}.{id}");

		try
		{
			await _mapper.InsertAsync<ScyllaBlobIndexEntry>(new ScyllaBlobIndexEntry(ns.ToString(), id, region));
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	private async Task<List<string>?> GetOldBlobRegionsAsync(NamespaceId ns, BlobId id)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.fetch_old_blob_index").SetAttribute("resource.name", $"{ns}.{id}");

		try
		{
			if (_scyllaSessionManager.IsScylla)
			{
				ScyllaBlobIndexTable o;
				o = await _mapper.SingleOrDefaultAsync<ScyllaBlobIndexTable>("SELECT namespace, blob_id, regions, references FROM blob_index WHERE namespace = ? AND blob_id = ?", ns.ToString(), new ScyllaBlobIdentifier(id));
				return o?.Regions?.ToList();
			}
			else
			{
				CassandraBlobIndexTable o;
				o = await _mapper.SingleOrDefaultAsync<CassandraBlobIndexTable>("SELECT namespace, blob_id, regions, references FROM blob_index WHERE namespace = ? AND blob_id = ?", ns.ToString(), id.HashData);
				return o?.Regions?.ToList();
			}
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	private async Task<List<ScyllaObjectReference>?> GetOldBlobReferencesAsync(NamespaceId ns, BlobId id)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.fetch_old_blob_index_references").SetAttribute("resource.name", $"{ns}.{id}");

		try
		{
			if (_scyllaSessionManager.IsScylla)
			{
				ScyllaBlobIndexTable o;
				o = await _mapper.SingleOrDefaultAsync<ScyllaBlobIndexTable>("SELECT namespace, blob_id, regions, references FROM blob_index WHERE namespace = ? AND blob_id = ?", ns.ToString(), new ScyllaBlobIdentifier(id));
				return o?.References?.ToList();
			}
			else
			{
				CassandraBlobIndexTable o;
				o = await _mapper.SingleOrDefaultAsync<CassandraBlobIndexTable>("SELECT namespace, blob_id, regions, references FROM blob_index WHERE namespace = ? AND blob_id = ?", ns.ToString(), id.HashData);
				return o?.References?.ToList();
			}
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async Task RemoveBlobFromRegionAsync(NamespaceId ns, BlobId id, string? region = null, CancellationToken cancellationToken = default)
	{
		region ??= _jupiterSettings.CurrentValue.CurrentSite;
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.remove_blob_index_region").SetAttribute("resource.name", $"{ns}.{id}");

		try
		{
			await _mapper.DeleteAsync<ScyllaBlobIndexEntry>(new ScyllaBlobIndexEntry(ns.ToString(), id, region));
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async Task RemoveBlobFromAllRegionsAsync(NamespaceId ns, BlobId id, CancellationToken cancellationToken = default)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.remove_blob_index_all_regions").SetAttribute("resource.name", $"{ns}.{id}");

		try
		{
			await _mapper.DeleteAsync<ScyllaBlobIndexEntry>("WHERE namespace = ? AND blob_id = ?", ns.ToString(), id.HashData);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}
	
	public async Task<bool> BlobExistsInRegionAsync(NamespaceId ns, BlobId blobIdentifier, string? region = null, CancellationToken cancellationToken = default)
	{
		region ??= _jupiterSettings.CurrentValue.CurrentSite;
		ScyllaBlobIndexEntry? entry;
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_blob_index").SetAttribute("resource.name", $"{ns}.{blobIdentifier}");

			try
			{
				entry = await _mapper.SingleOrDefaultAsync<ScyllaBlobIndexEntry>("WHERE namespace = ? AND blob_id = ? AND region = ?", ns.ToString(), blobIdentifier.HashData, region);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		bool blobMissing = entry == null;
		if (blobMissing && _scyllaSettings.CurrentValue.MigrateFromOldBlobIndex)
		{
			List<string>? regions = await GetOldBlobRegionsAsync(ns, blobIdentifier);
			if (regions == null)
			{
				// blob didn't exist in the old table either
				return false;
			}

			foreach (string oldRegion in regions)
			{
				await AddBlobToIndexAsync(ns, blobIdentifier, rawSize: 0, region: oldRegion, cancellationToken: cancellationToken);
			}

			// blob has been migrated so it existed
			return true;
		}

		return !blobMissing;
	}
	public async IAsyncEnumerable<(NamespaceId, BlobId)> GetAllBlobsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_all_blobs");

		string cqlOptions = _scyllaSessionManager.IsScylla ? "BYPASS CACHE" : "";

		IEnumerable<ScyllaBlobIndexEntry> entries;
		try
		{
			entries = await _mapper.FetchAsync<ScyllaBlobIndexEntry>(cqlOptions);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
		
		foreach (ScyllaBlobIndexEntry blobIndex in entries)
		{
			yield return (new NamespaceId(blobIndex.Namespace), new BlobId(blobIndex.BlobId));
		}
	}

	public async IAsyncEnumerable<(NamespaceId, BaseBlobReference)> GetAllBlobReferencesAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_all_blob_references");

		// generate a list of all the primary key ranges that exist on the cluster
		List<(long, long)> tableRanges = ScyllaUtils.GetTableRanges(_scyllaSettings.CurrentValue.CountOfNodes, _scyllaSettings.CurrentValue.CountOfCoresPerNode, 3).ToList();

		// randomly shuffle this list so that we do not scan them in the same order, means that we will eventually visit all ranges even if the process running this is restarted before we have finished
		List<int> tableRangeIndices = Enumerable.Range(0, tableRanges.Count).ToList();
		tableRangeIndices.Shuffle();

		foreach (int index in tableRangeIndices)
		{
			(long, long) range = tableRanges[index];

			RowSet rowSet;
			try
			{
				rowSet = await _session.ExecuteAsync(_getBlobReferenceStatement.Bind(range.Item1, range.Item2));
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
			
			foreach (Row row in rowSet)
			{
				string ns = row.GetValue<string>("namespace");
				string bucketId = row.GetValue<string>("bucket_id");
				short referenceTypeValue = row.GetValue<short>("reference_type");
				byte[] blobId = row.GetValue<byte[]>("blob_id");
				byte[] referenceId = row.GetValue<byte[]>("reference_id");

				ScyllaBlobIncomingReference.BlobReferenceType referenceType = (ScyllaBlobIncomingReference.BlobReferenceType)referenceTypeValue;
				if (referenceType == ScyllaBlobIncomingReference.BlobReferenceType.Ref)
				{
					yield return (new NamespaceId(ns), new RefBlobReference(new BlobId(blobId), new BucketId(bucketId), new RefId(StringUtils.FormatAsHexString(referenceId))));
				}
				else if (referenceType == ScyllaBlobIncomingReference.BlobReferenceType.Blob)
				{
					yield return (new NamespaceId(ns), new BlobToBlobReference(new BlobId(blobId), new BlobId(referenceId)));
				}
			}
		}
	}

	public async Task<List<string>> GetBlobRegionsAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken)
	{
		List<string> regions = new List<string>();

		IEnumerable<ScyllaBlobIndexEntry> entries;
		try
		{
			entries = await _mapper.FetchAsync<ScyllaBlobIndexEntry>("WHERE namespace = ? AND blob_id = ?", ns.ToString(), blob.HashData);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
		
		foreach (ScyllaBlobIndexEntry blobIndex in entries)
		{
			regions.Add(blobIndex.Region);
		}

		if (regions.Count == 0 && _scyllaSettings.CurrentValue.MigrateFromOldBlobIndex)
		{
			List<string>? oldRegions = await GetOldBlobRegionsAsync(ns, blob);
			if (oldRegions == null)
			{
				// regions didn't exist in the old table either
				return regions;
			}

			foreach (string oldRegion in oldRegions)
			{
				await AddBlobToIndexAsync(ns, blob, rawSize: 0, region: oldRegion, cancellationToken: cancellationToken);
			}

			// blob has been migrated lets return the old region list
			regions = oldRegions;
		}
		if (regions.Count == 0)
		{
			throw new BlobNotFoundException(ns, blob);
		}

		return regions;
	}

	public async Task AddBlobReferencesAsync(NamespaceId ns, BlobId sourceBlob, BlobId targetBlob, CancellationToken cancellationToken)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.add_blob_to_blob_ref");

		string nsAsString = ns.ToString();
		ScyllaBlobIncomingReference incomingReference = new ScyllaBlobIncomingReference(nsAsString, sourceBlob, targetBlob);

		try
		{
			await _mapper.InsertAsync<ScyllaBlobIncomingReference>(incomingReference);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async IAsyncEnumerable<BaseBlobReference> GetBlobReferencesAsync(NamespaceId ns, BlobId id, [EnumeratorCancellation] CancellationToken cancellationToken)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_blob_references").SetAttribute("resource.name", $"{ns}.{id}");

		// the references are rarely read so we bypass cache to reduce churn in it
		string cqlOptions = _scyllaSessionManager.IsScylla ? "BYPASS CACHE" : "";

		bool noReferencesFound = true;

		IEnumerable<ScyllaBlobIncomingReference>? references = null;
		const int MaxRetryAttempts = 3;
		Exception? timeoutException = null;
		for (int i = 0; i < MaxRetryAttempts; i++)
		{
			try
			{
				timeoutException = null;

				references = await _mapper.FetchAsync<ScyllaBlobIncomingReference>("WHERE namespace = ? AND blob_id = ? " + cqlOptions, ns.ToString(), id.HashData);
				break;
			}
			catch (ReadTimeoutException e)
			{
				// wait 2 seconds and try again as the Db is under heavy load right now
				await Task.Delay(TimeSpan.FromSeconds(2), cancellationToken);
				timeoutException = e;
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		if (timeoutException != null)
		{
			throw new RemoteServiceException("Error communicating with Scylla", timeoutException);
		}

		if (references == null)
		{
			throw new NotImplementedException("References not set, unhandled exception?");
		}

		foreach (ScyllaBlobIncomingReference incomingReference in references)
		{
			noReferencesFound = false;
			ScyllaBlobIncomingReference.BlobReferenceType referenceType = (ScyllaBlobIncomingReference.BlobReferenceType)incomingReference.ReferenceType;
			if (referenceType == ScyllaBlobIncomingReference.BlobReferenceType.Ref)
			{
				if (incomingReference.BucketId == null)
				{
					continue;
				}
				yield return new RefBlobReference(new BlobId(incomingReference.BlobId), new BucketId(incomingReference.BucketId), new RefId(StringUtils.FormatAsHexString(incomingReference.ReferenceId)));
			}
			else if (referenceType == ScyllaBlobIncomingReference.BlobReferenceType.Blob)
			{
				yield return new BlobToBlobReference(new BlobId(incomingReference.BlobId), new BlobId(incomingReference.ReferenceId));
			}
			else
			{
				throw new NotImplementedException("Unknown blob reference type");
			}
		}

		if (noReferencesFound && _scyllaSettings.CurrentValue.MigrateFromOldBlobIndex)
		{
			List<ScyllaObjectReference>? oldReferences = await GetOldBlobReferencesAsync(ns, id);
			if (oldReferences != null)
			{
				foreach (ScyllaObjectReference scyllaObjectReference in oldReferences)
				{
					BucketId bucket = new BucketId(scyllaObjectReference.Bucket);
					RefId key = new RefId(scyllaObjectReference.Key);
					await AddRefToBlobsAsync(ns, bucket, key, new[] { id }, cancellationToken);
					yield return new RefBlobReference(id, bucket, key);
				}
			}
		}
	}

	public async Task AddRefToBlobsAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId[] blobs, CancellationToken cancellationToken)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.add_ref_blobs");

		string nsAsString = ns.ToString();
		Task[] refUpdateTasks = new Task[blobs.Length];
		for (int i = 0; i < blobs.Length; i++)
		{
			BlobId id = blobs[i];
			ScyllaBlobIncomingReference incomingReference = new ScyllaBlobIncomingReference(nsAsString, id, key, bucket);

			refUpdateTasks[i] = _mapper.InsertAsync<ScyllaBlobIncomingReference>(incomingReference);
		}

		try
		{
			await Task.WhenAll(refUpdateTasks);
		}
		catch (AggregateException e)
		{
			if (e.InnerExceptions.Any(exception => exception.GetType() == typeof(QueryExecutionException)))
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}

			throw;
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async Task RemoveReferencesAsync(NamespaceId ns, BlobId id, List<BaseBlobReference>? referencesToRemove, CancellationToken cancellationToken)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.remove_ref_blobs");

		string nsAsString = ns.ToString();

		if (referencesToRemove == null)
		{
			try
			{
				await _mapper.DeleteAsync<ScyllaBlobIncomingReference>("WHERE namespace = ? AND blob_id = ?", nsAsString, id.HashData);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}
		else
		{
			int countOfReferencesToRemove = referencesToRemove.Count;
			Task[] removeRefTasks = new Task[countOfReferencesToRemove];
			for (int i = 0; i < countOfReferencesToRemove; i++)
			{
				BaseBlobReference baseRef = referencesToRemove[i];
				byte[] refId;
				if (baseRef is RefBlobReference refBlobReference)
				{
					refId = StringUtils.ToHashFromHexString(refBlobReference.Key.ToString());
				}
				else if (baseRef is BlobToBlobReference blobToBlobReference)
				{
					refId = blobToBlobReference.Blob.HashData;
				}
				else
				{
					throw new NotImplementedException("Unknown blob reference type");
				}

				removeRefTasks[i] = _mapper.DeleteAsync<ScyllaBlobIncomingReference>("WHERE namespace = ? AND blob_id = ? AND reference_id = ?", nsAsString, id.HashData, refId);
			}

			await Task.WhenAll(removeRefTasks);
		}
	}

	public async Task AddBlobToBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobId, long blobSize, CancellationToken cancellationToken)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.add_bucket_blob");

		int? ttl = null;
		NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
		NamespacePolicy.StoragePoolGCMethod gcMethod = policy.GcMethod ?? NamespacePolicy.StoragePoolGCMethod.LastAccess;
		if (gcMethod == NamespacePolicy.StoragePoolGCMethod.TTL)
		{
			ttl = (int)policy.DefaultTTL.TotalSeconds;
		}

		string nsAsString = ns.ToString();
		ScyllaBucketReferencedBlob blobRef = new ScyllaBucketReferencedBlob(nsAsString, bucket, blobId, blobSize);
		Task insertBlobRefTask = _mapper.InsertAsync<ScyllaBucketReferencedBlob>(blobRef, ttl: ttl, insertNulls: false);

		ScyllaBucketReferencedRef refReference = new ScyllaBucketReferencedRef(nsAsString, bucket, key);
		Task insertRefRefTask = _mapper.InsertAsync<ScyllaBucketReferencedRef>(refReference, ttl: ttl, insertNulls: false);

		await Task.WhenAll(insertRefRefTask, insertBlobRefTask);
	}

	public async Task RemoveBlobFromBucketListAsync(NamespaceId ns, BucketId bucket, RefId key, List<BlobId> blobIds, CancellationToken cancellationToken)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.remove_bucket_blob");

		string nsAsString = ns.ToString();
		List<Task> deleteBlobTasks = new List<Task>();
		foreach (BlobId blobId in blobIds)
		{
			ScyllaBucketReferencedBlob blobRef = new ScyllaBucketReferencedBlob(nsAsString, bucket, blobId);
			deleteBlobTasks.Add(_mapper.DeleteAsync<ScyllaBucketReferencedBlob>(blobRef));
		}

		ScyllaBucketReferencedRef refReference = new ScyllaBucketReferencedRef(nsAsString, bucket, key);
		Task deleteRefRefTask = _mapper.DeleteAsync<ScyllaBucketReferencedRef>(refReference);

		await Task.WhenAll(deleteBlobTasks);
		await deleteRefRefTask;
	}

	public async Task<BucketStats> CalculateBucketStatisticsAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
	{
		string nsAsString = ns.ToString();
		string bucketAsString = bucket.ToString();

		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.calc_bucket_stats")
			.SetAttribute("resource.name", $"{nsAsString}.{bucketAsString}");

		long totalCountOfRefs = 0;
		long totalCountOfBlobs = 0;
		long totalSizeOfBlobs = 0;

		List<long> smallestBlobPerPrefix = new List<long>();
		List<long> largestBlobPerPrefix = new List<long>();

		string[] hashPrefixes = new string[65536];
		int i = 0;
		for (int a = 0; a <= byte.MaxValue; a++)
		{
			for (int b = 0; b <= byte.MaxValue; b++)
			{
				hashPrefixes[i] = StringUtils.FormatAsHexString(new byte[] { (byte)a, (byte)b }).ToLower();
				i++;
			}
		}

		Debug.Assert(i == 65536);

		if (_getBucketStatsRefsStatement == null || _getBucketStatsStatement == null)
		{
			throw new Exception("Calculating bucket statistics is not supported when not using Scylla");
		}

		const int DegreeOfParallelism = 32;
		await Parallel.ForEachAsync(hashPrefixes, new ParallelOptions { MaxDegreeOfParallelism = DegreeOfParallelism }, async (hashPrefix, token) =>
		{
			Task calcRefStats = Task.Run(async () =>
			{
				BoundStatement? boundStatement = _getBucketStatsRefsStatement.Bind(nsAsString, bucketAsString, hashPrefix);
				RowSet? rowSet = await _session.ExecuteAsync(boundStatement);
				foreach (Row row in rowSet)
				{
					long countOfRefs = (long)row["system.count(reference_id)"];

					Interlocked.Add(ref totalCountOfRefs, countOfRefs);
				}
			}, token);

			Task calcBlobStats = Task.Run(async () =>
			{
				using TelemetrySpan _ = _tracer.BuildScyllaSpan("scylla.calc_bucket_stats_shard")
					.SetAttribute("resource.name", $"{nsAsString}.{bucketAsString}.{hashPrefix}");
				// blob stats
				BoundStatement? boundStatement = _getBucketStatsStatement.Bind(nsAsString, bucketAsString, hashPrefix);

				RowSet? rowSet = await _session.ExecuteAsync(boundStatement);

				bool rowFetched = false;
				foreach (Row row in rowSet)
				{
					if (rowFetched)
					{
						throw new Exception("Multiple rows when fetching bucket stats, this is not expected");
					}

					rowFetched = true;

					if (row["system.min(size)"] == null)
					{
						// no results
						continue;
					}

					long countOfBlobs = (long)row["system.count(blob_id)"];
					long smallestBlob = (long)row["system.min(size)"];
					long largestBlob = (long)row["system.max(size)"];
					long sumSizeOfBlobs = (long)row["system.sum(size)"];

					Interlocked.Add(ref totalCountOfBlobs, countOfBlobs);
					Interlocked.Add(ref totalSizeOfBlobs, sumSizeOfBlobs);

					smallestBlobPerPrefix.Add(smallestBlob);
					largestBlobPerPrefix.Add(largestBlob);
				}
			}, token);

			await Task.WhenAll(calcRefStats, calcBlobStats);
		});
		long smallestBlobFound = smallestBlobPerPrefix.Any() ? smallestBlobPerPrefix.Min() : 0;
		long largestBlobFound = largestBlobPerPrefix.Any() ? largestBlobPerPrefix.Max() : 0;

		return new BucketStats
		{
			Namespace = ns,
			Bucket = bucket,
			CountOfRefs = totalCountOfRefs,
			CountOfBlobs = totalCountOfBlobs,
			SmallestBlobFound = smallestBlobFound,
			LargestBlob = largestBlobFound,
			TotalSize = totalSizeOfBlobs,
#pragma warning disable CA1508
			AvgSize = totalCountOfBlobs == 0 ? 0 : (totalSizeOfBlobs / (double)totalCountOfBlobs)
#pragma warning restore CA1508
		};
	}
}

[Cassandra.Mapping.Attributes.Table("blob_index")]
class ScyllaBlobIndexTable
{
	public ScyllaBlobIndexTable()
	{
		Namespace = null!;
		BlobId = null!;
		Regions = null;
		References = null;
	}

	public ScyllaBlobIndexTable(string @namespace, BlobId blobId, HashSet<string> regions, List<ScyllaObjectReference> references)
	{
		Namespace = @namespace;
		BlobId = new ScyllaBlobIdentifier(blobId);
		Regions = regions;
		References = references;
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.PartitionKey]
	[Cassandra.Mapping.Attributes.Column("blob_id")]
	public ScyllaBlobIdentifier BlobId { get; set; }

	[Cassandra.Mapping.Attributes.Column("regions")]
	public HashSet<string>? Regions { get; set; }

	[Cassandra.Mapping.Attributes.Column("references")]
	public List<ScyllaObjectReference>? References { get; set; }
}

[Cassandra.Mapping.Attributes.Table("blob_index")]
class CassandraBlobIndexTable
{
	public CassandraBlobIndexTable()
	{
		Namespace = null!;
		BlobId = null!;
		Regions = null;
		References = null;
	}

	public CassandraBlobIndexTable(string @namespace, BlobId blobId, HashSet<string> regions, List<ScyllaObjectReference> references)
	{
		Namespace = @namespace;
		BlobId = blobId.HashData;
		Regions = regions;
		References = references;
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.PartitionKey]
	[Cassandra.Mapping.Attributes.Column("blob_id")]
	public byte[] BlobId { get; set; }

	[Cassandra.Mapping.Attributes.Column("regions")]
	public HashSet<string>? Regions { get; set; }

	[Cassandra.Mapping.Attributes.Column("references")]
	public List<ScyllaObjectReference>? References { get; set; }
}

[Cassandra.Mapping.Attributes.Table("blob_index_v2")]
class ScyllaBlobIndexEntry
{
	public ScyllaBlobIndexEntry()
	{
		Namespace = null!;
		BlobId = null!;
		Region = null!;
	}

	public ScyllaBlobIndexEntry(string @namespace, BlobId blobId, string region)
	{
		Namespace = @namespace;
		BlobId = blobId.HashData;
		Region = region;
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.PartitionKey]
	[Cassandra.Mapping.Attributes.Column("blob_id")]
	public byte[] BlobId { get; set; }

	[Cassandra.Mapping.Attributes.ClusteringKey]
	[Cassandra.Mapping.Attributes.Column("region")]
	public string Region { get; set; }
}

[Cassandra.Mapping.Attributes.Table("blob_incoming_references")]
class ScyllaBlobIncomingReference
{
	public enum BlobReferenceType { Ref = 0, Blob = 1 };

	public ScyllaBlobIncomingReference()
	{
		Namespace = null!;
		BlobId = null!;
		ReferenceId = null!;
		BucketId = null!;
	}

	public ScyllaBlobIncomingReference(string @namespace, BlobId blobId, BlobId referenceId)
	{
		Namespace = @namespace;
		BlobId = blobId.HashData;

		BucketId = null;
		ReferenceId = referenceId.HashData;
		ReferenceType = (int)BlobReferenceType.Blob;
	}

	public ScyllaBlobIncomingReference(string @namespace, BlobId blobId, RefId referenceId, BucketId bucketId)
	{
		Namespace = @namespace;
		BlobId = blobId.HashData;

		BucketId = bucketId.ToString();
		ReferenceId = StringUtils.ToHashFromHexString(referenceId.ToString());
		ReferenceType = (int)BlobReferenceType.Ref;
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.PartitionKey]
	[Cassandra.Mapping.Attributes.Column("blob_id")]
	public byte[] BlobId { get; set; }

	[Cassandra.Mapping.Attributes.ClusteringKey]
	[Cassandra.Mapping.Attributes.Column("reference_id")]
	public byte[] ReferenceId { get; set; }

	[Cassandra.Mapping.Attributes.Column("bucket_id")]
	public string? BucketId { get; set; }

	[Cassandra.Mapping.Attributes.Column("reference_type")]
	public short ReferenceType { get; set; }
}

[Cassandra.Mapping.Attributes.Table("bucket_referenced_blobs")]
class ScyllaBucketReferencedBlob
{
	public ScyllaBucketReferencedBlob()
	{
		Namespace = null!;
		Bucket = null!;
		HashPrefix = null!;
		BlobId = null!;
		Size = 0;
	}

	public ScyllaBucketReferencedBlob(string @namespace, BucketId bucket, BlobId blobId)
	{
		Namespace = @namespace;
		Bucket = bucket.ToString();
		HashPrefix = StringUtils.FormatAsHexLowerString(blobId.HashData).Substring(0, 4);
		BlobId = blobId.HashData;
		Size = 0;
	}

	public ScyllaBucketReferencedBlob(string @namespace, BucketId bucket, BlobId blobId, long blobSize)
	{
		Namespace = @namespace;
		Bucket = bucket.ToString();
		HashPrefix = StringUtils.FormatAsHexLowerString(blobId.HashData).Substring(0, 4);
		BlobId = blobId.HashData;
		Size = blobSize;
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.PartitionKey]
	[Cassandra.Mapping.Attributes.Column("bucket_id")]
	public string Bucket { get; set; }

	[Cassandra.Mapping.Attributes.PartitionKey]
	[Cassandra.Mapping.Attributes.Column("hash_prefix")]
	public string HashPrefix { get; set; }

	[Cassandra.Mapping.Attributes.ClusteringKey]
	[Cassandra.Mapping.Attributes.Column("blob_id")]
	public byte[] BlobId { get; set; }

	[Cassandra.Mapping.Attributes.Column("size")]
	public long Size { get; set; }
}

[Cassandra.Mapping.Attributes.Table("bucket_referenced_ref")]
class ScyllaBucketReferencedRef
{
	public ScyllaBucketReferencedRef()
	{
		Namespace = null!;
		Bucket = null!;
		HashPrefix = null!;
		ReferenceId = null!;
	}

	public ScyllaBucketReferencedRef(string @namespace, BucketId bucket, RefId refId)
	{
		Namespace = @namespace;
		Bucket = bucket.ToString();
		HashPrefix = refId.ToString().Substring(0, 4);
		ReferenceId = refId.ToString();
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.PartitionKey]
	[Cassandra.Mapping.Attributes.Column("bucket_id")]
	public string Bucket { get; set; }

	[Cassandra.Mapping.Attributes.PartitionKey]
	[Cassandra.Mapping.Attributes.Column("hash_prefix")]
	public string HashPrefix { get; set; }

	[Cassandra.Mapping.Attributes.ClusteringKey]
	[Cassandra.Mapping.Attributes.Column("reference_id")]
	public string ReferenceId { get; set; }
}
