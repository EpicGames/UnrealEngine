// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class BuildStoreConsistencyState
	{
	}

	// ReSharper disable once ClassNeverInstantiated.Global
	public class BuildStoreConsistencyCheckService : PollingService<BuildStoreConsistencyState>
	{
		private readonly IOptionsMonitor<ConsistencyCheckSettings> _settings;
		private readonly ILeaderElection _leaderElection;
		private readonly IBuildStore _buildStore;
		private readonly IBlobService _blobService;
		private readonly IBlockStore _blockStore;
		private readonly IRefService _refService;
		private readonly IReferenceResolver _referenceResolver;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;

		protected override bool ShouldStartPolling()
		{
			return _settings.CurrentValue.EnableBuildStoreConsistencyCheck;
		}

		public BuildStoreConsistencyCheckService(IOptionsMonitor<ConsistencyCheckSettings> settings, ILeaderElection leaderElection, IBuildStore buildStore, IBlobService blobService, IBlockStore blockStore, IRefService refService, IReferenceResolver referenceResolver, Tracer tracer, ILogger<BuildStoreConsistencyCheckService> logger) : base(serviceName: nameof(BuildStoreConsistencyCheckService), TimeSpan.FromSeconds(settings.CurrentValue.ConsistencyCheckPollFrequencySeconds), new BuildStoreConsistencyState(), logger)
		{
			_settings = settings;
			_leaderElection = leaderElection;
			_buildStore = buildStore;
			_blobService = blobService;
			_blockStore = blockStore;
			_refService = refService;
			_referenceResolver = referenceResolver;
			_tracer = tracer;
			_logger = logger;
		}

		public override async Task<bool> OnPollAsync(BuildStoreConsistencyState state, CancellationToken cancellationToken)
		{
			bool shouldEnable = _settings.CurrentValue.EnableBuildStoreConsistencyCheck;
			if (!shouldEnable)
			{
				_logger.LogInformation("Skipped running build store consistency check as it is disabled");
				return false;
			}

			if (!_leaderElection.IsThisInstanceLeader())
			{
				_logger.LogInformation("Skipped running build store consistency check because this instance was not the leader");
				return false;
			}

			await RunConsistencyCheckAsync(cancellationToken);

			return true;
		}
		
		private async Task RunConsistencyCheckAsync(CancellationToken cancellationToken)
		{
			ulong countOfBuildsChecked = 0;
			ulong countOfUnfinalizedBuilds = 0;
			ulong countOfIncorrectBuildsFound = 0;
			ulong countOfUnresolvableBlobs = 0;
			ulong countOfBlobsChecked = 0;
			ulong countOfMissingBlobsFound = 0;
			ulong countOfBlocksChecked = 0;
			ulong countOfMissingBlocksFound = 0;
			ulong countOfBlobErrors = 0;
			ulong countOfRefErrors = 0;
			await Parallel.ForEachAsync(_buildStore.ListAllBuildsAsync(cancellationToken), new ParallelOptions
			{
				CancellationToken = cancellationToken,
				MaxDegreeOfParallelism = _settings.CurrentValue.BuildStoreConsistencyMaxParallelBuilds,
			},
				async (tuple, token) =>
				{
					(NamespaceId ns, BucketId bucket, CbObjectId buildId, bool isFinalized) = tuple;
					Interlocked.Increment(ref countOfBuildsChecked);

					if (countOfBuildsChecked % 100 == 0)
					{
						_logger.LogInformation("Consistency check running on build store, count of builds processed so far: {CountOfBuilds}", countOfBuildsChecked);
					}

					using TelemetrySpan scope = _tracer.StartActiveSpan("consistency_check.build_store").SetAttribute("resource.name", $"{ns}.{bucket}.{buildId}").SetAttribute("operation.name", "consistency_check.build_store");

					if (_settings.CurrentValue.CheckBuildStorePartialObjects)
					{
						BuildRecord? record = await _buildStore.GetBuildAsync(ns, bucket, buildId);
						if (record == null)
						{
							_logger.LogWarning("Partial build {BuildId} in bucket {Bucket} and namespace {Namespace} .", buildId, bucket, ns);

							Interlocked.Increment(ref countOfIncorrectBuildsFound);
							if (_settings.CurrentValue.AllowDeletesInBuildStore)
							{
								await _buildStore.DeleteBuild(ns, bucket, buildId);
							}

							return;
						}
					}

					if (_settings.CurrentValue.CheckBuildStoreMissingBlobs)
					{
						if (!isFinalized)
						{
							_logger.LogInformation("Build {BuildId} in {Namespace}.{Bucket} is not finalized. Skipping.", buildId, ns, bucket);
							Interlocked.Increment(ref countOfUnfinalizedBuilds);
							return;
						}

						await foreach ((string _, CbObjectId partId) in _buildStore.GetBuildPartsAsync(ns, bucket, buildId).WithCancellation(token))
						{
							RefId refId = RefId.FromName($"{buildId}/{partId}");
							
							try
							{
								List<BlobId> referencedBlobs = await _refService.GetReferencedBlobsAsync(ns, bucket, refId, ignoreMissingBlobs: true, cancellationToken: token);

								await Parallel.ForEachAsync(referencedBlobs, new ParallelOptions
								{
									CancellationToken = token,
									MaxDegreeOfParallelism = _settings.CurrentValue.BuildStoreConsistencyMaxParallelBlobs
								},
									async (referencedBlobId, innerToken) =>
									{
										BlobId[]? blobIds = await _referenceResolver.ResolveIdAsync(ns, referencedBlobId, cancellationToken: innerToken);
										if (blobIds == null)
										{
											_logger.LogWarning("Unable to resolve blobId for referenced blob {ReferencedBlobId} in {Namespace}.{Bucket}.{BuildId}.{PartId}.", referencedBlobId, ns, bucket, buildId, partId);
											Interlocked.Increment(ref countOfUnresolvableBlobs);
											return;
										}

										foreach (BlobId blobId in blobIds)
										{
											Interlocked.Increment(ref countOfBlobsChecked);

											if (!await _blobService.ExistsAsync(ns, blobId, cancellationToken: innerToken))
											{
												_logger.LogWarning("Found missing blob {BlobId} in {Namespace}.{Bucket}.{BuildId}.{PartId}.", blobId, ns, bucket, buildId, partId);
												Interlocked.Increment(ref countOfMissingBlobsFound);
											}

											BlobId? metadataHash = await _blockStore.GetBlockMetadataAsync(ns, blobId);

											if (metadataHash != null)
											{
												Interlocked.Increment(ref countOfBlocksChecked);
												
												if (!await _blobService.ExistsAsync(ns, metadataHash, cancellationToken: innerToken))
												{
													_logger.LogWarning("Found missing block metadata {MetadataHash} for blob {BlobId} in {Namespace}.{Bucket}.{BuildId}.{PartId}.", metadataHash, blobId, ns, bucket, buildId, partId);
													Interlocked.Increment(ref countOfMissingBlocksFound);
												}
											}
										}
									}
								);
							}
							catch (BlobNotFoundException e)
							{
								_logger.LogWarning(e, "Could not retrieve referenced blobs for {RefId} in {Namespace}.{Bucket}", refId, ns, bucket);
								Interlocked.Increment(ref countOfBlobErrors);
							}
							catch (RefNotFoundException e)
							{
								_logger.LogWarning(e, "Could not retrieve referenced blobs for {RefId} in {Namespace}.{Bucket}", refId, ns, bucket);
								Interlocked.Increment(ref countOfRefErrors);
							}
						}
					}
				}
			);

			if (_settings.CurrentValue.CheckBuildStorePartialObjects)
			{
				_logger.LogInformation("Build Store Consistency check finished, found {CountOfIncorrectBuilds} incorrect builds. Processed {CountOfBuilds} builds.", countOfIncorrectBuildsFound, countOfBuildsChecked);
			}
			
			if (_settings.CurrentValue.CheckBuildStoreMissingBlobs)
			{
				_logger.LogInformation("Build Store Consistency check finished. Found {CountOfUnresolvableBlobs} unresolvable blobs. Found {CountOfMissingBlobsFound} missing blobs ({CountOfBlobsChecked} total). Found {CountOfMissingBlocksFound} missing blocks ({CountOfBlocksChecked} total). Encountered {CountOfBlobErrors} BlobNotFoundExceptions. Encountered {CountOfRefErrors} RefNotFoundExceptions. Processed {CountOfBuilds} builds ({CountOfUnfinalizedBuilds} not finalized).", countOfUnresolvableBlobs, countOfMissingBlobsFound, countOfBlobsChecked, countOfMissingBlocksFound, countOfBlocksChecked, countOfBlobErrors, countOfRefErrors, countOfBuildsChecked, countOfUnfinalizedBuilds);
			}
		}

		protected override Task OnStopping(BuildStoreConsistencyState state)
		{
			return Task.CompletedTask;
		}
	}
}
