// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using Jupiter.Common;
using EpicGames.Serialization;

namespace Jupiter.Implementation
{
	public class BlockMetadata
	{
		public DateTime LastUpdate { get; set; }
		public BlobId MetadataBlobId { get; set; } = null!;
	}

	public class RequiredFieldMissingException : Exception
	{
		public List<string> MissingFields { get; init; }

		public RequiredFieldMissingException(List<string> missingFields) : base($"The following required fields were missing or empty: {string.Join(",", missingFields)}")
		{
			MissingFields = missingFields;
		}
	}

	public class BlockContext
	{
		private readonly string _context;

		public BlockContext(string context)
		{
			_context = context;
		}

		public static BlockContext FromObject(CbObject o, bool useBaseBranch = false)
		{
			List<string> missingFields = new List<string>();
			// technically the name is not required for the block context but we do want all builds to always contain them anyway
			if (o.Find("name").Equals(CbField.Empty))
			{
				missingFields.Add("name");
			}
			if (o.Find("branch").Equals(CbField.Empty))
			{
				missingFields.Add("branch");
			}
			if (o.Find("baselineBranch").Equals(CbField.Empty))
			{
				missingFields.Add("baselineBranch");
			}
			// baseline branch must have a non-empty string value
			if (string.IsNullOrEmpty(o.Find("baselineBranch").AsString()))
			{
				missingFields.Add("baselineBranch");
			}
			if (o.Find("platform").Equals(CbField.Empty))
			{
				missingFields.Add("platform");
			}
			if (o.Find("project").Equals(CbField.Empty))
			{
				missingFields.Add("project");
			}

			if (missingFields.Any())
			{
				throw new RequiredFieldMissingException(missingFields);
			}

			string branchKey = useBaseBranch ? "baselineBranch" : "branch";
			string context = $"{o["project"].AsString()}.{o[branchKey].AsString()}.{o["platform"].AsString()}";
			return new BlockContext(context);
		}

		public override string ToString()
		{
			return _context;
		}
	}

	public interface IBlockStore
	{
		IAsyncEnumerable<BlockMetadata> ListBlockIndexAsync(NamespaceId ns, BlockContext blockContext);
		Task AddBlockToContextAsync(NamespaceId ns, BlockContext blockContext, BlobId metadataBlockId);
		Task PutBlockMetadataAsync(NamespaceId ns, BlobId blockIdentifier, BlobId metadataObjectId);
		Task<BlobId?> GetBlockMetadataAsync(NamespaceId ns, BlobId blockIdentifier);
		Task DeleteBlockAsync(NamespaceId ns, BlobId blockIdentifier);
	}
}
