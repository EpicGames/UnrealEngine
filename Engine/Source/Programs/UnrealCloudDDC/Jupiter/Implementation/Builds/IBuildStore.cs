// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using EpicGames.Serialization;

namespace Jupiter.Implementation;

public interface IBuildStore
{
	Task PutBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObject buildObject, uint ttl);
	Task FinalizeBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, uint ttl);
	Task<BuildRecord?> GetBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId);
	IAsyncEnumerable<BuildMetadata> ListBuildsAsync(NamespaceId ns, BucketId bucket, bool includeTTL = false);

	Task PutBuildPartAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObjectId partId, string partName, uint ttl);
	IAsyncEnumerable<(string, CbObjectId)> GetBuildPartsAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId);

	Task PutBuildAppendixAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string appendixName, uint ttl);
	IAsyncEnumerable<string> GetBuildAppendixesAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId);
	Task DeleteBuildAppendixAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string appendixFileName);

	Task UpdateTTL(NamespaceId ns, BucketId bucket, CbObjectId buildId, uint ttl);
	Task DeleteBuild(NamespaceId ns, BucketId bucket, CbObjectId buildId);
	Task<uint?> GetTTL(NamespaceId ns, BucketId bucket, CbObjectId buildId);
	IAsyncEnumerable<(NamespaceId, BucketId, CbObjectId, bool)> ListAllBuildsAsync(CancellationToken cancellationToken);
}

public class BuildMetadata
{
	public BuildMetadata(CbObjectId buildId, BucketId bucket, CbObject metadata, bool isFinalized, uint? ttl)
	{
		BuildId = buildId;
		Bucket = bucket;
		Metadata = metadata;
		IsFinalized = isFinalized;
		Ttl = ttl;
	}

	public CbObjectId BuildId { get; set; }
	public CbObject Metadata { get; set; } = null!;
	public bool IsFinalized { get; set; }
	public uint? Ttl { get; set; } = null;
	public BucketId Bucket { get; set; }
}

public class BuildRecord
{
	public bool IsFinalized { get; set; }
	public CbObject BuildObject { get; set; } = CbObject.Empty;

}