// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation;

public class MemoryBlockStore : IBlockStore
{
	private readonly ConcurrentDictionary<string, MemoryBlockContext> _blockContext = new();
	private readonly ConcurrentDictionary<string, BlobId> _blockMetadata = new();

	public async IAsyncEnumerable<BlockMetadata> ListBlockIndexAsync(NamespaceId ns, BlockContext blockContext)
	{
		await Task.CompletedTask;

		if (_blockContext.TryGetValue(BuildKey(ns, blockContext), out MemoryBlockContext? o))
		{
			foreach (BlockMetadata block in o.Blocks.Values.OrderByDescending(metadata => metadata.LastUpdate))
			{
				yield return block;
			}
		}
	}

	public Task AddBlockToContextAsync(NamespaceId ns, BlockContext blockContext, BlobId metadataBlockId)
	{
		string key = BuildKey(ns, blockContext);

		_blockContext.AddOrUpdate(key, _ =>
		{
			MemoryBlockContext o = new() { Blocks = { [metadataBlockId] = new BlockMetadata{LastUpdate = DateTime.Now, MetadataBlobId = metadataBlockId} } };
			return o;
		}, (s, o) =>
		{
			o.Blocks[metadataBlockId] = new BlockMetadata{LastUpdate = DateTime.Now, MetadataBlobId = metadataBlockId};
			return o;
		});

		return Task.CompletedTask;
	}

	public Task PutBlockMetadataAsync(NamespaceId ns, BlobId blockIdentifier, BlobId metadataObjectId)
	{
		string key = BuildKeyBlockMetadata(ns, blockIdentifier);

		_blockMetadata.AddOrUpdate(key, _ => metadataObjectId,  (_, _) => metadataObjectId);

		return Task.CompletedTask;
	}

	public Task<BlobId?> GetBlockMetadataAsync(NamespaceId ns, BlobId blockIdentifier)
	{
		string key = BuildKeyBlockMetadata(ns, blockIdentifier);

		return Task.FromResult(_blockMetadata.TryGetValue(key, out BlobId? blobId) ? blobId : null);
	}

	public Task DeleteBlockAsync(NamespaceId ns, BlobId blockIdentifier)
	{
		string key = BuildKeyBlockMetadata(ns, blockIdentifier);
		_blockMetadata.TryRemove(key, out BlobId? _);
		return Task.CompletedTask;
	}

	private static string BuildKey(NamespaceId ns, BlockContext blockContext)
	{
		return $"{ns}.{blockContext}";
	}

	private static string BuildKeyBlockMetadata(NamespaceId ns, BlobId blobId)
	{
		return $"{ns}.{blobId}";
	}
}

public class MemoryBlockContext
{
	public ConcurrentDictionary<BlobId, BlockMetadata> Blocks { get; } = new();
}
