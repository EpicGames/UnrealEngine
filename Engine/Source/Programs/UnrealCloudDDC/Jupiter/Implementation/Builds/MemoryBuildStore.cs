// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using EpicGames.Serialization;

namespace Jupiter.Implementation;

public class MemoryBuildStore : IBuildStore
{
	private readonly ConcurrentDictionary<string, MemoryBuildRecord> _builds = new();

	public Task PutBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObject buildObject, uint ttl)
	{
		string key = BuildKey(ns, bucket, buildId);

		_builds.AddOrUpdate(key, _ =>
		{
			MemoryBuildRecord o = new() { Namespace = ns, Bucket = bucket, BuildId = buildId, BuildObject = buildObject, ExpireTime = DateTime.Now.AddSeconds(ttl) };
			return o;
		}, (s, o) =>
		{
			o.BuildObject = buildObject;
			o.IsFinalized = false;
			o.ExpireTime = DateTime.Now.AddSeconds(ttl);
			return o;
		});

		return Task.CompletedTask;
	}

	public Task FinalizeBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, uint ttl)
	{
		string key = BuildKey(ns, bucket, buildId);

		if (_builds.TryGetValue(key, out MemoryBuildRecord? value))
		{
			value.IsFinalized = true;
		}

		return Task.CompletedTask;
	}

	public Task<BuildRecord?> GetBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		return Task.FromResult(_builds.TryGetValue(BuildKey(ns, bucket, buildId), out MemoryBuildRecord? value) ? value.ToBuildRecord() : null);
	}

	public Task DeleteBuild(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		_builds.TryRemove(BuildKey(ns, bucket, buildId), out MemoryBuildRecord? _);

		return Task.CompletedTask;
	}

	public async IAsyncEnumerable<BuildMetadata> ListBuildsAsync(NamespaceId ns, BucketId bucket, bool includeTTL = false)
	{
		await Task.CompletedTask;
		foreach (MemoryBuildRecord build in _builds.Values.OrderByDescending(record => record.CreationDate))
		{
			if (build.Namespace != ns || build.Bucket != bucket)
			{
				continue;
			}

			yield return new BuildMetadata(build.BuildId, build.Bucket, build.BuildObject, build.IsFinalized, includeTTL ? (uint)(build.ExpireTime - DateTime.Now).TotalSeconds : null);
		}
	}

	public Task PutBuildPartAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObjectId partId, string partName, uint ttl)
	{
		string key = BuildKey(ns, bucket, buildId);

		if (!_builds.TryGetValue(key, out MemoryBuildRecord? build))
		{
			return Task.CompletedTask;
		}

		build.BuildParts[partName] = new MemoryBuildPart { PartId = partId, PartName = partName };
		return Task.CompletedTask;

	}

	public async IAsyncEnumerable<(string, CbObjectId)> GetBuildPartsAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		string key = BuildKey(ns, bucket, buildId);

		if (_builds.TryGetValue(key, out MemoryBuildRecord? value))
		{
			foreach (KeyValuePair<string, MemoryBuildPart> buildPart in value.BuildParts)
			{
				yield return (buildPart.Value.PartName, buildPart.Value.PartId);
			}
		}

		await Task.CompletedTask;
	}

	public Task PutBuildAppendixAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string appendixName, uint ttl)
	{
		string key = BuildKey(ns, bucket, buildId);

		if (!_builds.TryGetValue(key, out MemoryBuildRecord? build))
		{
			return Task.CompletedTask;
		}

		build.BuildAppendix[appendixName] = appendixName;
		return Task.CompletedTask;
	}

	public async IAsyncEnumerable<string> GetBuildAppendixesAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		string key = BuildKey(ns, bucket, buildId);

		if (_builds.TryGetValue(key, out MemoryBuildRecord? value))
		{
			foreach (string appendixName in value.BuildAppendix.Keys)
			{
				yield return appendixName;
			}
		}

		await Task.CompletedTask;
	}

	public Task DeleteBuildAppendixAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string appendixFileName)
	{
		string key = BuildKey(ns, bucket, buildId);

		if (_builds.TryGetValue(key, out MemoryBuildRecord? value))
		{
			value.BuildAppendix.Remove(appendixFileName, out _);
		}

		return Task.CompletedTask;
	}

	public Task UpdateTTL(NamespaceId ns, BucketId bucket, CbObjectId buildId, uint ttl)
	{
		string key = BuildKey(ns, bucket, buildId);

		if (_builds.TryGetValue(key, out MemoryBuildRecord? value))
		{
			value.ExpireTime = DateTime.Now.AddSeconds(ttl);
		}

		return Task.CompletedTask;
	}

	public Task<uint?> GetTTL(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		string key = BuildKey(ns, bucket, buildId);

		return Task.FromResult(_builds.TryGetValue(key, out MemoryBuildRecord? value) ? (uint?)(value.ExpireTime - DateTime.Now).TotalSeconds : null);
	}

	public async IAsyncEnumerable<(NamespaceId, BucketId, CbObjectId, bool)> ListAllBuildsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
	{
		await Task.CompletedTask;
		foreach (MemoryBuildRecord build in _builds.Values.OrderByDescending(record => record.CreationDate))
		{
			yield return (build.Namespace, build.Bucket, build.BuildId, build.IsFinalized);
		}
	}

	private static string BuildKey(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		return $"{ns}.{bucket}.{buildId}";
	}
}

internal class MemoryBuildPart
{
	public CbObjectId PartId { get; init; }
	public string PartName { get; init; } = null!;
}

internal class MemoryBuildRecord
{
	public bool IsFinalized { get; set; }
	public DateTime ExpireTime { get; set; }
	public CbObject BuildObject { get; set; } = CbObject.Empty;
	public ConcurrentDictionary<string, MemoryBuildPart> BuildParts { get; set; } = new();
	public ConcurrentDictionary<string, string> BuildAppendix { get; set; } = new();
	public NamespaceId Namespace { get; set; }
	public BucketId Bucket { get; set; }
	public CbObjectId BuildId { get; set; }
	public DateTime CreationDate { get; set; } = DateTime.Now;

	public BuildRecord ToBuildRecord()
	{
		return new BuildRecord { BuildObject = BuildObject, IsFinalized = IsFinalized };
	}
}