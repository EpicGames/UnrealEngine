// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using Cassandra;
using Cassandra.Mapping;
using Jupiter.Common;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation;

public class ScyllaBlockStore : IBlockStore
{
	private readonly Tracer _tracer;
	private readonly ISession _session;
	private readonly Mapper _mapper;

	public ScyllaBlockStore(IScyllaSessionManager scyllaSessionManager, IOptionsMonitor<ScyllaSettings> scyllaSettings, Tracer tracer)
	{
		_tracer = tracer;
		_session = scyllaSessionManager.GetSessionForReplicatedKeyspace();
		_mapper = new Mapper(_session);

		if (!scyllaSettings.CurrentValue.AvoidSchemaChanges)
		{
			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS block_index (
				namespace text,
				block_id blob,
				metadata_blob_id blob,
				PRIMARY KEY ((namespace, block_id))
			);"
			));

			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS block_context (
				namespace text,
				block_context text,
				last_update timestamp,
				block_id blob,
				PRIMARY KEY ((namespace, block_context), block_id)
			);
				"
			));

			_session.Execute(new SimpleStatement(@$"CREATE INDEX IF NOT EXISTS block_context_by_time ON block_context ((namespace, block_context), last_update);"));
			_session.Execute(new SimpleStatement(@$"CREATE INDEX IF NOT EXISTS block_context_by_blocks ON block_context (block_id);"));
		}
	}

	public async IAsyncEnumerable<BlockMetadata> ListBlockIndexAsync(NamespaceId ns, BlockContext blockContext)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.list_blocks");

		foreach (ScyllaBlockContextEntry blockEntry in await _mapper.FetchAsync<ScyllaBlockContextEntry>(
			         "SELECT * FROM block_context_by_time_index WHERE namespace = ? AND block_context = ? ORDER BY last_update DESC", 
			         ns.ToString(), blockContext.ToString()))
		{
			yield return new BlockMetadata { LastUpdate = blockEntry.LastUpdate, MetadataBlobId = new BlobId(blockEntry.MetadataBlobId) };
		}
	}

	public async Task AddBlockToContextAsync(NamespaceId ns, BlockContext blockContext, BlobId metadataBlockId)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.update_block_index");

		try
		{
			await _mapper.InsertAsync<ScyllaBlockContextEntry>(new ScyllaBlockContextEntry(ns, blockContext, DateTime.Now, metadataBlockId));
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async Task PutBlockMetadataAsync(NamespaceId ns, BlobId blockIdentifier, BlobId metadataObjectId)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.put_block_metadata");

		try
		{
			await _mapper.InsertAsync<ScyllaBlockIndexEntry>(new ScyllaBlockIndexEntry(ns, blockIdentifier, metadataObjectId));
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async Task<BlobId?> GetBlockMetadataAsync(NamespaceId ns, BlobId blockIdentifier)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_block_metadata");

		try
		{
			ScyllaBlockIndexEntry? scyllaBlockIndexEntry = await _mapper.SingleOrDefaultAsync<ScyllaBlockIndexEntry>("WHERE namespace = ? AND block_id = ?", ns.ToString(), blockIdentifier.HashData);

			if (scyllaBlockIndexEntry != null)
			{
				scope.SetAttribute("found", bool.TrueString);
				return new BlobId(scyllaBlockIndexEntry.MetadataBlobId);
			}
			scope.SetAttribute("found", bool.FalseString);
			return null;
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async Task DeleteBlockAsync(NamespaceId ns, BlobId blockIdentifier)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.delete_block_metadata");
		
		try
		{
			List<string> foundContexts = new ();
			foreach (ScyllaBlockContextEntry blockEntry in await _mapper.FetchAsync<ScyllaBlockContextEntry>("SELECT * FROM block_context WHERE block_id = ? ", blockIdentifier.HashData))
			{
				if (string.Equals(blockEntry.Namespace, ns.ToString(), StringComparison.OrdinalIgnoreCase))
				{
					foundContexts.Add(blockEntry.BlockContext);
				}
			}

			foreach (string blockContext in foundContexts)
			{
				await _mapper.DeleteAsync<ScyllaBlockContextEntry>("WHERE namespace = ? AND block_context = ? AND block_id = ?", ns.ToString(), blockContext, blockIdentifier.HashData);
			}

			scope.SetAttribute("UsedInCountOfContexts", foundContexts.Count);
			await _mapper.DeleteAsync<ScyllaBlockIndexEntry>("WHERE namespace = ? AND block_id = ?", ns.ToString(), blockIdentifier.HashData);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}
}

[Cassandra.Mapping.Attributes.Table("block_index")]
public class ScyllaBlockIndexEntry
{
	public ScyllaBlockIndexEntry()
	{
		Namespace = null!;
		BlockId = null!;
		MetadataBlobId = null!;
	}
	
	public ScyllaBlockIndexEntry(NamespaceId @namespace, BlobId blockId, BlobId metadataBlobId)
	{
		Namespace = @namespace.ToString();
		BlockId = blockId.HashData;
		MetadataBlobId = metadataBlobId.HashData;
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.Column("block_id")]
	[Cassandra.Mapping.Attributes.PartitionKey]
	public byte[] BlockId { get; set; }

	[Cassandra.Mapping.Attributes.Column("metadata_blob_id")]
	public byte[] MetadataBlobId { get; set; }
}

[Cassandra.Mapping.Attributes.Table("block_context")]
public class ScyllaBlockContextEntry
{
	public ScyllaBlockContextEntry()
	{
		Namespace = null!;
		BlockContext = null!;
		LastUpdate = DateTime.Now;
		MetadataBlobId = null!;
	}

	public ScyllaBlockContextEntry(NamespaceId @namespace, BlockContext context, DateTime lastUpdate, BlobId blockId)
	{
		Namespace = @namespace.ToString();
		BlockContext = context.ToString();
		LastUpdate = lastUpdate;
		MetadataBlobId = blockId.HashData;
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.PartitionKey]
	[Cassandra.Mapping.Attributes.Column("block_context")]
	public string BlockContext { get; set; }

	[Cassandra.Mapping.Attributes.ClusteringKey]
	[Cassandra.Mapping.Attributes.Column("last_update")]
	public DateTime LastUpdate { get; set; }

	[Cassandra.Mapping.Attributes.Column("block_id")]
	public byte[] MetadataBlobId { get; set; }
}
