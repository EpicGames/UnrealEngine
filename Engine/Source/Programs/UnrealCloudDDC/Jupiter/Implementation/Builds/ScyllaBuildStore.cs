// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Cassandra;
using Cassandra.Mapping;
using EpicGames.Serialization;
using Jupiter.Common;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation;

public class ScyllaBuildStore : IBuildStore
{
	private readonly IScyllaSessionManager _scyllaSessionManager;
	private readonly Tracer _tracer;
	private readonly ISession _session;
	private readonly Mapper _mapper;
	private readonly PreparedStatement _getTtlStatement;

	public ScyllaBuildStore(IScyllaSessionManager scyllaSessionManager, IOptionsMonitor<ScyllaSettings> scyllaSettings, Tracer tracer)
	{
		_scyllaSessionManager = scyllaSessionManager;
		_tracer = tracer;
		_session = scyllaSessionManager.GetSessionForReplicatedKeyspace();
		_mapper = new Mapper(_session);

		string tableOptions = scyllaSessionManager.IsAWSKeyspace ? "WITH CUSTOM_PROPERTIES={'ttl':{'status':'enabled'}}" : "";
		string appendTableOptions = scyllaSessionManager.IsAWSKeyspace ? "AND CUSTOM_PROPERTIES={'ttl':{'status':'enabled'}}" : "";
		if (!scyllaSettings.CurrentValue.AvoidSchemaChanges)
		{
			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS builds (
					namespace text,
					bucket_id text,
					build_id blob,
					is_finalized boolean,
					build_object blob,
					PRIMARY KEY ((namespace, bucket_id), build_id)
				) WITH CLUSTERING ORDER BY (build_id DESC) {appendTableOptions}
				;"
			));

			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS build_parts (
					namespace text,
					bucket_id text,
					build_id blob,
					part_name text,
					part_id blob,
					PRIMARY KEY ((namespace, bucket_id, build_id), part_id)
				) {tableOptions};"
			));

			_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS build_appendixes (
					namespace text,
					bucket_id text,
					build_id blob,
					appendix_name text,
					PRIMARY KEY ((namespace, bucket_id, build_id), appendix_name)
				) {tableOptions};"
			));
		}
		_getTtlStatement = _session.Prepare("SELECT TTL(build_object) as ttl FROM builds WHERE namespace = ? AND bucket_id = ? AND build_id = ?");
	}

	public async IAsyncEnumerable<BuildMetadata> ListBuildsAsync(NamespaceId ns, BucketId bucket, bool includeTTL = false)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.list_builds");
		
		IEnumerable<ScyllaBuildEntry> entries;
		try
		{
			entries = await _mapper.FetchAsync<ScyllaBuildEntry>("WHERE namespace = ? AND bucket_id = ?", ns.ToString(), bucket.ToString());
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
		
		foreach (ScyllaBuildEntry buildEntry in entries)
		{
			BuildMetadata? metadata = buildEntry.ToBuildMetadata();
			// ignore invalid metadata
			if (metadata == null)
			{
				continue;
			}

			if (includeTTL)
			{
				uint? ttl = await GetTTL(ns, bucket, metadata.BuildId);
				metadata.Ttl = ttl;
			}

			yield return metadata;
		}
	}

	public async Task PutBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObject buildObject, uint ttl)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.put_build");

		try
		{
			await _mapper.InsertAsync<ScyllaBuildEntry>(new ScyllaBuildEntry(ns, bucket, buildId, buildObject), insertNulls: false, ttl: (int)ttl);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async Task FinalizeBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, uint ttl)
	{
		try
		{
			await _mapper.UpdateAsync<ScyllaBuildEntry>("USING TTL ? SET is_finalized = true WHERE namespace = ? and bucket_id = ? AND build_id = ?", (int)ttl, ns.ToString(), bucket.ToString(), buildId.ToByteArray());
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async Task<BuildRecord?> GetBuildAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		try
		{
			ScyllaBuildEntry? buildEntry = await _mapper.SingleOrDefaultAsync<ScyllaBuildEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ?", ns.ToString(), bucket.ToString(), buildId.ToByteArray());

			if (buildEntry != null && buildEntry.BuildObject != null) 
			{
				return new BuildRecord {BuildObject = new CbObject(buildEntry.BuildObject), IsFinalized = buildEntry.IsFinalized};
			}
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}

		return null;
	}

	public async Task PutBuildPartAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, CbObjectId partId, string partName, uint ttl)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.put_build_part");

		try
		{
			await _mapper.InsertAsync<ScyllaBuildPartEntry>(new ScyllaBuildPartEntry(ns, bucket, buildId, partId, partName), insertNulls: false, ttl: (int)ttl);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async IAsyncEnumerable<(string, CbObjectId)> GetBuildPartsAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_build_parts");
		
		IEnumerable<ScyllaBuildPartEntry> entries;
		try
		{
			entries = await _mapper.FetchAsync<ScyllaBuildPartEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ?", ns.ToString(),
				bucket.ToString(), buildId.ToByteArray());
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
		
		foreach (ScyllaBuildPartEntry buildPartEntry in entries)
		{
			yield return (buildPartEntry.PartName, new CbObjectId(buildPartEntry.PartId));
		}
	}

	public async Task PutBuildAppendixAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string appendixName, uint ttl)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.put_build_appendix");

		try
		{
			await _mapper.InsertAsync<ScyllaBuildAppendixEntry>(new ScyllaBuildAppendixEntry(ns, bucket, buildId, appendixName), insertNulls: false, ttl: (int)ttl);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async IAsyncEnumerable<string> GetBuildAppendixesAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_build_appendixes");
		
		IEnumerable<ScyllaBuildAppendixEntry> entries;
		try
		{
			entries = await _mapper.FetchAsync<ScyllaBuildAppendixEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ?", ns.ToString(),
				bucket.ToString(), buildId.ToByteArray());
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
		
		foreach (ScyllaBuildAppendixEntry entry in entries)
		{
			yield return entry.AppendixName;
		}
	}

	public async Task DeleteBuildAppendixAsync(NamespaceId ns, BucketId bucket, CbObjectId buildId, string appendixName)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.delete_build_appendix");

		try
		{
			await _mapper.DeleteAsync<ScyllaBuildAppendixEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ? AND appendix_name = ?", ns.ToString(), bucket.ToString(), buildId.ToByteArray(), appendixName);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}

	public async Task<uint?> GetTTL(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_build_ttl");

		try
		{
			BoundStatement? boundStatement = _getTtlStatement.Bind(ns.ToString(), bucket.ToString(), buildId.ToByteArray());
			RowSet rowSet = await _session.ExecuteAsync(boundStatement);
			foreach (Row row in rowSet)
			{
				// we only expect one row here so we just return once we have a value

				if (row["ttl"] is int ttl)
				{
					return (uint)ttl;
				}
			}
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}

		return null;
	}

	public async IAsyncEnumerable<(NamespaceId, BucketId, CbObjectId, bool)> ListAllBuildsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
	{
		using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_all_builds");

		string cqlOptions = _scyllaSessionManager.IsScylla ? "BYPASS CACHE" : "";

		IEnumerable<ScyllaBuildEntry> entries;
		try
		{
			entries = await _mapper.FetchAsync<ScyllaBuildEntry>(cqlOptions);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
		
		foreach (ScyllaBuildEntry buildEntry in entries)
		{
			yield return (new NamespaceId(buildEntry.Namespace), new BucketId(buildEntry.BucketId), new CbObjectId(buildEntry.BuildId), buildEntry.IsFinalized);
		}
	}

	public async Task UpdateTTL(NamespaceId ns, BucketId bucket, CbObjectId buildId, uint ttl)
	{
		try
		{
			ScyllaBuildEntry? buildEntry = await _mapper.SingleOrDefaultAsync<ScyllaBuildEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ?", ns.ToString(), bucket.ToString(), buildId.ToByteArray());

			if (buildEntry == null)
			{
				// failed to find the full build, very unlikely, should have failed earlier
				return;
			}

			// scylla does not support updating the ttl, instead we reinsert the same value with a new ttl
			await _mapper.InsertAsync<ScyllaBuildEntry>(buildEntry, false, (int)ttl);
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}

		IEnumerable<ScyllaBuildPartEntry> entries;
		try
		{
			entries = await _mapper.FetchAsync<ScyllaBuildPartEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ?", ns.ToString(),
				bucket.ToString(), buildId.ToByteArray());
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}

		foreach (ScyllaBuildPartEntry buildPartEntry in entries)
		{
			await _mapper.InsertAsync<ScyllaBuildPartEntry>(buildPartEntry, false, (int)ttl);
		}

		
		IEnumerable<ScyllaBuildAppendixEntry> appendixes;
		try
		{
			appendixes = await _mapper.FetchAsync<ScyllaBuildAppendixEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ?", ns.ToString(), bucket.ToString(), buildId.ToByteArray());
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}

		foreach (ScyllaBuildAppendixEntry appendixEntry in appendixes)
		{
			await _mapper.InsertAsync<ScyllaBuildAppendixEntry>(appendixEntry, false, (int)ttl);
		}
	}

	public async Task DeleteBuild(NamespaceId ns, BucketId bucket, CbObjectId buildId)
	{
		try
		{
			await _mapper.DeleteAsync<ScyllaBuildAppendixEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ?", ns.ToString(), bucket.ToString(), buildId.ToByteArray());
			await _mapper.DeleteAsync<ScyllaBuildPartEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ?", ns.ToString(), bucket.ToString(), buildId.ToByteArray());
			await _mapper.DeleteAsync<ScyllaBuildEntry>("WHERE namespace = ? AND bucket_id = ? AND build_id = ?", ns.ToString(), bucket.ToString(), buildId.ToByteArray());
		}
		catch (QueryExecutionException e)
		{
			throw new RemoteServiceException("Error communicating with Scylla", e);
		}
	}
}

[Cassandra.Mapping.Attributes.Table("builds")]
public class ScyllaBuildEntry
{
	public ScyllaBuildEntry()
	{
		Namespace = null!;
		BucketId = null!;
		BuildId = null!;
		BuildObject = null!;
		IsFinalized = false;
	}
	
	public ScyllaBuildEntry(NamespaceId @namespace, BucketId bucketId, CbObjectId buildId, CbObject buildObject)
	{
		Namespace = @namespace.ToString();
		BucketId = bucketId.ToString();
		BuildId = buildId.ToByteArray();
		BuildObject = buildObject.GetView().ToArray();
		IsFinalized = false;
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.Column("bucket_id")]
	[Cassandra.Mapping.Attributes.PartitionKey]
	public string BucketId { get; set; }

	[Cassandra.Mapping.Attributes.Column("build_id")]
	[Cassandra.Mapping.Attributes.ClusteringKey]
	public byte[] BuildId { get; set; }

	[Cassandra.Mapping.Attributes.Column("build_object")]
	public byte[]? BuildObject { get; set; }

	[Cassandra.Mapping.Attributes.Column("is_finalized")]
	public bool IsFinalized { get; set; }
	public BuildMetadata? ToBuildMetadata()
	{
		try
		{
			return new BuildMetadata(new CbObjectId(BuildId), new BucketId(BucketId), new CbObject(BuildObject), IsFinalized, null);
		}
		catch (IndexOutOfRangeException)
		{
			// incorrectly structured compact binaries are ignored
			return null;
		}
	}
}

[Cassandra.Mapping.Attributes.Table("build_parts")]
public class ScyllaBuildPartEntry
{
	public ScyllaBuildPartEntry()
	{
		Namespace = null!;
		BucketId = null!;
		BuildId = null!;
		PartName = null!;
		PartId = null!;
	}
	
	public ScyllaBuildPartEntry(NamespaceId @namespace, BucketId bucketId, CbObjectId buildId, CbObjectId partId, string partName)
	{
		Namespace = @namespace.ToString();
		BucketId = bucketId.ToString();
		BuildId = buildId.ToByteArray();
		PartName = partName;
		PartId = partId.ToByteArray();
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.Column("bucket_id")]
	[Cassandra.Mapping.Attributes.PartitionKey]
	public string BucketId { get; set; }

	[Cassandra.Mapping.Attributes.Column("build_id")]
	[Cassandra.Mapping.Attributes.PartitionKey]
	public byte[] BuildId { get; set; }

	[Cassandra.Mapping.Attributes.Column("part_name")]
	public string PartName { get; set; }

	[Cassandra.Mapping.Attributes.Column("part_id")]
	[Cassandra.Mapping.Attributes.ClusteringKey]
	public byte[] PartId { get; set; }
}

[Cassandra.Mapping.Attributes.Table("build_appendixes")]
public class ScyllaBuildAppendixEntry
{
	public ScyllaBuildAppendixEntry()
	{
		Namespace = null!;
		BucketId = null!;
		BuildId = null!;
		AppendixName = null!;
	}
	
	public ScyllaBuildAppendixEntry(NamespaceId @namespace, BucketId bucketId, CbObjectId buildId, string appendixName)
	{
		Namespace = @namespace.ToString();
		BucketId = bucketId.ToString();
		BuildId = buildId.ToByteArray();
		AppendixName = appendixName;
	}

	[Cassandra.Mapping.Attributes.PartitionKey]
	public string Namespace { get; set; }

	[Cassandra.Mapping.Attributes.Column("bucket_id")]
	[Cassandra.Mapping.Attributes.PartitionKey]
	public string BucketId { get; set; }

	[Cassandra.Mapping.Attributes.Column("build_id")]
	[Cassandra.Mapping.Attributes.PartitionKey]
	public byte[] BuildId { get; set; }

	[Cassandra.Mapping.Attributes.Column("appendix_name")]
	[Cassandra.Mapping.Attributes.ClusteringKey]
	public string AppendixName { get; set; }
}