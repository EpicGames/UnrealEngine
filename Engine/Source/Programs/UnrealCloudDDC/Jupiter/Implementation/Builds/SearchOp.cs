// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Serialization;
using System.Collections.Generic;
using System;
using System.Collections;
using System.Globalization;
using System.Linq;
using System.Text.Json;
using Microsoft.Extensions.Logging;

namespace Jupiter.Implementation.Builds
{
	interface ISearchOp
	{
		bool Matches(CbObjectId buildId, CbObject o, ILogger logger);
	}

	class CompareOp : ISearchOp
	{
		private readonly string _fieldName;
		private readonly OpType _opType;
		private readonly JsonElement? _expectedFieldJson;
		private readonly CbField? _expectedField;

		private CompareOp(string fieldName, OpType opType, JsonElement expectedFieldJson)
		{
			_fieldName = fieldName;
			_opType = opType;
			_expectedFieldJson = expectedFieldJson;
			_expectedField = null!;
		}

		private CompareOp(string fieldName, OpType opType, CbField expectedField)
		{
			_fieldName = fieldName;
			_opType = opType;
			_expectedField = expectedField;
			_expectedFieldJson = null!;
		}

		enum OpType
		{
			Equals,
			GreaterThen,
			GreaterThenOrEquals,
			In,
			LessThen,
			LessThenOrEquals,
			NotEquals,
			NotIn
		};

		private static bool MatchesJson(CbField field, JsonElement expectedField, OpType op, ILogger logger)
		{
			bool isComparable = field.IsFloat() || field.IsInteger() || field.IsDateTime() || field.IsTimeSpan() || field.IsString();
			bool isCompareOp = op is OpType.GreaterThen or OpType.GreaterThenOrEquals or OpType.LessThen or OpType.LessThenOrEquals;

			bool isArray = expectedField.ValueKind == JsonValueKind.Array;
			bool isArrayOp = op is OpType.In or OpType.NotIn;

			bool isEquatableOp = op is OpType.Equals or OpType.NotEquals;

			if (isComparable && isCompareOp)
			{
				string rawText = expectedField.ToString();
				object expectedType;
				object actualType;
				
				if (field.IsInteger())
				{
					actualType = field.AsInt64();
					if (long.TryParse(rawText, out long longType))
					{
						expectedType = longType;
					}
					else
					{
						throw new InvalidCastException($"Unable to convert value {rawText} to integer when comparing field {field.Name} in op {op}.");
					}
				}
				else if (field.IsFloat())
				{
					actualType = field.AsDouble();
					if (double.TryParse(rawText, out double doubleType))
					{
						expectedType = doubleType;
					}
					else
					{
						throw new InvalidCastException($"Unable to convert value {rawText} to float when comparing field {field.Name} in op {op}.");
					}
				}
				else if (field.IsDateTime())
				{
					actualType = field.AsDateTime();
					if (DateTime.TryParse(rawText, out DateTime datetimeType))
					{
						expectedType = datetimeType;
					}
					else
					{
						throw new InvalidCastException($"Unable to convert value {rawText} to datetime when comparing field {field.Name} in op {op}.");
					}
				}
				else if (field.IsTimeSpan())
				{
					actualType = field.AsTimeSpan();
					if (TimeSpan.TryParse(rawText, out TimeSpan timespanType))
					{
						expectedType = timespanType;
					}
					else
					{
						throw new InvalidCastException($"Unable to convert value {rawText} to timespan when comparing field {field.Name} in op {op}.");
					}
				}
				else if (field.IsString())
				{
					string s = field.AsString();
					// we lack enough information to know for certain what type this is, as such we just test different types that are increasingly more likely to pass
					if (TimeSpan.TryParse(rawText, out TimeSpan timespanType))
					{
						expectedType = timespanType;
						actualType = TimeSpan.Parse(s);
					} 
					else if (DateTime.TryParse(rawText, out DateTime dtType))
					{
						expectedType = dtType;
						actualType = DateTime.Parse(s);
					}
					else if (long.TryParse(rawText, out long longType))
					{
						expectedType = longType;
						actualType = long.Parse(s);
					}
					else if (double.TryParse(rawText, out double doubleType))
					{
						expectedType = doubleType;
						actualType = double.Parse(s);
					}
					else
					{
						throw new NotImplementedException($"Unable to guess compare type of string value \'{s}\', if you want to use a comparable op then make sure the original object uses a comparable type");
					}
				}
				else
				{
					throw new NotImplementedException($"Type ({field.TypeWithFlags.ToString()}) of field {field.Name} not convertible to a comparable type");
				}

				int compare = Comparer.Default.Compare(expectedType, actualType);
				logger.LogDebug("Running compare between values {Value0} string and cb field {Value1} result was {Result}", rawText, actualType, compare);
				switch (op)
				{
					case OpType.GreaterThen:
						return compare < 0;
					case OpType.GreaterThenOrEquals:
						return compare <= 0;
					case OpType.LessThen:
						return compare > 0;
					case OpType.LessThenOrEquals:
						return compare >= 0;
					default:
						throw new ArgumentOutOfRangeException(nameof(op));
				}
			}
			else if (isArray && isArrayOp)
			{
				// looks for expected value in the array
				JsonElement[] array = expectedField.EnumerateArray().ToArray();
				bool isInArray = array.Any(element => FieldEquals(field, element, logger));
				
				if (op == OpType.In)
				{
					return isInArray;
				}
				else
				{
					// not in array
					return !isInArray;
				}
			}
			else if (isEquatableOp)
			{
				bool isEqual = FieldEquals(field, expectedField, logger);
				
				logger.LogDebug("Ran equality check between values from json: {Value0} and cb of type: {Value1} result was {Result}", expectedField, field.Value?.ToString(), isEqual);
				if (op == OpType.Equals)
				{
					return isEqual;
				}
				else
				{
					// !not equals
					return !isEqual;
				}
			}
			throw new UnsupportedOperationException($"Field {field.Name} not convertible to a type that supports operation: {op} against type {expectedField.ValueKind}");
		}

		private static bool CbFieldEquals(CbField cbField, CbField expectedField)
		{
			if (cbField.IsString() && expectedField.IsString())
			{
				// special case handle string comparisons so that they are not case-sensitive
				string a = cbField.AsString();
				string b = expectedField.AsString();
				return string.Equals(a, b, StringComparison.OrdinalIgnoreCase);
			}

			// check for string and number comparisons to allow mixing string and number representation of fields
			bool? CheckStringAndNumberEquality(CbField p0, CbField p1)
			{
				if (p0.IsString())
				{
					string s = p0.AsString();
					if (p1.IsInteger())
					{
						long a = p1.AsInt64();
						if (long.TryParse(s, CultureInfo.InvariantCulture, out long b))
						{
							return a.Equals(b);
						}
					}
					if (p1.IsFloat())
					{
						double a = p1.AsDouble();
						if (double.TryParse(s, CultureInfo.InvariantCulture, out double b))
						{
							return a.Equals(b);
						}
					}
				}

				return null;
			}

			bool? equality = CheckStringAndNumberEquality(cbField, expectedField);
			if (equality.HasValue)
			{
				return equality.Value;
			}
			equality = CheckStringAndNumberEquality(expectedField, cbField);
			if (equality.HasValue)
			{
				return equality.Value;
			}

			return cbField.ValueEquals(expectedField);
		}

		private static bool FieldEquals(CbField field, JsonElement jsonElement, ILogger logger)
		{
			string jsonText = jsonElement.ToString();
			if (field.IsInteger() && long.TryParse(jsonText, out long longValue))
			{
				logger.LogDebug("Field equality check type was {Type} value was {Value} and converted value was {ConvertedValue}", "integer", field.AsInt64(), longValue);
				return field.AsInt64().Equals(longValue);
			}
			if (field.IsString())
			{
				logger.LogDebug("Field equality check type was {Type} value was {Value} and converted value was {ConvertedValue}", "string", field.AsString(), jsonText);

				return field.AsString().Equals(jsonText, StringComparison.OrdinalIgnoreCase);
			} 
			if (field.IsDateTime() && DateTime.TryParse(jsonText, out DateTime dtValue))
			{
				logger.LogDebug("Field equality check type was {Type} value was {Value} and converted value was {ConvertedValue}", "datetime", field.AsDateTime(), dtValue);

				return field.AsDateTime().Equals(dtValue);
			}
			if (field.IsTimeSpan() && TimeSpan.TryParse(jsonText, out TimeSpan tsValue))
			{
				logger.LogDebug("Field equality check type was {Type} value was {Value} and converted value was {ConvertedValue}", "timespan", field.AsTimeSpan(), tsValue);

				return field.AsTimeSpan().Equals(tsValue);
			}
			if (field.IsHash() && IoHash.TryParse(jsonText, out IoHash hashValue))
			{
				logger.LogDebug("Field equality check type was {Type} value was {Value} and converted value was {ConvertedValue}", "iohash", field.AsHash(), hashValue);

				return field.AsHash().Equals(hashValue);
			}
			if (field.IsUuid() && Guid.TryParse(jsonText, out Guid uuidValue))
			{
				logger.LogDebug("Field equality check type was {Type} value was {Value} and converted value was {ConvertedValue}", "uuid", field.AsUuid(), uuidValue);

				return field.AsUuid().Equals(uuidValue);
			}
			if (field.IsObjectId() && CbObjectId.TryParse(jsonText, out CbObjectId objectId))
			{
				logger.LogDebug("Field equality check type was {Type} value was {Value} and converted value was {ConvertedValue}", "objectId", field.AsObjectId(), objectId);

				return field.AsObjectId().Equals(objectId);
			}
			if (field.IsFloat() && double.TryParse(jsonText, out double doubleValue))
			{
				logger.LogDebug("Field equality check type was {Type} value was {Value} and converted value was {ConvertedValue}", "float", field.AsDouble(), doubleValue);

				return field.AsDouble().Equals(doubleValue);
			}
			throw new NotImplementedException($"Unable to convert json type {jsonElement.ValueKind} to type that can be compared to field {field.Name} of type {field.TypeWithFlags}");
		}

		private static bool MatchesCB(CbField field, CbField expectedField, OpType op, ILogger logger)
		{
			bool isComparable = field.IsFloat() || field.IsInteger() || field.IsDateTime() || field.IsTimeSpan() || field.IsString();
			bool isCompareOp = op is OpType.GreaterThen or OpType.GreaterThenOrEquals or OpType.LessThen or OpType.LessThenOrEquals;

			bool isArray = expectedField.IsArray();
			bool isArrayOp = op is OpType.In or OpType.NotIn;

			bool isEquatableOp = op is OpType.Equals or OpType.NotEquals;

			if (isComparable && isCompareOp)
			{
				// first check the type in the object we are searching for to see if compare operations even make sense
				object expectedType;
				if (expectedField.IsInteger())
				{
					expectedType = expectedField.AsInt64();
				}
				else if (expectedField.IsFloat())
				{
					expectedType = expectedField.AsDouble();
				}
				else if (expectedField.IsDateTime())
				{
					expectedType = expectedField.AsDateTime();
				}
				else if (expectedField.IsTimeSpan())
				{
					expectedType = expectedField.AsTimeSpan();
				}
				else
				{
					throw new NotImplementedException($"Type ({expectedField.TypeWithFlags.ToString()}) of field {expectedField.Name} not convertible to a comparable type. This field does not support comparision operators.");
				}

				object actualType;
				if (field.IsInteger())
				{
					actualType = field.AsInt64();
				}
				else if (field.IsFloat())
				{
					if (expectedField.IsInteger())
					{
						// if the type we are comparing against is an integer we convert to a int instead as compact binary assumes implicit double to int conversion
						actualType = (long)field.AsDouble();
					}
					else
					{
						actualType = field.AsDouble();
					}
				}
				else if (field.IsDateTime())
				{
					actualType = field.AsDateTime();
				}
				else if (field.IsTimeSpan())
				{
					actualType = field.AsTimeSpan();
				}
				else if (field.IsString())
				{
					// attempt to parse our string value into the type we are comparing against
					string s = field.AsString();
					if (expectedField.IsInteger())
					{
						if (long.TryParse(s, out long longResult))
						{
							actualType = longResult;
						}
						else
						{
							throw new Exception($"Failed to parse \'{s}\' into a integer for comparison");
						}
					}
					else if (expectedField.IsFloat())
					{
						if (double.TryParse(s, out double doubleResult))
						{
							actualType = doubleResult;
						}
						else
						{
							throw new Exception($"Failed to parse \'{s}\' into a double for comparison");
						}
					}
					else if (expectedField.IsDateTime())
					{
						if (DateTime.TryParse(s, out DateTime dtResult))
						{
							actualType = dtResult;
						}
						else
						{
							throw new Exception($"Failed to parse \'{s}\' into a datetime for comparison");
						}
					}
					else if (expectedField.IsTimeSpan())
					{
						if (TimeSpan.TryParse(s, out TimeSpan tsResult))
						{
							actualType = tsResult;
						}
						else
						{
							throw new Exception($"Failed to parse \'{s}\' into a timespan for comparison");
						}
					}
					else
					{
						throw new NotImplementedException($"Failed to convert string value of field {field.Name} to type \'{expectedField}\'");
					}
				}
				else
				{
					throw new NotImplementedException($"Type ({field.TypeWithFlags.ToString()}) of field {field.Name} not convertible to a comparable type");
				}

				int compare = Comparer.Default.Compare(expectedType, actualType);
				logger.LogDebug("Running compare between values {Expected} {Actual} result was {CompareResult}", expectedType, actualType, compare);

				switch (op)
				{
					case OpType.GreaterThen:
						return compare < 0;
					case OpType.GreaterThenOrEquals:
						return compare <= 0;
					case OpType.LessThen:
						return compare > 0;
					case OpType.LessThenOrEquals:
						return compare >= 0;
					default:
						throw new ArgumentOutOfRangeException(nameof(op));
				}
			}
			else if (isArray && isArrayOp)
			{
				// looks for expected value in the array
				CbArray array = expectedField.AsArray();
				bool isInArray = array.Any(cbField => CbFieldEquals(cbField, field));
				
				if (op == OpType.In)
				{
					return isInArray;
				}
				else
				{
					// not in array
					return !isInArray;
				}
			}
			else if (isEquatableOp)
			{
				bool isEqual = CbFieldEquals(field, expectedField);
				if (op == OpType.Equals)
				{
					return isEqual;
				}
				else
				{
					// !not equals
					return !isEqual;
				}
			}
			throw new UnsupportedOperationException($"Field {field.Name} not convertible to a type that supports operation: {op} against type {expectedField.Name}");
		}

		public bool Matches(CbObjectId buildId, CbObject o, ILogger logger)
		{
			CbField field = FindField(_fieldName, o);

			// special case handle the build id field which is not part of the cb object but should still be searchable
			if (string.Equals(_fieldName, "buildid", StringComparison.OrdinalIgnoreCase))
			{
				CbWriter writer = new CbWriter();
				writer.BeginObject();
				writer.WriteObjectId("buildId", buildId);
				writer.EndObject();

				CbObject generatedObject = writer.ToObject();
				field = generatedObject["buildId"];
			}

			if (field.Equals(CbField.Empty))
			{
				logger.LogDebug("Failed to find field with name {Name} in object. No Match", _fieldName);
				return false;
			}

			if (_expectedFieldJson != null)
			{
				return MatchesJson(field, _expectedFieldJson.Value, _opType, logger);
			}
			else if (_expectedField != null)
			{
				return MatchesCB(field, _expectedField, _opType, logger);
			}

			throw new NotImplementedException();
		}

		private static CbField FindField(string fieldName, CbObject cbObject)
		{
			// check to see if this is a embedded field
			int embeddedFieldSep = fieldName.IndexOf(".", StringComparison.InvariantCultureIgnoreCase);
			if (embeddedFieldSep != -1)
			{
				string key = fieldName.Substring(0, embeddedFieldSep);
				fieldName = fieldName.Substring(embeddedFieldSep + 1);
				CbField subField = cbObject.FindIgnoreCase(key);

				if (subField.Equals(CbField.Empty))
				{
					// field did not exist
					return CbField.Empty;
				}

				CbObject subObject = subField.AsObject();
				if (subObject.Equals(CbObject.Empty))
				{
					// field was not an object
					return CbField.Empty;
				}

				return FindField(fieldName, subObject);
			}

			// not an embedded field, find the field name
			CbField field = cbObject.FindIgnoreCase(fieldName);
			// if the field didn't exist it's an empty field which is what we expect to return anyway if it didn't exist
			return field;
		}

		public static IEnumerable<CompareOp> Parse(string fieldName, JsonElement jsonElement)
		{
			foreach (JsonProperty jsonProperty in jsonElement.EnumerateObject())
			{
				OpType? opType = FindOpType(jsonProperty.Name);
				if (opType.HasValue)
				{
					yield return new CompareOp(fieldName, opType.Value, jsonProperty.Value);
				}
			}
		}

		public static IEnumerable<CompareOp> Parse(string fieldName, CbObject cbObject)
		{
			foreach (CbField field in cbObject)
			{
				OpType? opType = FindOpType(field.Name.ToString());
				if (opType.HasValue)
				{
					yield return new CompareOp(fieldName, opType.Value, field);
				}
			}
		}

		private static OpType? FindOpType(string key)
		{
			switch (key)
			{
				case "$eq":
					return OpType.Equals;
				case "$neq":
					return OpType.NotEquals;
				case "$gt":
					return OpType.GreaterThen;
				case "$gte":
					return OpType.GreaterThenOrEquals;
				case "$in":
					return OpType.In;
				case "$lt":
					return OpType.LessThen;
				case "$lte":
					return OpType.LessThenOrEquals;
				case "$nin":
					return OpType.NotIn;
			}

			return null;
		}
	}

	abstract class LogicalOp : ISearchOp
	{
		protected LogicalOp(List<ISearchOp> childOps)
		{
			ChildOps = childOps;
		}

		protected readonly List<ISearchOp> ChildOps;

		public abstract bool Matches(CbObjectId buildId, CbObject o, ILogger logger);
	}

	class AndOp : LogicalOp
	{
		private AndOp(List<ISearchOp> childOps) : base(childOps)
		{
		}

		public override bool Matches(CbObjectId buildId, CbObject o, ILogger logger)
		{
			foreach (ISearchOp childOp in ChildOps)
			{
				if (!childOp.Matches(buildId, o, logger))
				{
					return false;
				}
			}

			return true;
		}

		public static AndOp Parse(CbObject o)
		{
			List<ISearchOp> ops = new();
			foreach (CbField field in o)
			{
				ops.AddRange(SearchOpHelpers.Parse(field));
			}
			return new AndOp(ops);
		}

		public static AndOp Parse(JsonElement element)
		{
			List<ISearchOp> ops = new();
			foreach (JsonProperty prop in element.EnumerateObject())
			{
				ops.AddRange(SearchOpHelpers.Parse(prop));
			}
			return new AndOp(ops);
		}
	}

	class OrOp : LogicalOp
	{
		private OrOp(List<ISearchOp> childOps) : base(childOps)
		{
		}

		public override bool Matches(CbObjectId buildId, CbObject o, ILogger logger)
		{
			foreach (ISearchOp childOp in ChildOps)
			{
				if (childOp.Matches(buildId, o, logger))
				{
					return true;
				}
			}

			return false;
		}

		public static OrOp Parse(CbObject o)
		{
			List<ISearchOp> ops = new();
			foreach (CbField field in o)
			{
				ops.AddRange(SearchOpHelpers.Parse(field));
			}
			return new OrOp(ops);
		}

		public static OrOp Parse(JsonElement element)
		{
			List<ISearchOp> ops = new();
			foreach (JsonProperty prop in element.EnumerateObject())
			{
				ops.AddRange(SearchOpHelpers.Parse(prop));
			}
			return new OrOp(ops);
		}
	}

	static class SearchOpHelpers
	{
		internal static IEnumerable<ISearchOp> Parse(JsonProperty prop)
		{
			if (prop.Name == "$and")
			{
				AndOp andOp = AndOp.Parse(prop.Value);
				yield return andOp;
			}
			else if (prop.Name == "$or")
			{
				OrOp orOp = OrOp.Parse(prop.Value);
				yield return orOp;
			}
			else if (prop.Name.StartsWith("$", StringComparison.InvariantCultureIgnoreCase))
			{
				throw new UnsupportedOperationException($"Unsupported operation in query: {prop.Name}");
			}
			else
			{
				bool opFound = false;
				// this is a sub object, parse that
				foreach (CompareOp logicOp in CompareOp.Parse(prop.Name, prop.Value))
				{
					opFound = true;
					yield return logicOp;
				}

				if (!opFound)
				{
					throw new InvalidSyntaxException(prop.Name, prop.Value.ToString());
				}
			}
		}

		internal static IEnumerable<ISearchOp> Parse(CbField field)
		{
			if (field.Name == new Utf8String("$and"))
			{
				AndOp andOp = AndOp.Parse(field.AsObject());
				yield return andOp;
			}
			else if (field.Name == new Utf8String("$or"))
			{
				OrOp orOp = OrOp.Parse(field.AsObject());
				yield return orOp;
			}
			else if (field.Name.StartsWith(new Utf8String("$")))
			{
				throw new UnsupportedOperationException($"Unsupported operation in query: {field.Name}");
			}
			else if (field.IsObject())
			{
				bool opFound = false;
				// this is a sub object, parse that
				foreach (CompareOp logicOp in CompareOp.Parse(field.Name.ToString(), field.AsObject()))
				{
					opFound = true;
					yield return logicOp;
				}

				if (!opFound)
				{
					throw new InvalidSyntaxException(field.Name.ToString(), field.Value!);
				}
			}
			else
			{
				throw new NotImplementedException($"Unsupported token in query: {field.Name}");
			}
		}
	}
	public class InvalidSyntaxException : Exception
	{
		public InvalidSyntaxException(string key, object value): base( $"Invalid syntax for key ({key}) with value {value}")
		{
		}
	}

	public class UnsupportedOperationException : Exception
	{
		public UnsupportedOperationException(string s) :base(s)
		{
		}
	}
}
