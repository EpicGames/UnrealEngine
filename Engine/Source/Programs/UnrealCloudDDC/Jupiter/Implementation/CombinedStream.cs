// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;

namespace Jupiter.Controllers;

internal class CombinedStream : Stream
{
	private readonly List<Stream> _streams;
	private readonly long _length;

	public CombinedStream(IEnumerable<Stream> streams)
	{
		_streams = streams.ToList();
		_length = _streams.Sum(stream => stream.Length);
	}
	public override int Read(byte[] buffer, int offset, int count)
	{
		int totalBytesRead = 0;

		foreach (Stream stream in _streams)
		{
			if (count <= 0)
			{
				break;
			}

			int bytesRead = stream.Read(buffer, offset, count);
			while (bytesRead != 0)
			{
				totalBytesRead += bytesRead;
				offset += bytesRead;
				count -= bytesRead;

				if (count <= 0)
				{
					break;
				}

				bytesRead = stream.Read(buffer, offset, count);
			}
		}

		return totalBytesRead;
	}

	public override void Flush()
	{
		throw new NotImplementedException();
	}
	public override long Seek(long offset, SeekOrigin origin)
	{
		throw new NotImplementedException();
	}

	public override void SetLength(long value)
	{
		throw new NotImplementedException();
	}

	public override void Write(byte[] buffer, int offset, int count)
	{
		throw new NotImplementedException();
	}

	public override bool CanRead { get; } = true;
	public override bool CanSeek { get; } = false;
	public override bool CanWrite { get; } = false;

	public override long Length => _length;

	public override long Position { get; set; }
}
