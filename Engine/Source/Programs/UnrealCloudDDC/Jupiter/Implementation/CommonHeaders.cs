// Copyright Epic Games, Inc. All Rights Reserved.

namespace Jupiter.Implementation
{
	public static class CommonHeaders
	{
		public const string HashHeaderSHA1Name = "X-Jupiter-Sha1";
		public const string HashHeaderName = "X-Jupiter-IoHash";

		public const string LastAccessHeaderName = "X-Jupiter-LastAccess";

		public const string InlinePayloadHash = "X-Jupiter-InlinePayloadHash";

		public const string AllowOverwrite = "X-Jupiter-Allow-Overwrite";
		public const string ForceConsistentRead = "X-Jupiter-Force-Consistent-Read";
	}
}
