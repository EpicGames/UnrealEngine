// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net;
using System.Net.Http;
using System.Net.Http.Json;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Controllers;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class CacheContentIdStore : RelayStore, IContentIdStore
	{

		public CacheContentIdStore(IOptionsMonitor<UpstreamRelaySettings> settings, IHttpClientFactory httpClientFactory, IServiceCredentials serviceCredentials) : base(settings, httpClientFactory, serviceCredentials)
		{
		}

		public async Task<BlobId[]?> ResolveAsync(NamespaceId ns, ContentId contentId, bool mustBeContentId, CancellationToken cancellationToken)
		{
			using HttpRequestMessage getContentIdRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/content-id/{ns}/{contentId}", UriKind.Relative));
			HttpResponseMessage response = await HttpClient.SendAsync(getContentIdRequest, cancellationToken);

			if (response.StatusCode == HttpStatusCode.NotFound)
			{
				return null;
			}

			response.EnsureSuccessStatusCode();
			ResolvedContentIdResponse? resolvedContentId = await response.Content.ReadFromJsonAsync<ResolvedContentIdResponse>(cancellationToken);

			if (resolvedContentId == null)
			{
				throw new Exception("Unable to deserialize resolved content id response");
			}
			return resolvedContentId.Blobs;
		}

		public async Task PutAsync(NamespaceId ns, ContentId contentId, BlobId[] blobIdentifiers, ulong compressedSize, ulong rawSize, CancellationToken cancellationToken)
		{
			foreach (BlobId blobIdentifier in blobIdentifiers)
			{
				using HttpRequestMessage putContentIdRequest = await BuildHttpRequestAsync(HttpMethod.Put, new Uri($"api/v1/content-id/{ns}/{contentId}/update/{blobIdentifier}/{compressedSize}/{rawSize}", UriKind.Relative));
				HttpResponseMessage response = await HttpClient.SendAsync(putContentIdRequest, cancellationToken);

				response.EnsureSuccessStatusCode();
			}
		}

		public IAsyncEnumerable<ContentIdMapping> GetContentIdMappingsAsync(NamespaceId ns, ContentId identifier, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}
	}
}
