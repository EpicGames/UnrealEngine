// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Serialization;
using Jupiter.Common.Implementation;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class CompressedBufferUtils
	{
		private readonly Tracer _tracer;
		private readonly BufferedPayloadFactory _payloadFactory;

		public CompressedBufferUtils(Tracer tracer, BufferedPayloadFactory payloadFactory)
		{
			_tracer = tracer;
			_payloadFactory = payloadFactory;
		}

		public async Task<(IBufferedPayload, IoHash)> DecompressContentAsync(Stream sourceStream, ulong streamSize, CancellationToken cancellationToken = default)
		{
			CompressedBufferHeader header;
			using FilesystemBufferedPayloadWriter bufferedPayloadWriter = _payloadFactory.CreateFilesystemBufferedPayloadWriter("cb-decompress");
			{
				await using Stream targetStream = bufferedPayloadWriter.GetWritableStream();
				header = await CompressedBuffer.DecompressContentAsync(sourceStream, streamSize, targetStream, cancellationToken).ConfigureAwait(false);
			}

			// not using the buffered payload as we transfer the ownership to the caller of this method
			FilesystemBufferedPayload? finalizedBufferedPayload = null;
			try
			{
				finalizedBufferedPayload = bufferedPayloadWriter.Done();

				if (header.TotalRawSize != (ulong)finalizedBufferedPayload.Length)
				{
					throw new Exception("Did not decompress the full payload");
				}

				{
					using TelemetrySpan _ = _tracer.StartActiveSpan("web.hash").SetAttribute("operation.name", "web.hash");

					// only read the first 20 bytes of the hash field as IoHashes are 20 bytes and not 32 bytes
					byte[] slicedHash = new byte[20];
					Array.Copy(header.RawHash, 0, slicedHash, 0, 20);

					BlobId headerIdentifier = new BlobId(slicedHash);
					await using Stream hashStream = finalizedBufferedPayload.GetStream();
					BlobId contentHash = await BlobId.FromStreamAsync(hashStream, cancellationToken);

					if (!headerIdentifier.Equals(contentHash))
					{
						throw new Exception($"Payload was expected to be {headerIdentifier} but was {contentHash}");
					}
				}

				return (finalizedBufferedPayload, new IoHash(header.RawHash));
			}
			catch
			{
				finalizedBufferedPayload?.Dispose();
				throw;
			}
		}
	}
}
