// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Jupiter.Common;
using Jupiter.DataStore;

namespace Jupiter.Implementation
{
	public class DataStoreContentIdStore : IContentIdStore
	{
		private readonly IBlobService _blobService;
		private readonly INamespacePolicyResolver _policyResolver;
		private readonly DataStoreTable<DataStoreCidRecord> _cidTable;

		public DataStoreContentIdStore(IDataStoreTableService tableService, IBlobService blobService, INamespacePolicyResolver policyResolver)
		{
			_blobService = blobService;
			_policyResolver = policyResolver;
			_cidTable = tableService.GetTableByType<DataStoreCidRecord>();
		}

		public async Task<BlobId[]?> ResolveAsync(NamespaceId ns, ContentId contentId, bool mustBeContentId = false, CancellationToken cancellationToken = default)
		{
			// if cid is not found, we should check if the cid resolves to a blob
			DataStorePartitionValue? value = await _cidTable.FindAsync(GetKeyspace(ns), CalculatePrefix(contentId), DataStoreCidRecord.GetPartitionKey(contentId), cancellationToken);
			DataStoreCidRecord? record = null;
			if (value != null)
			{
				value.AsObject(out DataStoreCidRecord innerRecord);
				record = innerRecord;
			}

			// TODO: Multiple records could match this cid and we should resolve them based on weight
			/*if (record != null && record.Blobs != null)
			{
				SortedList<int, BlobId> blobs = new SortedList<int, BlobId>();
				for (int index = 0; index < record.Blobs.Length; index++)
				{
					BlobId blob = record.Blobs[index];
					int weight = record.Weight![index];
					blobs.Add(weight, blob);
				}

				foreach (BlobId blob in blobs.Values)
				{
					if (await _blobService.ExistsAsync(ns, blob, cancellationToken: cancellationToken))
					{
						return [blob];
					}
				}

			}*/
			if (record != null)
			{
				BlobId b = BlobId.FromIoHash(record.Value.Blob);
				if (await _blobService.ExistsAsync(ns, b, cancellationToken: cancellationToken))
				{
					return [b];
				}
			}

			bool blobExists = await _blobService.ExistsAsync(ns, contentId.AsBlobIdentifier(), cancellationToken: cancellationToken);
			if (blobExists)
			{
				return [contentId.AsBlobIdentifier()];
			}

			return null;
		}

		public async Task PutAsync(NamespaceId ns, ContentId contentId, BlobId[] blobIdentifiers, ulong compressedSize, ulong rawSize, CancellationToken cancellationToken = default)
		{
			if (blobIdentifiers.Length != 1)
			{
				throw new NotImplementedException("Chunked blob identifiers not currently supported");
			}

			await _cidTable.InsertAsync(GetKeyspace(ns), CalculatePrefix(contentId), DataStoreCidRecord.GetPartitionKey(contentId), new DataStoreCidRecord(blobIdentifiers.First()), isFinalized: true, cancellationToken: cancellationToken);
			//await _cidTable.InsertAsync(ns, CalculatePrefix(contentId), new DataStoreCidRecord(blobIdentifiers.First(), contentId, uncompressedSize: rawSize, compressedSize: compressedSize), isFinalized: true, cancellationToken: cancellationToken);
		}

		public async IAsyncEnumerable<ContentIdMapping> GetContentIdMappingsAsync(NamespaceId ns, ContentId contentId, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			DataStorePartitionValue? value = await _cidTable.FindAsync(GetKeyspace(ns), CalculatePrefix(contentId), DataStoreCidRecord.GetPartitionKey(contentId), cancellationToken);

			if (value == null)
			{
				yield break;
			}

			// TODO: We should be able to have multiple blobs for the same content id
			//yield return new ContentIdMapping((int)record.UncompressedSize, [BlobId.FromIoHash(record.BlobId!.Value)]);
		}

		private static string CalculatePrefix(ContentId id)
		{
			return StringUtils.FormatAsHexLowerString(id.HashData.AsSpan()[0..1]);
		}

		private KeyspaceId GetKeyspace(NamespaceId ns)
		{
			return KeyspaceId.FromStoragePool(_policyResolver.GetPoliciesForNs(ns).StoragePool);
		}
	}

	internal record struct DataStoreCidRecord : IDataStoreTableObject<DataStoreCidRecord>
	{
		public IoHash Blob { get; set; }

		public int Weight { get; set; }

		public DataStoreCidRecord(BlobId blob)
		{
			Blob = blob.AsIoHash();
		}

		public static DataStorePartitionKey GetPartitionKey(ContentId cid)
		{
			return new DataStorePartitionKey(StringUtils.ToHashFromHexString(cid.ToString()));
		}

		public static DataStorePartitionKey? MaybeGetPartitionKey(ContentId? cid)
		{
			if (cid == null)
			{
				return null;
			}
			return new DataStorePartitionKey(StringUtils.ToHashFromHexString(cid.ToString()));
		}

		public static byte[] Serialize(DataStoreCidRecord self)
		{
			byte[] payload = new byte[20 + 4];
			MemoryWriter writer = new MemoryWriter(payload);

			writer.WriteFixedLengthBytes(self.Blob.ToByteArray());
			writer.WriteInt32(self.Weight);

			return payload;
		}

		public static void Deserialize(byte[] payload, out DataStoreCidRecord record)
		{
			MemoryReader reader = new MemoryReader(payload);
			BlobId hash = new BlobId(reader.ReadFixedLengthBytes(20).ToArray());
			int weight = reader.ReadInt32();

			record = new DataStoreCidRecord(hash) {Weight = weight};
		}

		public static bool HasSortKey()
		{
			return false;
		}

		public static byte[] GetSortKey(DataStoreCidRecord self)
		{
			return BitConverter.GetBytes(self.Weight);
		}
	}
}
