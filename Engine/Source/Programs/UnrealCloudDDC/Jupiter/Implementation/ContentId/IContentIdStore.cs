// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation
{
	public interface IContentIdStore
	{
		/// <summary>
		/// Resolve a content id from its hash into the actual blob (that can in turn be chunked into a set of blobs)
		/// </summary>
		/// <param name="ns">The namespace to operate in</param>
		/// <param name="contentId">The identifier for the content id</param>
		/// <param name="mustBeContentId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<BlobId[]?> ResolveAsync(NamespaceId ns, ContentId contentId, bool mustBeContentId = false, CancellationToken cancellationToken = default);

		/// <summary>
		/// Add a mapping from contentId to a chunked set of blobIdentifiers
		/// </summary>
		/// <param name="ns">The namespace to operate in</param>
		/// <param name="contentId">The contentId</param>
		/// <param name="blobIdentifiers">The blob the content id maps to</param>
		/// <param name="compressedSize">Size of the blob when compressed</param>
		/// <param name="rawSize">Size of the blob when decompressed</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task PutAsync(NamespaceId ns, ContentId contentId, BlobId[] blobIdentifiers, ulong compressedSize, ulong rawSize, CancellationToken cancellationToken = default);

		/// <summary>
		/// Retrieve the underlying content id mapping structure
		/// </summary>
		/// <param name="ns">The namespace to operate in</param>
		/// <param name="identifier">The content id you wish to fetch the mapping for</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>All existing mappings for that content id</returns>
		IAsyncEnumerable<ContentIdMapping> GetContentIdMappingsAsync(NamespaceId ns, ContentId identifier, CancellationToken cancellationToken = default);
	}

	public class ContentIdMapping
	{
		public ContentIdMapping(int weight, BlobId[] referencedBlobs)
		{
			Weight = weight;
			ReferencedBlobs = referencedBlobs;
		}

		public int Weight { get; set; }
		public BlobId[] ReferencedBlobs { get; set; }
	}

	public class InvalidContentIdException : Exception
	{
		public InvalidContentIdException(ContentId contentId) : base($"Unknown content id {contentId}")
		{

		}
	}

	public class ContentIdResolveException : Exception
	{
		public ContentId ContentId { get; }

		public ContentIdResolveException(ContentId contentId) : base($"Unable to find any mapping of contentId {contentId} that has all blobs present")
		{
			ContentId = contentId;
		}
	}
}
