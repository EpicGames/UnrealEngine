// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class MemoryCachedContentIdStore : IContentIdStore
	{
		private readonly IContentIdStore _actualContentIdStore;
		private readonly ConcurrentDictionary<NamespaceId, MemoryCache> _contentIdCaches = new ConcurrentDictionary<NamespaceId, MemoryCache>();
		private readonly IOptionsMonitor<MemoryCacheContentIdSettings> _options;

		public MemoryCachedContentIdStore(IContentIdStore actualContentIdStore, IOptionsMonitor<MemoryCacheContentIdSettings> options)
		{
			_actualContentIdStore = actualContentIdStore;
			_options = options;
		}

		public IContentIdStore GetUnderlyingContentIdStore()
		{
			return _actualContentIdStore;
		}

		public void Clear()
		{
			_contentIdCaches.Clear();
		}

		public async Task<BlobId[]?> ResolveAsync(NamespaceId ns, ContentId contentId, bool mustBeContentId, CancellationToken cancellationToken)
		{
			MemoryCache cache = GetCacheForNamespace(ns);
			if (cache.TryGetValue(contentId, out CachedContentIdEntry? cachedResult))
			{
				return cachedResult?.ReferencedBlobs;
			}

			BlobId[]? referencedBlobs = await _actualContentIdStore.ResolveAsync(ns, contentId, mustBeContentId, cancellationToken);
			// we do not cache unresolvable content ids
			if (referencedBlobs == null)
			{
				return null;
			}

			AddCacheEntry(ns, contentId, referencedBlobs);

			return referencedBlobs;
		}

		public IAsyncEnumerable<ContentIdMapping> GetContentIdMappingsAsync(NamespaceId ns, ContentId identifier, CancellationToken cancellationToken)
		{
			return _actualContentIdStore.GetContentIdMappingsAsync(ns, identifier, cancellationToken);
		}

		public Task PutAsync(NamespaceId ns, ContentId contentId, BlobId[] blobIdentifiers, ulong compressedSize, ulong rawSize, CancellationToken cancellationToken)
		{
			Task actualPutTask = _actualContentIdStore.PutAsync(ns, contentId, blobIdentifiers, compressedSize, rawSize, cancellationToken);
			AddCacheEntry(ns, contentId, blobIdentifiers);
			return actualPutTask;
		}

		private void AddCacheEntry(NamespaceId ns, ContentId contentId, BlobId[] blobs)
		{
			MemoryCache cache = GetCacheForNamespace(ns);

			CachedContentIdEntry cachedEntry = new CachedContentIdEntry
			{
				ReferencedBlobs = blobs
			};
			using ICacheEntry entry = cache.CreateEntry(contentId);
			entry.Value = cachedEntry;
			entry.Size = cachedEntry.GetSize();

			if (_options.CurrentValue.EnableSlidingExpiry)
			{
				entry.SlidingExpiration = TimeSpan.FromMinutes(_options.CurrentValue.SlidingExpirationMinutes);
			}
		}

		private MemoryCache GetCacheForNamespace(NamespaceId ns)
		{
			return _contentIdCaches.GetOrAdd(ns, id => new MemoryCache(_options.CurrentValue));
		}
	}

	class CachedContentIdEntry
	{
		public int GetSize() { return ReferencedBlobs.Length * 20; }

		public BlobId[] ReferencedBlobs { get; set; } = Array.Empty<BlobId>();
	}
}
