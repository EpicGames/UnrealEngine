// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation
{
	public class MemoryContentIdStore : IContentIdStore
	{
		private readonly IBlobService _blobStore;
		private readonly ConcurrentDictionary<NamespaceId, ConcurrentDictionary<ContentId, SortedList<int, BlobId[]>>> _contentIds = new ConcurrentDictionary<NamespaceId, ConcurrentDictionary<ContentId, SortedList<int, BlobId[]>>>();

		public MemoryContentIdStore(IBlobService blobStore)
		{
			_blobStore = blobStore;
		}

		public async Task<BlobId[]?> ResolveAsync(NamespaceId ns, ContentId contentId, bool mustBeContentId, CancellationToken cancellationToken)
		{
			if (_contentIds.TryGetValue(ns, out ConcurrentDictionary<ContentId, SortedList<int, BlobId[]>>? contentIdsForNamespace))
			{
				if (contentIdsForNamespace.TryGetValue(contentId, out SortedList<int, BlobId[]>? contentIdMappings))
				{
					foreach ((int _, BlobId[] blobs) in contentIdMappings)
					{
						BlobId[] missingBlobs = await _blobStore.FilterOutKnownBlobsAsync(ns, blobs, cancellationToken);
						if (missingBlobs.Length == 0)
						{
							return blobs;
						}
						// blobs are missing continue testing with the next content id in the weighted list as that might exist
					}
				}
			}

			BlobId uncompressedBlobIdentifier = contentId.AsBlobIdentifier();
			// if no content id is found, but we have a blob that matches the content id (so a unchunked and uncompressed version of the data) we use that instead
			if (!mustBeContentId && await _blobStore.ExistsAsync(ns, uncompressedBlobIdentifier, cancellationToken: cancellationToken))
			{
				return new[] { uncompressedBlobIdentifier };
			}

			return null;
		}

		public Task PutAsync(NamespaceId ns, ContentId contentId, BlobId[] blobIdentifiers, ulong compressedSize, ulong rawSize, CancellationToken cancellationToken)
		{
			int contentWeight = (int)compressedSize;
			_contentIds.AddOrUpdate(ns, (_) =>
			{
				ConcurrentDictionary<ContentId, SortedList<int, BlobId[]>> dict = new()
				{
					[contentId] = new SortedList<int, BlobId[]>
					{
						{contentWeight, blobIdentifiers }
					}
				};

				return dict;
			}, (_, dict) =>
			{
				dict.AddOrUpdate(contentId, (_) =>
				{
					return new SortedList<int, BlobId[]>
					{
						{ contentWeight, blobIdentifiers }
					};
				}, (_, mappings) =>
				{
					mappings[contentWeight] = blobIdentifiers;
					return mappings;
				});

				return dict;
			});

			return Task.CompletedTask;
		}

		public async IAsyncEnumerable<ContentIdMapping> GetContentIdMappingsAsync(NamespaceId ns, ContentId identifier, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			await Task.CompletedTask;

			if (_contentIds.TryGetValue(ns, out ConcurrentDictionary<ContentId, SortedList<int, BlobId[]>>? contentIdsForNamespace))
			{
				if (contentIdsForNamespace.TryGetValue(identifier, out SortedList<int, BlobId[]>? contentIdMappings))
				{
					// copy the mappings so that it can be updated while this enumerator is running
					SortedList<int, BlobId[]> mappings = new SortedList<int, BlobId[]>(contentIdMappings);
					foreach ((int weight, BlobId[] blobs) in mappings)
					{
						yield return new ContentIdMapping(weight, blobs);
					}
				}
			}
		}
	}
}
