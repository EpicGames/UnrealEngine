// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Options;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Bson.Serialization.Options;
using MongoDB.Driver;

namespace Jupiter.Implementation
{
	public class MongoContentIdStore : MongoStore, IContentIdStore
	{
		private readonly IBlobService _blobStore;

		public MongoContentIdStore(IBlobService blobStore, IOptionsMonitor<MongoSettings> settings) : base(settings)
		{
			_blobStore = blobStore;

			CreateCollectionIfNotExistsAsync<MongoContentIdModelV0>().Wait();

			IndexKeysDefinitionBuilder<MongoContentIdModelV0> indexKeysDefinitionBuilder = Builders<MongoContentIdModelV0>.IndexKeys;
			CreateIndexModel<MongoContentIdModelV0> indexModel = new CreateIndexModel<MongoContentIdModelV0>(
				indexKeysDefinitionBuilder.Combine(
					indexKeysDefinitionBuilder.Ascending(m => m.Ns),
					indexKeysDefinitionBuilder.Ascending(m => m.ContentId)
					)
				, new CreateIndexOptions()
				{
					Name = "CompoundIndex"
				});

			AddIndexFor<MongoContentIdModelV0>().CreateOne(indexModel);
		}

		public async Task<BlobId[]?> ResolveAsync(NamespaceId ns, ContentId contentId, bool mustBeContentId, CancellationToken cancellationToken)
		{
			IMongoCollection<MongoContentIdModelV0> collection = GetCollection<MongoContentIdModelV0>();

			IAsyncCursor<MongoContentIdModelV0> cursor =
				await collection.FindAsync(model => model.Ns == ns.ToString() && model.ContentId == contentId.ToString(), cancellationToken: cancellationToken);

			MongoContentIdModelV0 model = await cursor.FirstOrDefaultAsync(cancellationToken);
			if (model != null)
			{
				foreach (int weight in model.ContentWeightToBlobsMap.Keys.OrderBy(contentWeight => contentWeight))
				{
					BlobId[] blobs = model.ContentWeightToBlobsMap[weight].Select(s => new BlobId(s)).ToArray();

					{
						BlobId[] missingBlobs = await _blobStore.FilterOutKnownBlobsAsync(ns, blobs, cancellationToken);
						if (missingBlobs.Length == 0)
						{
							return blobs;
						}
					}
					// blobs are missing continue testing with the next content id in the weighted list as that might exist

				}
			}

			BlobId contentIdAsBlobIdentifier = contentId.AsBlobIdentifier();
			// no content id where all blobs are present, check if its present in the blob store as a uncompressed version of the blob
			if (!mustBeContentId && await _blobStore.ExistsAsync(ns, contentIdAsBlobIdentifier, cancellationToken: cancellationToken))
			{
				return new[] { contentIdAsBlobIdentifier };
			}

			return null;
		}

		public async Task PutAsync(NamespaceId ns, ContentId contentId, BlobId[] blobIdentifiers, ulong compressedSize, ulong rawSize, CancellationToken cancellationToken)
		{
			IMongoCollection<MongoContentIdModelV0> collection = GetCollection<MongoContentIdModelV0>();

			UpdateDefinition<MongoContentIdModelV0> update = Builders<MongoContentIdModelV0>.Update.AddToSet(m => m.ContentWeightToBlobsMap,
				new KeyValuePair<int, string[]>((int)compressedSize, blobIdentifiers.Select(id => id.ToString()).ToArray()));
			FilterDefinition<MongoContentIdModelV0> filter = Builders<MongoContentIdModelV0>.Filter.Where(m => m.Ns == ns.ToString() && m.ContentId == contentId.ToString());
			await collection.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<MongoContentIdModelV0>()
			{
				IsUpsert = true
			}, cancellationToken);
		}

		public async IAsyncEnumerable<ContentIdMapping> GetContentIdMappingsAsync(NamespaceId ns, ContentId contentId, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			IMongoCollection<MongoContentIdModelV0> collection = GetCollection<MongoContentIdModelV0>();

			IAsyncCursor<MongoContentIdModelV0> cursor = await collection.FindAsync(model => model.Ns == ns.ToString() && model.ContentId == contentId.ToString(), cancellationToken: cancellationToken);

			MongoContentIdModelV0 model = await cursor.FirstOrDefaultAsync(cancellationToken);

			foreach (KeyValuePair<int, string[]> entry in model.ContentWeightToBlobsMap)
			{
				yield return new ContentIdMapping(entry.Key, entry.Value.Select(s => new BlobId(s)).ToArray());
			}
		}
	}

	[BsonDiscriminator("content-id.v0")]
	[BsonIgnoreExtraElements]
	[MongoCollectionName("ContentId")]
	public class MongoContentIdModelV0
	{
		[BsonConstructor]
		public MongoContentIdModelV0(string ns, string contentId, Dictionary<int, string[]> contentWeightToBlobsMap)
		{
			Ns = ns;
			ContentId = contentId;
			ContentWeightToBlobsMap = contentWeightToBlobsMap;
		}

		public MongoContentIdModelV0(NamespaceId ns, ContentId contentId, int contentWeight, BlobId[] blobs)
		{
			Ns = ns.ToString();
			ContentId = contentId.ToString();

			ContentWeightToBlobsMap = new Dictionary<int, string[]>
			{
				[contentWeight] = blobs.Select(identifier => identifier.ToString()).ToArray()
			};
		}

		[BsonRequired]
		public string Ns { get; set; }

		[BsonRequired]
		public string ContentId { get; set; }

		[BsonDictionaryOptions(DictionaryRepresentation.ArrayOfDocuments)]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public Dictionary<int, string[]> ContentWeightToBlobsMap { get; set; }
	}
}
