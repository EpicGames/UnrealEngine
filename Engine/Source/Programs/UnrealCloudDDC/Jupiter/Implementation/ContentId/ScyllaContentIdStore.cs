// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Cassandra;
using Cassandra.Mapping;
using Jupiter.Common;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class ScyllaContentIdStore : IContentIdStore
	{
		private readonly ISession _session;
		private readonly IBlobService _blobStore;
		private readonly Tracer _tracer;
		private readonly IOptionsMonitor<ScyllaSettings> _settings;
		private readonly Mapper _mapper;
		private readonly IScyllaSessionManager _scyllaSessionManager;

		public ScyllaContentIdStore(IScyllaSessionManager scyllaSessionManager, IBlobService blobStore, Tracer tracer, IOptionsMonitor<ScyllaSettings> settings)
		{
			_scyllaSessionManager = scyllaSessionManager;
			_session = scyllaSessionManager.GetSessionForReplicatedKeyspace();
			_blobStore = blobStore;
			_tracer = tracer;
			_settings = settings;

			_mapper = new Mapper(_session);

			if (!settings.CurrentValue.AvoidSchemaChanges)
			{
				string blobType = scyllaSessionManager.IsCassandra ? "blob" : "frozen<blob_identifier>";
				_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS content_id (
					content_id {blobType},
					content_weight int, 
					chunks set<{blobType}>, 
					PRIMARY KEY ((content_id), content_weight)
				);"
				));
			}
		}

		public async Task<BlobId[]?> ResolveAsync(NamespaceId ns, ContentId contentId, bool mustBeContentId, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("ScyllaContentIdStore.ResolveContentId").SetAttribute("resource.name", contentId.ToString());

			BlobId contentIdBlob = contentId.AsBlobIdentifier();

			{
				using TelemetrySpan contentIdFetchScope = _tracer.BuildScyllaSpan("ScyllaContentIdStore.FetchContentId").SetAttribute("resource.name", contentId.ToString());

				const string ConsistentProfile = "local_consistent";
				const string DefaultProfile = "default";
				string executionProfile = _settings.CurrentValue.UseConsistentContentIdResolve ? ConsistentProfile : DefaultProfile;
				const int MaxRetries = 3;
				for (int i = 0; i < MaxRetries; i++)
				{
					try
					{
						if (_scyllaSessionManager.IsScylla)
						{
							// lower content_weight means its a better candidate to resolve to
							foreach (ScyllaContentId? resolvedContentId in await _mapper.FetchAsync<ScyllaContentId>(Cql.New("WHERE content_id = ? ORDER BY content_weight ASC", new ScyllaBlobIdentifier(contentId)).WithExecutionProfile(executionProfile)))
							{
								if (resolvedContentId == null)
								{
									throw new InvalidContentIdException(contentId);
								}

								BlobId[] blobs = resolvedContentId.Chunks.Select(b => b.AsBlobIdentifier()).ToArray();

								{
									using TelemetrySpan _ = _tracer.StartActiveSpan("ScyllaContentIdStore.FindMissingBlobs").SetAttribute("operation.name", "ScyllaContentIdStore.FindMissingBlobs");

									BlobId[] missingBlobs = await _blobStore.FilterOutKnownBlobsAsync(ns, blobs, cancellationToken);
									if (missingBlobs.Length == 0)
									{
										return blobs;
									}
								}
								// blobs are missing continue testing with the next content id in the weighted list as that might exist
							}
						}
						else
						{
							// lower content_weight means its a better candidate to resolve to
							foreach (CassandraContentId? resolvedContentId in await _mapper.FetchAsync<CassandraContentId>(Cql.New("WHERE content_id = ? ORDER BY content_weight ASC", contentId.HashData).WithExecutionProfile(executionProfile)))
							{
								if (resolvedContentId == null)
								{
									throw new InvalidContentIdException(contentId);
								}

								BlobId[] blobs = resolvedContentId.Chunks.Select(b => new BlobId(b!)).ToArray();

								{
									using TelemetrySpan _ = _tracer.StartActiveSpan("ScyllaContentIdStore.FindMissingBlobs").SetAttribute("operation.name", "ScyllaContentIdStore.FindMissingBlobs");

									BlobId[] missingBlobs = await _blobStore.FilterOutKnownBlobsAsync(ns, blobs, cancellationToken);
									if (missingBlobs.Length == 0)
									{
										return blobs;
									}
								}
								// blobs are missing continue testing with the next content id in the weighted list as that might exist
							}
						}
					}
					catch (UnavailableException)
					{
						// not enough replicas available fall back to non-consistent reads to limit impact
						executionProfile = DefaultProfile;
						continue;
					}
					catch (QueryExecutionException e)
					{
						// Other communication issues occurred
						throw new RemoteServiceException("Error communicating with Scylla", e);
					}

					break;
				}
			}

			if (!mustBeContentId)
			{
				// if no content id is found, but we have a blob that matches the content id (so a unchunked and uncompressed version of the data) we use that instead
				bool contentIdBlobExists = await _blobStore.ExistsAsync(ns, contentIdBlob, cancellationToken: cancellationToken)!;

				if (contentIdBlobExists)
				{
					return new[] { contentIdBlob };
				}
			}

			// unable to resolve the content id
			return null;
		}

		public async IAsyncEnumerable<ContentIdMapping> GetContentIdMappingsAsync(NamespaceId ns, ContentId contentId, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("ScyllaContentIdStore.GetContentIdMappingsAsync").SetAttribute("resource.name", contentId.ToString());

			if (_scyllaSessionManager.IsScylla)
			{
				// lower content_weight means its a better candidate to resolve to
				foreach (ScyllaContentId? resolvedContentId in await _mapper.FetchAsync<ScyllaContentId>("WHERE content_id = ? ORDER BY content_weight ASC", new ScyllaBlobIdentifier(contentId)))
				{
					if (resolvedContentId == null || resolvedContentId.ContentWeight == null || resolvedContentId.Chunks == null)
					{
						throw new InvalidContentIdException(contentId);
					}

					yield return new ContentIdMapping(resolvedContentId.ContentWeight.Value, resolvedContentId.Chunks.Select(identifier => identifier.AsBlobIdentifier()).ToArray());
				}
			}
			else
			{
				// lower content_weight means its a better candidate to resolve to
				foreach (CassandraContentId? resolvedContentId in await _mapper.FetchAsync<CassandraContentId>("WHERE content_id = ? ORDER BY content_weight ASC", contentId.HashData))
				{
					if (resolvedContentId == null || resolvedContentId.ContentWeight == null || resolvedContentId.Chunks == null)
					{
						throw new InvalidContentIdException(contentId);
					}

					yield return new ContentIdMapping(resolvedContentId.ContentWeight.Value, resolvedContentId.Chunks.Select(identifier => new BlobId(identifier)).ToArray());
				}
			}
		}

		public async Task PutAsync(NamespaceId ns, ContentId contentId, BlobId[] blobIdentifiers, ulong compressedSize, ulong rawSize, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("ScyllaContentIdStore.PutContentId").SetAttribute("resource.name", contentId.ToString());

			int contentWeight = (int)compressedSize;
			try
			{
				if (_scyllaSessionManager.IsScylla)
				{
					await _mapper.UpdateAsync<ScyllaContentId>("SET chunks = ? WHERE content_id = ? AND content_weight = ?", blobIdentifiers.Select(id => new ScyllaBlobIdentifier(id)).ToArray(), new ScyllaBlobIdentifier(contentId), contentWeight);

				}
				else
				{
					await _mapper.UpdateAsync<CassandraContentId>("SET chunks = ? WHERE content_id = ? AND content_weight = ?", blobIdentifiers.Select(id => id.HashData).ToArray(), contentId.HashData, contentWeight);
				}
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}
	}

	[Cassandra.Mapping.Attributes.Table("content_id")]
	public class ScyllaContentId
	{
		public ScyllaContentId()
		{

		}

		public ScyllaContentId(BlobId contentId, BlobId[] chunks, int contentWeight)
		{
			ContentId = new ScyllaBlobIdentifier(contentId);
			ContentWeight = contentWeight;
			Chunks = chunks.Select(b => new ScyllaBlobIdentifier(b)).ToArray();
		}

		[Cassandra.Mapping.Attributes.PartitionKey]
		[Cassandra.Mapping.Attributes.Column("content_id")]
		public ScyllaBlobIdentifier? ContentId { get; set; }

		[Cassandra.Mapping.Attributes.ClusteringKey]
		[Cassandra.Mapping.Attributes.Column("content_weight")]
		public int? ContentWeight { get; set; }

		public ScyllaBlobIdentifier[] Chunks { get; set; } = Array.Empty<ScyllaBlobIdentifier>();
	}

	[Cassandra.Mapping.Attributes.Table("content_id")]
	public class CassandraContentId
	{
		public CassandraContentId()
		{

		}

		public CassandraContentId(BlobId contentId, BlobId[] chunks, int contentWeight)
		{
			ContentId = contentId.HashData;
			ContentWeight = contentWeight;
			Chunks = chunks.Select(b => b.HashData).ToArray();
		}

		[Cassandra.Mapping.Attributes.PartitionKey]
		[Cassandra.Mapping.Attributes.Column("content_id")]
		public byte[]? ContentId { get; set; }

		[Cassandra.Mapping.Attributes.ClusteringKey]
		[Cassandra.Mapping.Attributes.Column("content_weight")]
		public int? ContentWeight { get; set; }

		public byte[][] Chunks { get; set; } = Array.Empty<byte[]>();
	}
}
