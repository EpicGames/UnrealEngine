// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

namespace Jupiter.DataStore;

public class DataStoreSettings
{
	public string RootDir { get; set; } = "";
	public bool Enabled { get; set; } = false;
}

public record DataStoreFile(string Path, long Size, DateTime LastModifiedUtc)
{
}

public interface IDataStoreObjectStore
{
	Task PutFileAsync(KeyspaceId keyspace, FileInfo localPath, string remotePath, CancellationToken cancellationToken);
	IAsyncEnumerable<(KeyspaceId keyspace, string prefix)> EnumeratePrefixesAsync(TableType typeName, CancellationToken cancellationToken);
	Task<bool> GetFileAsync(KeyspaceId keyspace, string remotePath, FileInfo localFile, CancellationToken token);
	IAsyncEnumerable<DataStoreFile> EnumerateFilesAsync(KeyspaceId keyspace, string remotePath, CancellationToken cancellationToken);
	Task DeleteFilesAsync(KeyspaceId keyspace, string remotePath, CancellationToken cancellationToken);
}
