// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Runtime.CompilerServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace Jupiter.DataStore;
public record CbObjectStreamHeader
{
	private const uint ExpectedMagic = 0xb7756362;
	private const uint CurrentVersion = 2;

	public CbObjectStreamHeader()
	{
		Magic = ExpectedMagic;
		Version = CurrentVersion;
	}

	public void WriteToStream(Stream s)
	{
		using BinaryWriter bw = new BinaryWriter(s, Encoding.ASCII, leaveOpen: true);

		bw.Write(Magic);
		bw.Write(Version);
		bw.Write(0u);
		bw.Write(0u);
	}

	public static CbObjectStreamHeader ReadHeader(Stream s)
	{
		CbObjectStreamHeader header = new CbObjectStreamHeader();

		using BinaryReader br = new BinaryReader(s, Encoding.ASCII, leaveOpen: true);
		uint magic = br.ReadUInt32();
		if (magic != ExpectedMagic)
		{
			throw new InvalidStreamException("Magic did not match, stream was not a cb object stream");
		}

		header.Magic = magic;

		uint version = br.ReadUInt32();
		header.Version = version;

		if (version != CurrentVersion)
		{
			throw new InvalidStreamException($"Unsupported version found version {version} but expected version {CurrentVersion}");
		}

		// skip the 2 reserved uints
		uint reserved0 = br.ReadUInt32();
		uint reserved1 = br.ReadUInt32();

		return header;
	}

	public uint Version { get; set; }

	public uint Magic { get; set; }
}

internal class DataStoreObjectStreamReader(FileStream fs, int bufferSize) : IDisposable, IAsyncDisposable
{
	public void Dispose()
	{
		fs.Dispose();
	}

	public async ValueTask DisposeAsync()
	{
		await fs.DisposeAsync();
	}
		
	public async IAsyncEnumerable<(DataStorePartitionKey, byte[])> GetObjectBuffersAsync([EnumeratorCancellation] CancellationToken cancellationToken)
	{
		using IMemoryOwner<byte> owner = MemoryPool<byte>.Shared.Rent(bufferSize);
		Memory<byte> savedMem = owner.Memory;
		int amountOfMemorySaved = 0;

		CbObjectStreamHeader header = CbObjectStreamHeader.ReadHeader(fs);
		// make sure its version 2
		Debug.Assert(header.Version == 2);

		// we need at least 24 bytes to be sure we have enough space to fit the key and the payload length int
		const int NeededBytesRemaining = 24;

		// double buffered read all bytes of the stream, enqueuing objects as we find them
		ConcurrentQueue<(DataStorePartitionKey, byte[])> objects = new ConcurrentQueue<(DataStorePartitionKey, byte[])>(); 
		Task readTask = fs.ReadAllBytesAsync(bufferSize, (mem) =>
		{
			int bytesRemaining = mem.Length;
			if (amountOfMemorySaved != 0)
			{
				// had partial read in the previous iteration, combining the two buffers together to form a complete object
				using IMemoryOwner<byte> tempMemory = MemoryPool<byte>.Shared.Rent(bufferSize);

				int remainingRoomInBuffer = bytesRemaining - amountOfMemorySaved;
				savedMem[..amountOfMemorySaved].CopyTo(tempMemory.Memory);
				mem[0..remainingRoomInBuffer].CopyTo(tempMemory.Memory[amountOfMemorySaved..]);
				int usedInTempMemory = amountOfMemorySaved + remainingRoomInBuffer;

				MemoryReader reader = new MemoryReader(tempMemory.Memory[..usedInTempMemory]);
				DataStorePartitionKey key = new DataStorePartitionKey(reader.ReadFixedLengthBytes(20));
				int length = reader.ReadInt32();
				byte[] b = reader.ReadFixedLengthBytes(length).ToArray();

				int fieldSize = 20 + 4 + length;
				Debug.Assert(fieldSize <= FilesystemDataStoreTableLog.MaxObjectSize);

				int bytesFromNewMemory = fieldSize - amountOfMemorySaved;
				objects.Enqueue((key, b));

				mem = mem[bytesFromNewMemory..];
				amountOfMemorySaved = 0;
				bytesRemaining -= bytesFromNewMemory;
			}

			while (bytesRemaining > NeededBytesRemaining)
			{
				MemoryReader reader = new MemoryReader(mem);
				DataStorePartitionKey key = new DataStorePartitionKey(reader.ReadFixedLengthBytes(20));
				int length = reader.ReadInt32();
				
				int payloadSize = 20 + 4 + length;
				Debug.Assert(payloadSize <= FilesystemDataStoreTableLog.MaxObjectSize);
				// if the expected payload size is larger than what is left in the buffer we have made a partial read and need the next chunk
				if (payloadSize > bytesRemaining)
				{
					// partial read, we need to save these bytes for the next iteration
					mem[..bytesRemaining].CopyTo(savedMem);
					amountOfMemorySaved = bytesRemaining;
					bytesRemaining = 0;
					break;
				}
				else
				{
					byte[] b = reader.ReadFixedLengthBytes(length).ToArray();

					// full object read
					bytesRemaining -= payloadSize;

					objects.Enqueue((key, b));
					mem = mem[payloadSize..];
				}
			}

			if (bytesRemaining != 0)
			{
				// partial read, we need to save these bytes for the next iteration
				mem[..bytesRemaining].CopyTo(savedMem);
				amountOfMemorySaved = bytesRemaining;
			}

			return Task.CompletedTask;
		}, cancellationToken);

		while (true)
		{
			while (objects.TryDequeue(out (DataStorePartitionKey, byte[]) pair))
			{
				if (cancellationToken.IsCancellationRequested)
				{
					break;
				}

				yield return (pair.Item1, pair.Item2);
			}

			if (cancellationToken.IsCancellationRequested)
			{
				break;
			}
			// once all current objects have been processed wait a while before attempting again
			await Task.WhenAny(readTask, Task.Delay(1, cancellationToken));

			if (readTask.IsCompleted)
			{
				break;
			}
		}

		await readTask;

		// process any remaining objects
		while (objects.TryDequeue(out (DataStorePartitionKey, byte[]) pair))
		{
			if (cancellationToken.IsCancellationRequested)
			{
				break;
			}

			yield return (pair.Item1, pair.Item2);
		}
	}
}

internal class InvalidStreamException(string msg) : Exception(msg);