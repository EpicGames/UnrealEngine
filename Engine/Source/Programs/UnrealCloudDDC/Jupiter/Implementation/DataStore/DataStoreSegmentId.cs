// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.ComponentModel;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using EpicGames.Core;

namespace Jupiter.DataStore;

/// <summary>
/// SegmentId similar to an uuid type 4, also similar to CbObjectId but with a bigger time component to prevent overflows
/// </summary>
/// <param name="a">upper time component</param>
/// <param name="b">lower time component</param>
/// <param name="c">serial component</param>
/// <param name="d">run component</param>
[JsonConverter(typeof(DataStoreSegmentIdJsonConverter))]
[TypeConverter(typeof(DataStoreSegmentIdTypeConverter))]
#pragma warning disable CA1036 // we do not need the compare operators
public readonly struct DataStoreSegmentId(uint a, uint b, uint c, uint d) : IEquatable<DataStoreSegmentId>, IComparer<DataStoreSegmentId>, IComparable<DataStoreSegmentId>
#pragma warning restore CA1036
{
	private readonly uint _a = a;
	private readonly uint _b = b;
	private readonly uint _c = c;
	private readonly uint _d = d;

	// initialize the serial to a random integer
	private static uint s_objectIdSerial = CalculateRunId();

	/// <summary>
	/// The zero value 
	/// </summary>
	public static DataStoreSegmentId Zero { get; } = new DataStoreSegmentId(0u, 0u, 0u, 0u);

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="payload"></param>
	public DataStoreSegmentId(ReadOnlySpan<byte> payload) : this(BinaryPrimitives.ReadUInt32BigEndian(payload[0..4]), BinaryPrimitives.ReadUInt32BigEndian(payload[4..8]), BinaryPrimitives.ReadUInt32BigEndian(payload[8..12]), BinaryPrimitives.ReadUInt32BigEndian(payload[12..16]))
	{
	}

	/// <summary>
	/// Parses a segment id from the given hex string
	/// </summary>
	/// <param name="text"></param>
	/// <returns></returns>
	public static DataStoreSegmentId Parse(string text)
	{
		byte[] bytes = EpicGames.Core.StringUtils.ParseHexString(text);
		if (bytes.Length == 12)
		{
			return DataStoreSegmentId.FromObjectId(bytes);
		}
		if (bytes.Length != 16)
		{
			throw new Exception("SegmentId is expected to be 16 bytes if its not a CbObjectId");
		}
		return new DataStoreSegmentId(bytes);
	}

	private static DataStoreSegmentId FromObjectId(ReadOnlySpan<byte> payload)
	{
		return new DataStoreSegmentId(0, BinaryPrimitives.ReadUInt32BigEndian(payload[0..4]), BinaryPrimitives.ReadUInt32BigEndian(payload[4..8]), BinaryPrimitives.ReadUInt32BigEndian(payload[8..12]));
	}

	/// <summary>
	/// Parses a digest from the given hex string
	/// </summary>
	/// <param name="text"></param>
	/// <param name="objectId">Receives the object id if successful</param>
	/// <returns></returns>
	public static bool TryParse(string text, out DataStoreSegmentId objectId)
	{
		if (!EpicGames.Core.StringUtils.TryParseHexString(text, out byte[]? bytes))
		{
			objectId = Zero;
			return false;
		}
		if (bytes.Length == 12)
		{
			objectId = DataStoreSegmentId.FromObjectId(bytes);
			return true;
		}
		if (bytes.Length != 16)
		{
			objectId = Zero;
			return false;
		}
		objectId = new DataStoreSegmentId(bytes);
		return true;
	}

	/// <summary>
	/// Generates a new object id
	/// </summary>
	/// <returns></returns>
	public static DataStoreSegmentId NewId()
	{
		const long Offset = 1_609_459_200; // Seconds from 1970 -> 2021
		long longTime = DateTimeOffset.Now.ToUnixTimeSeconds() - Offset;

		byte[] b = new byte[8];
		Span<byte> span = b;
		BinaryPrimitives.WriteUInt64BigEndian(span, (ulong)longTime);
		uint timeUpper = BinaryPrimitives.ReadUInt32BigEndian(span[0..4]);
		uint timeLower = BinaryPrimitives.ReadUInt32BigEndian(span[4..8]);

		uint serial = Interlocked.Increment(ref s_objectIdSerial);
		uint runId = RunId;
		return new DataStoreSegmentId(timeUpper, timeLower, serial, runId);
	}

	private static uint RunId { get; } = CalculateRunId();

	private static uint CalculateRunId()
	{
		Guid g = Guid.NewGuid();
		return (uint)g.GetHashCode();
	}

	/// <summary>
	/// Copies this segment id into a span
	/// </summary>
	/// <param name="span"></param>
	public void CopyTo(Span<byte> span)
	{
		BinaryPrimitives.WriteUInt32BigEndian(span, _a);
		BinaryPrimitives.WriteUInt32BigEndian(span[4..], _b);
		BinaryPrimitives.WriteUInt32BigEndian(span[8..], _c);
		BinaryPrimitives.WriteUInt32BigEndian(span[12..], _d);
	}

	/// <summary>
	/// Converts this object id to a byte array
	/// </summary>
	/// <returns></returns>
	public byte[] ToByteArray()
	{
		byte[] bytes = new byte[16];
		CopyTo(bytes);
		return bytes;
	}

	/// <summary>
	/// Creates an utf8string of the segment id
	/// </summary>
	/// <returns></returns>
	public Utf8String ToUtf8String()
	{
		return EpicGames.Core.StringUtils.FormatUtf8HexString(ToByteArray());
	}

	/// <inheritdoc/>
	public override string ToString()
	{
		return EpicGames.Core.StringUtils.FormatHexString(ToByteArray());
	}

	/// <inheritdoc/>
	public bool Equals(DataStoreSegmentId other) => _a == other._a && _b == other._b && _c == other._c && _d == other._d;

	/// <inheritdoc/>
	public override bool Equals(object? obj) => (obj is DataStoreSegmentId cid) && Equals(cid);

	/// <inheritdoc/>
	public override int GetHashCode()
	{
		return HashCode.Combine(_a, _b, _c, _d);
	}

	/// <summary>
	/// Test two segment ids for equality
	/// </summary>
	public static bool operator ==(DataStoreSegmentId a, DataStoreSegmentId b) => a.Equals(b);

	/// <summary>
	/// Test two segment ids for equality
	/// </summary>
	public static bool operator !=(DataStoreSegmentId a, DataStoreSegmentId b) => !(a == b);

	/// <summary>
	/// Compare two segment ids by comparing their components
	/// </summary>
	public int Compare(DataStoreSegmentId x, DataStoreSegmentId y)
	{
		int aComparison = x._a.CompareTo(y._a);
		if (aComparison != 0)
		{
			return aComparison;
		}

		int bComparison = x._b.CompareTo(y._b);
		if (bComparison != 0)
		{
			return bComparison;
		}

		int cComparison = x._c.CompareTo(y._c);
		if (cComparison != 0)
		{
			return cComparison;
		}

		return x._d.CompareTo(y._d);
	}

	/// <summary>
	///  Compare two object ids by comparing their components
	/// </summary>
	public int CompareTo(DataStoreSegmentId other)
	{
		return Compare(this, other);
	}

	public DateTimeOffset AsTimeOffset()
	{
		byte[] b = new byte[8];
		Span<byte> span = b;

		BinaryPrimitives.WriteUInt32BigEndian(span[0..4], _a);
		BinaryPrimitives.WriteUInt32BigEndian(span[4..8], _b);

		const long Offset = 1_609_459_200; // Seconds from 1970 -> 2021
		long fullTime = (long)BinaryPrimitives.ReadUInt64BigEndian(span);
		fullTime += Offset;
		return DateTimeOffset.FromUnixTimeSeconds(fullTime).ToUniversalTime();
	}
}

/// <summary>
/// A comparer for inverse comparisons of a cb object (newest first)
/// </summary>
public sealed class DataStoreSegmentIdReverseComparer : IComparer<DataStoreSegmentId>
{
	/// <summary>
	/// Compare two segment ids returning the newest first
	/// </summary>
	public int Compare(DataStoreSegmentId x, DataStoreSegmentId y)
	{
		return y.CompareTo(x);
	}
}

/// <summary>
/// Type converter for DataStoreSegmentId to and from JSON
/// </summary>
internal sealed class DataStoreSegmentIdJsonConverter : JsonConverter<DataStoreSegmentId>
{
	/// <inheritdoc/>
	public override DataStoreSegmentId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options) => DataStoreSegmentId.Parse(reader.GetString()!);

	/// <inheritdoc/>
	public override void Write(Utf8JsonWriter writer, DataStoreSegmentId value, JsonSerializerOptions options) => writer.WriteStringValue(value.ToString());
}

/// <summary>
/// Type converter for DataStoreSegmentId
/// </summary>
internal sealed class DataStoreSegmentIdTypeConverter : TypeConverter
{
	public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType) => sourceType == typeof(string);

	/// <inheritdoc/>
	public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value) => DataStoreSegmentId.Parse((string)value);
}