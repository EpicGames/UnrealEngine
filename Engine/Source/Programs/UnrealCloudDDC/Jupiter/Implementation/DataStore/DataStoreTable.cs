// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.DependencyInjection;

namespace Jupiter.DataStore;

public class DataStoreTable<TObjectType> where TObjectType : struct, IDataStoreTableObject<TObjectType>
{
	private readonly IServiceProvider _provider;
	private readonly TableType _typeName;
	private readonly Func<KeyspaceId, TObjectType, Task>? _onObjectWritten;

	private readonly ConcurrentDictionary<KeyspaceId, ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>>> _partitions = new ConcurrentDictionary<KeyspaceId, ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>>>();
	private readonly IDataStoreTableLogService _tableLogService;

	public DataStoreTable(IServiceProvider provider, TableType typeName, Func<KeyspaceId, TObjectType, Task>? onObjectWritten = null)
	{
		_provider = provider;
		_typeName = typeName;
		_onObjectWritten = onObjectWritten;

		_tableLogService = _provider.GetRequiredService<IDataStoreTableLogService>();
	}

	public async Task InsertAsync(KeyspaceId keyspace, string prefix, DataStorePartitionKey key, TObjectType record, bool isFinalized, CancellationToken cancellationToken)
	{
		ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>> partitions = GetPartitionsForKeyspace(keyspace);
		DataStoreTablePartition<TObjectType> partition = partitions.GetOrAdd(prefix, _ => new DataStoreTablePartition<TObjectType>(_provider, keyspace, _typeName, prefix, _onObjectWritten));

		await partition.InsertAsync(key, record, isFinalized, cancellationToken);
	}

	public async Task EnsureRefreshAsync(KeyspaceId keyspace, string prefix, DataStorePartitionKey key, CancellationToken cancellationToken)
	{
		DataStoreTablePartition<TObjectType> partition = await GetPartitionForPrefixAsync(keyspace, prefix, cancellationToken);
		await partition.EnsureRefreshAsync(key, cancellationToken);
	}

	public async Task<DataStorePartitionValue?> FindAsync(KeyspaceId keyspace, string prefix, DataStorePartitionKey key, CancellationToken cancellationToken)
	{
		DataStoreTablePartition<TObjectType> partition = await GetPartitionForPrefixAsync(keyspace, prefix, cancellationToken);
		DataStorePartitionValue? value = await partition.FindAsync(key, cancellationToken);
		return value;
	}

	private ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>> GetPartitionsForKeyspace(KeyspaceId keyspace)
	{
		ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>> partitions = _partitions.GetOrAdd(keyspace, _ => new ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>>());
		return partitions;
	}

	private Task<DataStoreTablePartition<TObjectType>> GetPartitionForPrefixAsync(KeyspaceId keyspace, string prefix, CancellationToken cancellationToken)
	{
		ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>> partitions = GetPartitionsForKeyspace(keyspace);

		if (cancellationToken.IsCancellationRequested)
		{
			throw new TaskCanceledException();
		}

		DataStoreTablePartition<TObjectType> partition = partitions.GetOrAdd(prefix, _ => new DataStoreTablePartition<TObjectType>(_provider, keyspace, _typeName, prefix, _onObjectWritten));
		return Task.FromResult(partition);
	}

	public IEnumerable<KeyspaceId> GetKeyspacesAsync(CancellationToken cancellationToken)
	{
		foreach (KeyValuePair<KeyspaceId, ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>>> partition in _partitions)
		{
			if (cancellationToken.IsCancellationRequested)
			{
				yield break;
			}

			foreach (KeyValuePair<string, DataStoreTablePartition<TObjectType>> prefix in partition.Value)
			{
				if (!prefix.Value.IsEmpty)
				{
					// keyspace has some data present so its valid
					yield return partition.Key;
					break;
				}
			}
		}
	}

	public IEnumerable<string> GetPrefixesForNamespaceAsync(KeyspaceId keyspace, CancellationToken cancellationToken)
	{
		ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>> partitions = GetPartitionsForKeyspace(keyspace);

		foreach (string prefix in partitions.Keys)
		{
			if (cancellationToken.IsCancellationRequested)
			{
				yield break;
			}
			yield return prefix;
		}
	}

	public async Task HydratePrefixAsync(KeyspaceId keyspace, string prefix, CancellationToken cancellationToken)
	{
		DataStoreTablePartition<TObjectType> partition = await GetPartitionForPrefixAsync(keyspace, prefix, cancellationToken);
		await partition.EnsureHydrationAsync(cancellationToken);
	}

	public async IAsyncEnumerable<(DataStorePartitionKey, DataStorePartitionValue)> EnumerateRecordsAsync(KeyspaceId keyspace, string prefix, [EnumeratorCancellation] CancellationToken cancellationToken)
	{
		DataStoreTablePartition<TObjectType> partition = await GetPartitionForPrefixAsync(keyspace, prefix, cancellationToken);
		await foreach ((DataStorePartitionKey key, DataStorePartitionValue p) in partition.EnumerateRecordsAsync(cancellationToken))
		{
			yield return (key, p);
		}
	}

	public async Task DeleteAsync(KeyspaceId keyspace, string prefix, DataStorePartitionKey key, CancellationToken cancellationToken)
	{
		DataStoreTablePartition<TObjectType> partition = await GetPartitionForPrefixAsync(keyspace, prefix, cancellationToken);
		await partition.DeleteAsync(key, cancellationToken);
	}

	public async Task<long> DropKeyspaceAsync(KeyspaceId keyspace, CancellationToken cancellationToken)
	{
		if (_partitions.Remove(keyspace, out ConcurrentDictionary<string, DataStoreTablePartition<TObjectType>>? value))
		{
			long totalCountOfEntriesDropped = 0;
			foreach (DataStoreTablePartition<TObjectType> partition in value.Values)
			{
				long countOfEntries = await partition.DropPartitionAsync(cancellationToken);
				Interlocked.Add(ref totalCountOfEntriesDropped, countOfEntries);
			}

			await _tableLogService.DropKeyspaceAsync(keyspace, cancellationToken);

			return totalCountOfEntriesDropped;
		}

		return 0;
	}

	public async Task<long> DropPrefixAsync(KeyspaceId keyspace, string prefix, CancellationToken cancellationToken)
	{
		DataStoreTablePartition<TObjectType> partition = await GetPartitionForPrefixAsync(keyspace, prefix, cancellationToken);
		return await partition.DropPartitionAsync(cancellationToken);
	}

	public async Task RunGarbageCollectionAsync(KeyspaceId keyspace, string prefix, CancellationToken cancellationToken)
	{
		DataStoreTablePartition<TObjectType> partition = await GetPartitionForPrefixAsync(keyspace, prefix, cancellationToken);
		await partition.RunGarbageCollectionAsync(cancellationToken);
	}
}

