// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.DataStore;

public interface IDataStoreTableLog
{
	public Task InsertAsync(DataStorePartitionKey key, byte[] data, CancellationToken cancellationToken = default);
	public IAsyncEnumerable<(DataStoreSegmentId, DataStorePartitionKey, byte[])> GetEntriesAsync(DataStoreSegmentId? startSegment = null, CancellationToken cancellationToken = default);
	public Task FlushAsync(CancellationToken cancellationToken);
	public Task ReconcileLogEntriesAsync(CancellationToken cancellationToken);
	public Task DropPartitionAsync(CancellationToken cancellationToken);
	public Task<bool> RunGarbageCollectionAsync(CancellationToken cancellationToken);

	TableType Type { get; }
	KeyspaceId Keyspace { get; }
	string Prefix { get; }
}

public interface IDataStoreTableLogService
{
	public IDataStoreTableLog GetLogFor(KeyspaceId keyspaceId, TableType type, string prefix);
	public Task DropKeyspaceAsync(KeyspaceId keyspace, CancellationToken cancellationToken);
}

public class DataStoreTableLogServicePollingState
{
}

public class DataStoreTableLogService : PollingService<DataStoreTableLogServicePollingState>, IDataStoreTableLogService
{
	private readonly IServiceProvider _provider;
	private readonly IDataStoreObjectStore _objectStore;
	private readonly IOptionsMonitor<DataStoreSettings> _settings;
	private readonly ILogger<DataStoreTableLogService> _logger;

	private readonly ConcurrentDictionary<string, IDataStoreTableLog> _instances = new ConcurrentDictionary<string, IDataStoreTableLog>();

	public DataStoreTableLogService(IServiceProvider provider, IDataStoreObjectStore objectStore, IOptionsMonitor<DataStoreSettings> settings, ILogger<DataStoreTableLogService> logger) : base("DataStoreTableLogService", TimeSpan.FromMinutes(5), new DataStoreTableLogServicePollingState(), logger)
	{
		_provider = provider;
		_objectStore = objectStore;
		_settings = settings;
		_logger = logger;
	}

	public IDataStoreTableLog GetLogFor(KeyspaceId keyspaceId, TableType type, string prefix)
	{
		return _instances.GetOrAdd(GetKey(keyspaceId, type, prefix), key => ActivatorUtilities.CreateInstance<FilesystemDataStoreTableLog>(_provider, keyspaceId, type, prefix));
	}

	public Task DropKeyspaceAsync(KeyspaceId keyspace, CancellationToken cancellationToken)
	{
		List<string> keysToRemove = new List<string>();
		string keyspacePrefix = keyspace.ToString();
		foreach (string key in _instances.Keys)
		{
			// check if keyspace matches
			if (key.StartsWith(keyspacePrefix, StringComparison.OrdinalIgnoreCase))
			{
				keysToRemove.Add(key);
			}
		}

		foreach (string key in keysToRemove)
		{
			_instances.TryRemove(key, out _);
		}
		return Task.CompletedTask;
	}

	public IEnumerable<IDataStoreTableLog> GetAllLogs()
	{
		return _instances.Values;
	}

	protected override bool ShouldStartPolling()
	{
		return _settings.CurrentValue.Enabled;
	}

	public async Task ReconcilePrefixesAsync(CancellationToken cancellationToken)
	{
		// Loop across all types
		// Download any segment that might be missing
		IDataStoreTableService tableService = _provider.GetRequiredService<IDataStoreTableService>();
		//await Parallel.ForEachAsync(tableService.GetTableTypes(), cancellationToken, async (type, token) =>
		foreach (TableType type in tableService.GetTableTypes())
		{
			CancellationToken token = cancellationToken;
			_logger.LogInformation("Reconciling table type \"{TableType}\"", type);

			ConcurrentDictionary<KeyspaceId, HashSet<string>> processedPrefixes = new ConcurrentDictionary<KeyspaceId, HashSet<string>>();

			int countOfPrefixesProcessed = 0;

			void PrintProgress(int count)
			{
				if (count % 1000 == 0)
				{
					_logger.LogInformation("{Count} of prefixes processed in {Type}", count, type);
				}
			}

			// fetch prefixes from the object store
			// only hydrate the blob and cid tables as these always needs to be available for performance reasons, refs can be dynamically hydrated
			bool shouldHydrate = type == TableType.Blob || type == TableType.Cid;

			await Parallel.ForEachAsync(_objectStore.EnumeratePrefixesAsync(type, token), token, async (tuple, innertoken) =>
			{
				KeyspaceId keyspaceId = tuple.keyspace;
				string prefix = tuple.prefix;

				HashSet<string> prefixes = processedPrefixes.GetOrAdd(keyspaceId, id => new HashSet<string>());
				prefixes.Add(prefix);

				// this makes sure we have all the segments of each prefix available locally, this needs to be as fast as possible as it will block startup
				IDataStoreTableLog log = GetLogFor(keyspaceId, type, prefix);
				//_logger.LogInformation("Reconciling log entries for {Type} {Namespace} {Prefix}", type, ns, prefix);
				await log.ReconcileLogEntriesAsync(innertoken);

				//_logger.LogInformation("Hydrating log entries in {Type} {Namespace} {Prefix}", type, ns, prefix);
				if (shouldHydrate)
				{
					await HydratePrefixAsync(type, keyspaceId, prefix, innertoken);
				}

				//_logger.LogInformation("Finished processing log entries for {Type} {Namespace} {Prefix}", type, ns, prefix);
				int i = Interlocked.Increment(ref countOfPrefixesProcessed);
				PrintProgress(i);
			});

			// verify consistency on any prefix that only exists locally
			await Parallel.ForEachAsync(FilesystemDataStoreTableLog.EnumeratePrefixes(_settings, type, token), token,
				async (tuple, innertoken) =>
				{
					KeyspaceId keyspaceId = tuple.Item1;
					string prefix = tuple.Item2;

					bool prefixProcessed = false;
					if (processedPrefixes.TryGetValue(keyspaceId, out HashSet<string>? prefixes))
					{
						if (prefixes.TryGetValue(prefix, out _))
						{
							prefixProcessed = true;
						}
					}

					if (!prefixProcessed)
					{
						// this prefix only exists locally
						IDataStoreTableLog log = GetLogFor(keyspaceId, type, prefix);
						//_logger.LogInformation("Reconciling log entries for {Type} {Keyspace} {Prefix}", type, keyspaceId, prefix);
						await log.ReconcileLogEntriesAsync(innertoken);

						//_logger.LogInformation("Hydrating log entries in {Type} {Keyspace} {Prefix}", type, keyspaceId, prefix);
						if (shouldHydrate)
						{
							await HydratePrefixAsync(type, keyspaceId, prefix, innertoken);
						}

						//_logger.LogInformation("Finished processing log entries for {Type} {Keyspace} {Prefix}", type, keyspaceId, prefix);
					}

					int i = Interlocked.Increment(ref countOfPrefixesProcessed);
					PrintProgress(i);
				}
			);

			_logger.LogInformation("Done reconciling table type \"{TableType}\"", type);

		}
		//)
		;
	}

	private static string GetKey(KeyspaceId keyspace, TableType type, string prefix)
	{
		return $"{keyspace}.{type}.{prefix}";
	}

	protected override async Task OnStartingAsync(DataStoreTableLogServicePollingState state)
	{
		if (!_settings.CurrentValue.Enabled)
		{
			return;
		}

		_logger.LogInformation("Starting data store reconcile with object store");

		// Download any missing log entry on startup
		await ReconcilePrefixesAsync(CancellationToken.None);

		_logger.LogInformation("Finished data store reconcile with object store");

	}

	private async Task HydratePrefixAsync(TableType type, KeyspaceId keyspace, string prefix, CancellationToken cancellationToken)
	{
		IDataStoreTableService tableService = _provider.GetRequiredService<IDataStoreTableService>();
		await tableService.HydratePrefixAsync(type, keyspace, prefix, cancellationToken);
	}

	protected override async Task OnStopping(DataStoreTableLogServicePollingState state)
	{
		if (!_settings.CurrentValue.Enabled)
		{
			return;
		}

		foreach (IDataStoreTableLog tableLog in GetAllLogs())
		{
			/*if (cancellationToken.IsCancellationRequested)
			{
				break;
			}*/

			// flush all writes and set the log into a readonly state
			await tableLog.FlushAsync(CancellationToken.None);
				
			// TODO: Set table into readonly mode
			//await tableLog.StopAsync(cancellationToken);
		}
	}

	public override async Task<bool> OnPollAsync(DataStoreTableLogServicePollingState state, CancellationToken cancellationToken)
	{
		if (!_settings.CurrentValue.Enabled)
		{
			return false;
		}

		foreach (IDataStoreTableLog tableLog in GetAllLogs())
		{
			if (cancellationToken.IsCancellationRequested)
			{
				break;
			}

			await tableLog.FlushAsync(cancellationToken);
		}

		IDataStoreTableService tableService = _provider.GetRequiredService<IDataStoreTableService>();

		foreach (IDataStoreTableLog tableLog in GetAllLogs())
		{
			if (cancellationToken.IsCancellationRequested)
			{
				break;
			}

			await tableService.RunGarbageCollectionAsync(tableLog.Type, tableLog.Keyspace, tableLog.Prefix, cancellationToken);
		}

		return true;
	}
}