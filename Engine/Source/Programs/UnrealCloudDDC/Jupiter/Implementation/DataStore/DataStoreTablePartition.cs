// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace Jupiter.DataStore;

public readonly struct DataStorePartitionKey : IEquatable<DataStorePartitionKey>
{
	private static readonly ByteArrayComparer s_comparer = new ByteArrayComparer();
	public byte[] Bytes { get; }

	public DataStorePartitionKey(byte[] key)
	{
		Bytes = key;
	}

	public DataStorePartitionKey(ReadOnlyMemory<byte> mem)
	{
		Bytes = mem.ToArray();
	}

	public bool Equals(DataStorePartitionKey other)
	{
		return s_comparer.Equals(Bytes, other.Bytes);
	}

	public override bool Equals(object? obj)
	{
		return obj is DataStorePartitionKey other && Equals(other);
	}

	public override int GetHashCode()
	{
		return s_comparer.GetHashCode(Bytes);
	}

	public override string ToString()
	{
		return StringUtils.FormatAsHexString(Bytes);
	}

	public static bool operator ==(DataStorePartitionKey left, DataStorePartitionKey right)
	{
		return left.Equals(right);
	}

	public static bool operator !=(DataStorePartitionKey left, DataStorePartitionKey right)
	{
		return !(left == right);
	}
}

public record DataStorePartitionValue
{
	public DataStorePartitionValue(byte[] bytes, bool isFinalized, byte[]? sortKey = null)
	{
		sortKey ??= Array.Empty<byte>();

		{
			using Lock.Scope scope = _valueLock.EnterScope();
			Values ??= new SortedList<byte[], byte[]>(new ByteArrayComparer());
			Values[sortKey] = bytes;
		}
		IsFinalized = isFinalized;
	}

	public DataStorePartitionValue(byte[] bytes, bool isFinalized, DateTime creationTime, byte[]? sortKey = null) : this(bytes, isFinalized, sortKey)
	{
		CreationTime = creationTime;
	}

	public void AsObject<TObjectType>(out TObjectType record) where TObjectType : struct, IDataStoreTableObject<TObjectType>
	{
		if (!HasSingleValue)
		{
			throw new Exception("Required sort key");
		}

		lock (_valueLock)
		{
			TObjectType.Deserialize(Values.GetValueAtIndex(0), out record);
		}
	}

	public byte[]? GetRow(byte[] sortKey)
	{
		lock (_valueLock)
		{
			if (Values.TryGetValue(sortKey, out byte[]? value))
			{
				return value;
			}
		}

		return null;
	}

	public byte[] GetFirstRow()
	{
		lock (_valueLock)
		{
			return Values.GetValueAtIndex(0);
		}
	}

	private bool HasSingleValue => Values.Count == 1;

	public DateTime CreationTime { get; init; } = DateTime.UtcNow;
	private SortedList<byte[], byte[]> Values { get; init; }
	public bool IsFinalized { get; init; }

	private readonly Lock _valueLock = new Lock();
}

public interface IDataStoreTableObject<TObjectType>
{
	static abstract byte[] Serialize(TObjectType self);
	static abstract void Deserialize(byte[] payload, out TObjectType record);
	static abstract bool HasSortKey();
	static abstract byte[] GetSortKey(TObjectType self);
}

internal class DataStoreTablePartition<TObjectType> : IDisposable where TObjectType : struct, IDataStoreTableObject<TObjectType>
{
	private readonly ILogger<DataStoreTablePartition<TObjectType>> _logger;
	private readonly KeyspaceId _keyspaceId;
	private readonly TableType _type;
	private readonly string _prefix;
	private readonly Func<KeyspaceId, TObjectType, Task>? _onObjectWritten;
	private readonly ConcurrentDictionary<DataStorePartitionKey, DataStorePartitionValue> _values = new ConcurrentDictionary<DataStorePartitionKey, DataStorePartitionValue>(Environment.ProcessorCount, capacity: 4096);
	private readonly IDataStoreTableLog _tableLog;
	private DataStoreSegmentId? _lastHydratedSegment;
	private bool _isHydrated;
	private readonly ManualResetEventSlim _hydrationInProgress= new ManualResetEventSlim(false);

	public DataStoreTablePartition(IServiceProvider provider, KeyspaceId keyspaceId, TableType type, string prefix, Func<KeyspaceId, TObjectType, Task>? onObjectWritten = null)
	{
		_logger = provider.GetRequiredService<ILogger<DataStoreTablePartition<TObjectType>>>();
		_keyspaceId = keyspaceId;
		_type = type;
		_prefix = prefix;
		_onObjectWritten = onObjectWritten;
		_tableLog = provider.GetRequiredService<IDataStoreTableLogService>().GetLogFor(keyspaceId, type, prefix);
	}

	public bool IsEmpty => _values.IsEmpty;

	public async Task EnsureHydrationAsync(CancellationToken token)
	{
		// ReSharper disable once InconsistentlySynchronizedField
		if (_isHydrated)
		{
			return;
		}

		if (_hydrationInProgress.IsSet)
		{
			while (_hydrationInProgress.IsSet)
			{
				_hydrationInProgress.Wait(TimeSpan.FromMilliseconds(10), token);
				await Task.Yield();
			}
		}
		else
		{
			_hydrationInProgress.Set();
		}

		if (!_isHydrated)
		{
			try
			{
				await HydratePartitionAsync(token);
			}
			finally
			{
				_isHydrated = true;
				_hydrationInProgress.Reset();
			}
		}
	}

	private async Task RehydratePartitionAsync(CancellationToken token)
	{
		_isHydrated = false;
		_values.Clear();
		_lastHydratedSegment = null;
		await EnsureHydrationAsync(token);
	}

	private async Task HydratePartitionAsync(CancellationToken token)
	{
		try
		{
			await foreach ((DataStoreSegmentId segmentId, DataStorePartitionKey key, byte[] b) in _tableLog.GetEntriesAsync(startSegment: _lastHydratedSegment, cancellationToken: token))
			{
				DateTimeOffset time = segmentId.AsTimeOffset();
				if (b.Length == 0)
				{
					// tombstone found - this entry was deleted
					_values.TryRemove(key, out _);
				}
				else
				{
					_values.AddOrUpdate(key, new DataStorePartitionValue(b, isFinalized: true, time.DateTime), (_, _) => new DataStorePartitionValue(b, isFinalized: true, time.DateTime));
				}

				_lastHydratedSegment = segmentId;
			}

			_isHydrated = true;
		}
		catch (InvalidStreamException e)
		{
			_logger.LogError(e, "Invalid stream when parsing table log {Type} {Keyspace} {Prefix}", _type, _keyspaceId, _prefix);
		}
	}

	public async Task InsertAsync(DataStorePartitionKey key, TObjectType record, bool isFinalized, CancellationToken cancellationToken)
	{
		await EnsureHydrationAsync(cancellationToken);

		bool hasSortKey = TObjectType.HasSortKey();
		byte[] payload = TObjectType.Serialize(record);
		// only commit to the log if this record is finalized
		if (isFinalized)
		{
			bool writeToLog = true;
			bool forceWriteToLog = false;
			if (_values.TryGetValue(key, out DataStorePartitionValue? value))
			{
				byte[] oldPayload;
				if (hasSortKey)
				{
					byte[] sortKey = TObjectType.GetSortKey(record);
					byte[]? p = value.GetRow(sortKey);

					if (p != null)
					{
						oldPayload = p;
					}
					else
					{
						// if we had no payload then we should always refresh
						forceWriteToLog = true;
						oldPayload = null!;
					}
				}
				else
				{
					oldPayload = value.GetFirstRow();
				}

				// do not update the log entry if the object is the same, and we have recently written it and it was already finalized
				if (!forceWriteToLog && !ShouldRefresh(value.CreationTime) && ByteArrayComparer.Default.Equals(payload, oldPayload) && value.IsFinalized)
				{
					writeToLog = false;
				}
			}

			if (writeToLog)
			{
				Task? onObjectWrittenTask = _onObjectWritten?.Invoke(_keyspaceId, record);
				await _tableLog.InsertAsync(key, payload, cancellationToken: cancellationToken);
				if (onObjectWrittenTask != null)
				{
					await onObjectWrittenTask;
				}
			}
		}

		_values.AddOrUpdate(key, new DataStorePartitionValue(payload, isFinalized), (_, _) => new DataStorePartitionValue(payload, isFinalized));
	}

	private async Task RefreshAsync(DataStorePartitionKey key, DataStorePartitionValue partitionValue, bool isFinalized, CancellationToken cancellationToken)
	{
		await EnsureHydrationAsync(cancellationToken);

		if (!isFinalized)
		{
			throw new Exception("Refreshing non-finalized records is not supported");
		}

		// TODO: Refresh all sort key permutations of the value
		byte[] payload = partitionValue.GetFirstRow();
		bool writeToLog = true;
		if (_values.TryGetValue(key, out DataStorePartitionValue? value))
		{
			// do not update the log entry if the object is the same, and we have recently written it and it was already finalized
			if (!ShouldRefresh(value.CreationTime) && ByteArrayComparer.Default.Equals(payload, value.GetFirstRow()) && value.IsFinalized)
			{
				writeToLog = false;
			}
		}

		if (writeToLog)
		{
			Task? onObjectWrittenTask = null;
			if (_onObjectWritten != null)
			{
				TObjectType.Deserialize(payload, out TObjectType record);
				onObjectWrittenTask = _onObjectWritten.Invoke(_keyspaceId, record);
			}
			await _tableLog.InsertAsync(key, payload, cancellationToken: cancellationToken);
			if (onObjectWrittenTask != null)
			{
				await onObjectWrittenTask;
			}
		}

		_values.AddOrUpdate(key, new DataStorePartitionValue(payload, isFinalized), (_, _) => new DataStorePartitionValue(payload, isFinalized));
	}

	private static bool ShouldRefresh(DateTime creationTime)
	{
		if (creationTime < DateTime.Now.AddDays(-7))
		{
			// if the record is more than 7 days old we touch it to make sure its kept alive
			return true;
		}

		return false;
	}

	public async Task EnsureRefreshAsync(DataStorePartitionKey key, CancellationToken cancellationToken)
	{
		await EnsureHydrationAsync(cancellationToken);
		if (_values.TryGetValue(key, out DataStorePartitionValue? value))
		{
			await RefreshAsync(key, value, isFinalized: true, cancellationToken);
		}
	}

	public async Task<DataStorePartitionValue?> FindAsync(DataStorePartitionKey key, CancellationToken cancellationToken)
	{
		await EnsureHydrationAsync(cancellationToken);
		if (_values.TryGetValue(key, out DataStorePartitionValue? value))
		{
			// object was requested, refresh it if we last read it a while ago to make sure it stays alive
			bool shouldRefresh = ShouldRefresh(value.CreationTime);
			
			try
			{
				if (shouldRefresh)
				{
					await RefreshAsync(key, value, isFinalized: true, cancellationToken);
				}

				return value;
			}
			catch (Exception)
			{
				// invalid object, ignore it
				return null;
			}
		}

		return null;
	}

	public async Task DeleteAsync(DataStorePartitionKey key, CancellationToken cancellationToken)
	{
		await EnsureHydrationAsync(cancellationToken);
		// insert the delete record into the log
		await _tableLog.InsertAsync(key, Array.Empty<byte>(), cancellationToken: cancellationToken);

		_values.TryRemove(key, out _);
	}

	public async Task FlushAsync(CancellationToken cancellationToken)
	{
		await _tableLog.FlushAsync(cancellationToken);
	}

	public async IAsyncEnumerable<(DataStorePartitionKey, DataStorePartitionValue)> EnumerateRecordsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
	{
		await EnsureHydrationAsync(cancellationToken);

		foreach (KeyValuePair<DataStorePartitionKey, DataStorePartitionValue> pair in _values)
		{
			DataStorePartitionValue value = pair.Value;
			DataStorePartitionKey key = pair.Key;

			yield return (key, value);
		}
	}

	public async Task<long> DropPartitionAsync(CancellationToken cancellationToken)
	{
		await _tableLog.DropPartitionAsync(cancellationToken);
		long countOfEntries = _values.Keys.Count;
		_values.Clear();
		return countOfEntries;
	}

	public async Task RunGarbageCollectionAsync(CancellationToken cancellationToken)
	{
		bool gcHappened = await _tableLog.RunGarbageCollectionAsync(cancellationToken);

		if (gcHappened)
		{
			await RehydratePartitionAsync(cancellationToken);
		}
	}

	public void Dispose()
	{
		_hydrationInProgress.Dispose();
	}
}