// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Serialization;
using Jupiter.Common;
using Jupiter.Implementation;
using Jupiter.Implementation.Blob;
using Microsoft.Extensions.DependencyInjection;

namespace Jupiter.DataStore;

public enum TableType
{
	Blob, Ref, Cid
}

public interface IDataStoreTableService
{
	DataStoreTable<T> GetTableByType<T>() where T : struct, IDataStoreTableObject<T>;
	IEnumerable<TableType> GetTableTypes();

	Task HydratePrefixAsync(TableType type, KeyspaceId keyspace, string prefix, CancellationToken cancellationToken);
	Task RunGarbageCollectionAsync(TableType type, KeyspaceId keyspace, string prefix, CancellationToken cancellationToken);
}

public class DataStoreTableService : IDataStoreTableService
{
	private readonly IServiceProvider _provider;
	private readonly ConcurrentDictionary<TableType, object> _tables = new ConcurrentDictionary<TableType, object>();
	private readonly ConcurrentDictionary<string, TableType> _typeToName = new ConcurrentDictionary<string, TableType>();

	private readonly DataStoreTable<DataStoreBlobRecord> _blobTable;
	private IReferenceResolver? _referenceResolver = null;
	private IBlobService? _blobService = null;

	public DataStoreTableService(IServiceProvider provider)
	{
		_provider = provider;

		// define the schema
		_blobTable = AddTableOfType<DataStoreBlobRecord>(TableType.Blob);
		AddTableOfType<DataStoreCidRecord>(TableType.Cid);
		AddTableOfType<DataStoreRefRecord>(TableType.Ref, OnRefWrittenAsync);
	}

	private async Task OnRefWrittenAsync(KeyspaceId keyspace, DataStoreRefRecord record)
	{
		NamespaceId ns = keyspace.ToNamespaceId();
		CbObject cb;
		if (record.Blob.Length == 0)
		{
			_blobService ??= _provider.GetRequiredService<IBlobService>();
			await using BlobContents blobContents = await _blobService.GetObjectAsync(ns, BlobId.FromIoHash(record.BlobHash));

			byte[] b = await blobContents.Stream.ReadAllBytesAsync();
			cb = new CbObject(b);
		}
		else
		{
			cb = new CbObject(record.Blob);
		}

		_referenceResolver ??= _provider.GetRequiredService<IReferenceResolver>();

		IAsyncEnumerable<BlobId> referencedBlobsEnumerable = _referenceResolver.GetReferencedBlobsAsync(ns, cb);
		await foreach (BlobId blobId in referencedBlobsEnumerable)
		{
			// make sure all referenced blobs are updated
			await _blobTable.EnsureRefreshAsync(keyspace, DataStoreBlobIndex.CalculatePrefix(blobId), DataStoreBlobRecord.GetPartitionKey(blobId), CancellationToken.None);
		}
	}

	public async Task HydratePrefixAsync(TableType type, KeyspaceId keyspace, string prefix, CancellationToken cancellationToken)
	{
		switch (type)
		{
			case TableType.Ref:
				await HydrateTableByTypeAsync<DataStoreRefRecord>(keyspace, prefix, cancellationToken);
				break;
			case TableType.Blob:
				await HydrateTableByTypeAsync<DataStoreBlobRecord>(keyspace, prefix, cancellationToken);
				break;
			case TableType.Cid:
				await HydrateTableByTypeAsync<DataStoreCidRecord>(keyspace, prefix, cancellationToken);
				break;
			default:
				throw new NotImplementedException($"Unknown table type {type}");
		}
	}

	private DataStoreTable<T> AddTableOfType<T>(TableType key, Func<KeyspaceId, T, Task>? onRecordUpdated = null) where T : struct, IDataStoreTableObject<T>
	{
		DataStoreTable<T> table = new DataStoreTable<T>(_provider, key, onRecordUpdated);
		string typeName = typeof(T).Name;
		if (!_tables.TryAdd(key, table))
		{
			throw new Exception($"Table with key {key} already added");
		}

		if (!_typeToName.TryAdd(typeName, key))
		{
			throw new Exception($"Table with type {typeName} already added");
		}

		return table;
	}

	private Task HydrateTableByTypeAsync<T>(KeyspaceId keyspace, string prefix, CancellationToken cancellationToken) where T : struct, IDataStoreTableObject<T>
	{
		DataStoreTable<T> table = GetTableByType<T>();
		return table.HydratePrefixAsync(keyspace, prefix, cancellationToken);
	}

	public DataStoreTable<T> GetTableByType<T>() where T : struct, IDataStoreTableObject<T>
	{
		string typeName = typeof(T).Name;
		if (_typeToName.TryGetValue(typeName, out TableType tableName))
		{
			if (_tables.TryGetValue(tableName, out object? o))
			{
				if (o is DataStoreTable<T> table)
				{
					return table;
				}
			
				throw new InvalidCastException($"Table {tableName} not convertible to type {typeName}");
			}
		}

		throw new Exception($"Failed to find table of type {typeName}");
	}

	public async Task RunGarbageCollectionAsync(TableType type, KeyspaceId keyspace, string prefix, CancellationToken cancellationToken)
	{
		Task GcTableByTypeAsync<T>(KeyspaceId k, string p, CancellationToken ct) where T : struct, IDataStoreTableObject<T>
		{
			DataStoreTable<T> table = GetTableByType<T>();
			return table.RunGarbageCollectionAsync(k, p, ct);
		}

		switch (type)
		{
			case TableType.Ref:
				await GcTableByTypeAsync<DataStoreRefRecord>(keyspace, prefix, cancellationToken);
				break;
			case TableType.Blob:
				await GcTableByTypeAsync<DataStoreBlobRecord>(keyspace, prefix, cancellationToken);
				break;
			case TableType.Cid:
				await GcTableByTypeAsync<DataStoreCidRecord>(keyspace, prefix, cancellationToken);
				break;
			default:
				throw new NotImplementedException($"Unknown table type {type}");
		}
	}

	public IEnumerable<TableType> GetTableTypes()
	{
		return _tables.Keys;
	}
}