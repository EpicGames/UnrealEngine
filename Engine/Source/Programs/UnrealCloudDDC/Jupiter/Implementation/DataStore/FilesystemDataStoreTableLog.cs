// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Implementation;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.DataStore;

internal class FilesystemDataStoreTableLog : IDataStoreTableLog, IDisposable
{
	private readonly IOptionsMonitor<DataStoreSettings> _settings;
	private readonly IOptionsMonitor<GCSettings> _gcSettings;
	private readonly IDataStoreObjectStore _objectStore;
	private readonly ILogger<FilesystemDataStoreTableLog> _logger;
	private readonly KeyspaceId _keyspaceId;
	private readonly TableType _type;
	private readonly string _prefix;
	private readonly ReaderWriterLockSlim _rwLock;

	//private CbObjectId _currentGeneration;
	private DataStoreSegmentId _currentSegmentId;
	private FileStream? _currentLogWriter;

	internal const int MaxObjectSize = 65536;
		
	public FilesystemDataStoreTableLog(IOptionsMonitor<DataStoreSettings> settings, IOptionsMonitor<GCSettings> gcSettings, IDataStoreObjectStore objectStore, ILogger<FilesystemDataStoreTableLog> logger, KeyspaceId keyspaceId, TableType type, string prefix)
	{
		_settings = settings;
		_gcSettings = gcSettings;
		_objectStore = objectStore;
		_logger = logger;
		_keyspaceId = keyspaceId;
		_type = type;
		_prefix = prefix;
		_currentSegmentId = DataStoreSegmentId.NewId();
		_rwLock = new ReaderWriterLockSlim();
	}

	public void Dispose()
	{
		_rwLock.Dispose();
		_currentLogWriter?.Dispose();
	}

	private FileStream GetLogFileStream()
	{
		if (_currentLogWriter != null)
		{
			return _currentLogWriter;
		}

		FileInfo currentLogFile = GetFilePath(_currentSegmentId);

		if (!currentLogFile.Directory!.Exists)
		{
			Directory.CreateDirectory(currentLogFile.Directory.FullName);
		}
		_currentLogWriter = currentLogFile.Open(FileMode.OpenOrCreate, FileAccess.Write, FileShare.Read);
		CbObjectStreamHeader header = new CbObjectStreamHeader();
		header.WriteToStream(_currentLogWriter);
		return _currentLogWriter;
	}
	public Task InsertAsync(DataStorePartitionKey key, byte[] data, CancellationToken cancellationToken = default)
	{
		if (data.Length > MaxObjectSize)
		{
			throw new NotImplementedException($"Objects needs to be smaller then {MaxObjectSize}");
		}

		try
		{
			_rwLock.EnterWriteLock();

			FileStream logWriter = GetLogFileStream();

			WriteObjectEntry(logWriter, key, data);
			// TODO: It is slow to flush on every insert but if we do not there is risk of data loss...
#pragma warning disable CA1849
			logWriter.Flush();
#pragma warning restore CA1849
			//IndexEntry index = new IndexEntry(key, (uint)data.Length, offset);

			// TODO: Write index to index file
		}
		finally
		{
			_rwLock.ExitWriteLock();
		}

		return Task.CompletedTask;
	}

	private static void WriteObjectEntry(Stream s, DataStorePartitionKey key, byte[] data)
	{
		using BinaryWriter bw = new BinaryWriter(s, Encoding.ASCII, leaveOpen: true);
		if (key.Bytes.Length != 20)
		{
			throw new Exception("Keys are assumed to be 20 bytes");
		}
		bw.Write(key.Bytes);
		bw.Write(data.Length);
		bw.Write(data);
	}

	/*[StructLayout(LayoutKind.Explicit)]
	internal record struct IndexEntry
	{
		// 32 byte record
		// 20 byte key
		[FieldOffset(0)] private uint _keyA;
		[FieldOffset(4)] private ulong _keyB;
		[FieldOffset(12)] private ulong _keyC;
		// 4 byte length
		[FieldOffset(20)] private uint _length;
		// 8 byte offset
		[FieldOffset(24)] private ulong _offset;

		public IndexEntry(DataStorePartitionKey key, uint length, ulong offset)
		{
			byte[] keyBytes = key.Bytes;
			Debug.Assert(keyBytes.Length == 20);
			_keyA = BitConverter.ToUInt32(keyBytes.AsSpan(0, 4));
			_keyB = BitConverter.ToUInt64(keyBytes.AsSpan(4, 8));
			_keyC = BitConverter.ToUInt64(keyBytes.AsSpan(12, 8));
			_length = length;
			_offset = offset;
		}

		public uint Length => _length;

		public ulong Offset => _offset;
	}*/

	private static string GetRootDir(IOptionsMonitor<DataStoreSettings> settings)
	{
		return PathUtil.ResolvePath(settings.CurrentValue.RootDir);
	}

	private DirectoryInfo GetStateDirectory()
	{
		return new DirectoryInfo(Path.Combine(GetRootDir(_settings), _type.ToString(), _keyspaceId.ToString(), _prefix));
	}

	private FileInfo GetFilePath(DataStoreSegmentId segment)
	{
		FileInfo fi = new FileInfo(Path.Combine(GetStateDirectory().FullName, segment.ToString() + ".bulk"));
		return fi;
	}

	/*private FileInfo GetFilePathIndex(CbObjectId generation)
	{
		string path = Path.ChangeExtension(GetStateDirectory().FullName, generation.ToString() + ".index");
		return new FileInfo(path);
	}*/

	public async IAsyncEnumerable<(DataStoreSegmentId, DataStorePartitionKey, byte[])> GetEntriesAsync(DataStoreSegmentId? startSegment = null, [EnumeratorCancellation] CancellationToken cancellationToken = default)
	{	
		DirectoryInfo di = GetStateDirectory();

		if (!di.Exists)
		{
			yield break;
		}

		Dictionary<DataStoreSegmentId, string> knownGenerations = new Dictionary<DataStoreSegmentId, string>();
			
		foreach (string fullpath in Directory.EnumerateFiles(di.FullName,"*.bulk"))
		{
			string filename = Path.GetFileName(fullpath);
			string s = filename.Substring(0, filename.Length - 5);
			DataStoreSegmentId segmentId = DataStoreSegmentId.Parse(s);

			knownGenerations[segmentId] = fullpath;
		}

		bool foundStartGeneration = false;
		foreach (KeyValuePair<DataStoreSegmentId, string> pair in knownGenerations.OrderByDescending(pair => pair.Key))
		{
			if (startSegment != null && !foundStartGeneration && !pair.Key.Equals(startSegment))
			{
				// Ignore each generation until we find the selected start generation
				continue;
			}

			foundStartGeneration = true;

			DataStoreSegmentId segmentId = pair.Key;

			const int BufferSize = 4096;
			FileStream? fs;

			try
			{
#pragma warning disable CA2000
				fs = new FileStream(pair.Value, FileMode.Open, FileAccess.Read, FileShare.Read, BufferSize, FileOptions.Asynchronous | FileOptions.SequentialScan);
#pragma warning restore CA2000
			}
			catch (IOException)
			{
				// if we can not access the file we ignore it as its being written by another instance
				continue;
			}

			await using DataStoreObjectStreamReader streamReader = new DataStoreObjectStreamReader(fs, FilesystemDataStoreTableLog.MaxObjectSize * 2);

			if (cancellationToken.IsCancellationRequested)
			{
				yield break;
			}

			await foreach ((DataStorePartitionKey key, byte[] b) in streamReader.GetObjectBuffersAsync(cancellationToken))
			{
				yield return (segmentId, key, b);
			}

			await fs.DisposeAsync();
		}
	}

	public async Task FlushAsync(CancellationToken cancellationToken)
	{
		FileInfo oldLogFilePath = GetFilePath(_currentSegmentId);

		try
		{
			_rwLock.EnterWriteLock();

			if (_currentLogWriter == null || _currentLogWriter.Position == 0L)
			{
				// nothing written to this log yet so no need to flush it
				return;
			}

			_currentSegmentId = DataStoreSegmentId.NewId();

			await _currentLogWriter.DisposeAsync();
			_currentLogWriter = null;
		}
		finally
		{
			_rwLock.ExitWriteLock();
		}

		// upload to object store for backup
		string remotePath = $"Objects/{_type}/{_keyspaceId}/{_prefix}/{oldLogFilePath.Name}";
		await _objectStore.PutFileAsync(_keyspaceId, oldLogFilePath, remotePath, cancellationToken);
	}

	public async Task ReconcileLogEntriesAsync(CancellationToken cancellationToken)
	{
		string remotePath = $"Objects/{_type}/{_keyspaceId}/{_prefix}/";
		Dictionary<string, long> knownRemoteFiles = new Dictionary<string, long>(StringComparer.OrdinalIgnoreCase);
		await foreach (DataStoreFile file in _objectStore.EnumerateFilesAsync(_keyspaceId, remotePath, cancellationToken))
		{
			knownRemoteFiles.Add(file.Path, file.Size);
		}

		List<FileInfo> filesToUpload = new List<FileInfo>();
		List<FileInfo> filesToDownload = new List<FileInfo>();

		DirectoryInfo stateDirectory = GetStateDirectory();
		foreach (KeyValuePair<string, long> knownRemoteFile in knownRemoteFiles)
		{
			string remoteFileName = knownRemoteFile.Key;
			long size = knownRemoteFile.Value;

			FileInfo fi = new FileInfo(Path.Combine(stateDirectory.FullName, remoteFileName));
			if (!fi.Exists || fi.Length != size)
			{
				filesToDownload.Add(fi);
			}
		}

		if (stateDirectory.Exists)
		{
			foreach (FileInfo localFile in stateDirectory.EnumerateFiles())
			{
				// skip uploading the log file we are currently writing to
				if (localFile.FullName == GetFilePath(_currentSegmentId).FullName)
				{
					continue;
				}

				// skip empty files
				if (localFile.Length == 0)
				{
					continue;
				}

				// do not upload files which are up for GC as they may have been removed from object store on purpose
				if (!IsLocalFileAlive(localFile.LastWriteTimeUtc))
				{
					continue;
				}

				bool shouldUpload = false;
				if (knownRemoteFiles.TryGetValue(localFile.Name, out long fileSize))
				{
					// check if the known file size matches what we have, if it does not then schedule it for upload
					FileInfo fi = new FileInfo(Path.Combine(stateDirectory.FullName, localFile.Name));

					if (fi.Length != fileSize)
					{
						shouldUpload = true;
					}
				}
				else
				{
					// file does not exist on object store
					shouldUpload = true;
				}

				if (shouldUpload)
				{
					filesToUpload.Add(localFile);
				}
			}
		}

		if (filesToDownload.Count != 0 || filesToUpload.Count != 0)
		{
			_logger.LogInformation("Found {CountMissing} missing segments and {CountLocalOnly} local segments for {Prefix} in {Keyspace} of type {Type}. Reconciling differences.", filesToDownload.Count, filesToUpload.Count, _prefix, _keyspaceId, _type);
		}
		Task uploadTask = Parallel.ForEachAsync(filesToUpload, cancellationToken, async (fi, token) =>
		{
			await _objectStore.PutFileAsync(_keyspaceId, fi, remotePath + fi.Name, token);
		});

		Task downloadTask = Parallel.ForEachAsync(filesToDownload, cancellationToken, async (fi, token) =>
		{
			if (!await _objectStore.GetFileAsync(_keyspaceId, remotePath + fi.Name, fi, token))
			{
				throw new Exception($"Failed to download missing remote file {fi.Name}");
			}
		});

		await Task.WhenAll(uploadTask, downloadTask);
	}

	public async Task DropPartitionAsync(CancellationToken cancellationToken)
	{
		// flush any pending writes and stop the writer to drop the current in flight writes
		_currentLogWriter?.Close();

		// drop the local state
		DirectoryInfo stateDirectory = GetStateDirectory();
		stateDirectory.Delete(true);

		// drop the state from the object store
		string remotePath = $"Objects/{_type}/{_keyspaceId}/{_prefix}/";
		await _objectStore.DeleteFilesAsync(_keyspaceId, remotePath, cancellationToken);
	}

	public async Task<bool> RunGarbageCollectionAsync(CancellationToken cancellationToken)
	{
		bool gcHappened = false;
		{
			// check the local storage for files to GC
			DirectoryInfo stateDirectory = GetStateDirectory();

			if (stateDirectory.Exists)
			{
				foreach (FileInfo localFile in stateDirectory.EnumerateFiles())
				{
					if (IsLocalFileAlive(localFile.LastWriteTimeUtc))
					{
						continue;
					}

					localFile.Delete();
					gcHappened = true;
				}
			}
		}

		{
			// check the object store for files to GC
			string remotePath = $"Objects/{_type}/{_keyspaceId}/{_prefix}/";

			// check out of date files in the object store and trigger clean up of them
			await foreach (DataStoreFile file in _objectStore.EnumerateFilesAsync(_keyspaceId, remotePath, cancellationToken))
			{
				if (IsLocalFileAlive(file.LastModifiedUtc))
				{
					continue;
				}

				await _objectStore.DeleteFilesAsync(_keyspaceId, remotePath + file.Path, cancellationToken);
				gcHappened = true;
			}
		}

		return gcHappened;
	}

	public TableType Type => _type;

	public KeyspaceId Keyspace => _keyspaceId;
	public string Prefix => _prefix;

	private bool IsLocalFileAlive(DateTime lastAccessTimeUtc)
	{
		if (lastAccessTimeUtc > DateTime.UtcNow.AddDays(-1 * _gcSettings.CurrentValue.LastAccessCutoff.TotalDays))
		{
			return true;
		}

		return false;
	}

	public static IEnumerable<KeyspaceId> GetKeyspaces(IOptionsMonitor<DataStoreSettings> settings, TableType type)
	{
		DirectoryInfo di = new DirectoryInfo(Path.Combine(GetRootDir(settings), type.ToString()));

		if (!di.Exists)
		{
			yield break;
		}

		foreach (DirectoryInfo d in di.EnumerateDirectories())
		{
			yield return new KeyspaceId(d.Name);
		}
	}

	public static IEnumerable<(KeyspaceId, string)> EnumeratePrefixes(IOptionsMonitor<DataStoreSettings> settings, TableType type, CancellationToken cancellationToken)
	{
		foreach (KeyspaceId ns in GetKeyspaces(settings, type))
		{
			if (cancellationToken.IsCancellationRequested)
			{
				yield break;
			}

			foreach (DirectoryInfo prefixDirectory in new DirectoryInfo(Path.Combine(GetRootDir(settings), type.ToString(), ns.ToString())).EnumerateDirectories())
			{
				if (cancellationToken.IsCancellationRequested)
				{
					yield break;
				}

				yield return (ns, prefixDirectory.Name);
			}
		}
	}
}