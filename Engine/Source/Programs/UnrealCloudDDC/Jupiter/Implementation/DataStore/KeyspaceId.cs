// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using Jupiter.Common;

namespace Jupiter.DataStore;

public readonly struct KeyspaceId : IEquatable<KeyspaceId>
{
	/// <summary>
	/// The text representing this id
	/// </summary>
	private StringId Text { get; }

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="text">Unique id for the namespace</param>
	public KeyspaceId(string text)
		: this(new Utf8String(text))
	{
	}

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="text">Unique id for the namespace</param>
	public KeyspaceId(Utf8String text)
	{
		Text = new StringId(text);
	}

	/// <inheritdoc/>
	public override bool Equals(object? obj) => obj is KeyspaceId id && Text.Equals(id.Text);

	/// <inheritdoc/>
	public override int GetHashCode() => Text.GetHashCode();

	/// <inheritdoc/>
	public bool Equals(KeyspaceId other) => Text.Equals(other.Text);

	/// <inheritdoc/>
	public override string ToString() => Text.ToString();

	/// <inheritdoc cref="StringId.op_Equality"/>
	public static bool operator ==(KeyspaceId left, KeyspaceId right) => left.Text == right.Text;

	/// <inheritdoc cref="StringId.op_Inequality"/>
	public static bool operator !=(KeyspaceId left, KeyspaceId right) => left.Text != right.Text;
	public static KeyspaceId FromNamespace(NamespaceId ns)
	{
		return new KeyspaceId(ns.ToString());
	}

	public static KeyspaceId FromStoragePool(string storagePool)
	{
		if (string.IsNullOrEmpty(storagePool))
		{
			return new KeyspaceId("default");
		}
		return new KeyspaceId(storagePool);
	}

	public NamespaceId ToNamespaceId()
	{
		return new NamespaceId(Text.ToString());
	}
}