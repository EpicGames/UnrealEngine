// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public interface IBlobCleanup
	{
		bool ShouldRun();
		Task<ulong> CleanupAsync(CancellationToken none);
	}

	public class BlobCleanupState
	{
		public List<IBlobCleanup> BlobCleanups { get; } = new List<IBlobCleanup>();
	}

	public class BlobCleanupService : PollingService<BlobCleanupState>
	{
		private readonly IOptionsMonitor<GCSettings> _settings;
		private volatile bool _alreadyPolling;
		private readonly ILogger _logger;

		protected override bool ShouldStartPolling()
		{
			return _settings.CurrentValue.BlobCleanupServiceEnabled;
		}

		public BlobCleanupService(IServiceProvider provider, IOptionsMonitor<GCSettings> settings, ILogger<BlobCleanupService> logger) : base(serviceName: nameof(BlobCleanupService), settings.CurrentValue.BlobCleanupPollFrequency, new BlobCleanupState(), logger)
		{
			_settings = settings;
			_logger = logger;

			if (settings.CurrentValue.CleanOldBlobs)
			{
				OrphanBlobCleanupRefs orphanBlobCleanupRefs = provider.GetService<OrphanBlobCleanupRefs>()!;
				RegisterCleanup(orphanBlobCleanupRefs);
			}

			if (settings.CurrentValue.RunFilesystemCleanup)
			{
				FileSystemStore? fileSystemStore = provider.GetService<FileSystemStore>();
				if (fileSystemStore != null)
				{
					RegisterCleanup(fileSystemStore);
				}
			}
		}

		public void RegisterCleanup(IBlobCleanup cleanup)
		{
			State.BlobCleanups.Add(cleanup);
		}

		public override async Task<bool> OnPollAsync(BlobCleanupState state, CancellationToken cancellationToken)
		{
			if (_alreadyPolling)
			{
				return false;
			}

			_alreadyPolling = true;
			try
			{
				await CleanupAsync(state, cancellationToken);
				return true;
			}
			finally
			{
				_alreadyPolling = false;
			}
		}

		public async Task CleanupAsync(BlobCleanupState state, CancellationToken cancellationToken)
		{
			foreach (IBlobCleanup blobCleanup in state.BlobCleanups)
			{
				if (!blobCleanup.ShouldRun())
				{
					continue;
				}

				string type = blobCleanup.GetType().ToString();
				_logger.LogInformation("Blob cleanup running for {BlobCleanup}", type);

				_logger.LogInformation("Attempting to run Blob Cleanup {BlobCleanup}. ", type);
				try
				{
					ulong countOfBlobsCleaned = await blobCleanup.CleanupAsync(cancellationToken);
					_logger.LogInformation("Ran blob cleanup {BlobCleanup}. Deleted {CountBlobRecords}", type, countOfBlobsCleaned);
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Exception running Blob Cleanup {BlobCleanup} .", type);
				}
			}
		}
	}
}

