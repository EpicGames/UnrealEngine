// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.Metrics;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Implementation.Blob;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public interface IRefCleanup
	{
		Task<int> Cleanup(CancellationToken cancellationToken);
	}

	public class RefLastAccessCleanup : IRefCleanup
	{
		private readonly IOptionsMonitor<GCSettings> _settings;
		private readonly IReferencesStore _referencesStore;
		private readonly IRefService _objectService;
		private readonly IBlobIndex _blobIndex;
		private readonly IRefService _refService;
		private readonly IReplicationLog _replicationLog;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;
		private readonly Gauge<long> _cleanupRefsConsidered;
		private readonly IOptionsMonitor<UnrealCloudDDCSettings> _cloudDDCSettings;

		public RefLastAccessCleanup(IOptionsMonitor<GCSettings> settings, IOptionsMonitor<UnrealCloudDDCSettings> cloudDDCSettings,
			IReferencesStore referencesStore, IRefService objectService, IBlobIndex blobIndex, IRefService refService,
			IReplicationLog replicationLog, INamespacePolicyResolver namespacePolicyResolver, Meter meter, Tracer tracer, ILogger<RefLastAccessCleanup> logger)
		{
			_settings = settings;
			_cloudDDCSettings = cloudDDCSettings;
			_referencesStore = referencesStore;
			_objectService = objectService;
			_blobIndex = blobIndex;
			_refService = refService;
			_replicationLog = replicationLog;
			_namespacePolicyResolver = namespacePolicyResolver;
			_tracer = tracer;
			_logger = logger;

			_cleanupRefsConsidered = meter.CreateGauge<long>("refs.considered");
		}

		private bool ShouldGCNamespace(NamespaceId ns)
		{
			if (ns == INamespacePolicyResolver.JupiterInternalNamespace)
			{
				// do not apply our cleanup policies to the internal namespace
				return false;
			}

			try
			{
				NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);

				NamespacePolicy.StoragePoolGCMethod? gcMethod = policy.GcMethod;
				if (gcMethod == null)
				{
					gcMethod = _settings.CurrentValue.DefaultGCPolicy;
				}

				if (gcMethod == NamespacePolicy.StoragePoolGCMethod.LastAccess)
				{
					// only run for namespaces set to use last access tracking
					return true;
				}

				if (gcMethod == NamespacePolicy.StoragePoolGCMethod.Always)
				{
					// this is a old namespace that should be cleaned up
					return true;
				}
			}
			catch (NamespaceNotFoundException)
			{
				_logger.LogWarning("Unknown namespace {Namespace} when attempting to GC References. To opt in to deleting the old namespace add a policy for it with the GcMethod set to always", ns);
				return false;
			}

			return false;
		}

		public async Task<int> Cleanup(CancellationToken cancellationToken)
		{
			int countOfDeletedRecords = 0;
			DateTime cutoffTime = DateTime.Now.AddSeconds(-1 * _settings.CurrentValue.LastAccessCutoff.TotalSeconds);
			long consideredCount = 0;
			DateTime cleanupStart = DateTime.Now;

			Task cleanupTrackedStateTask = _referencesStore.CleanupTrackedStateAsync();

			await Parallel.ForEachAsync(_referencesStore.GetRecordsAsync(cancellationToken),
				new ParallelOptions
				{
					MaxDegreeOfParallelism = _settings.CurrentValue.OrphanRefMaxParallelOperations,
					CancellationToken = cancellationToken
				}, async (tuple, token) =>
				{
					(NamespaceId ns, BucketId bucket, RefId name, DateTime lastAccessTime) = tuple;

					if (!ShouldGCNamespace(ns))
					{
						return;
					}

					_logger.LogDebug(
						"Considering object in {Namespace} {Bucket} {Name} for deletion, was last updated {LastAccessTime}",
						ns, bucket, name, lastAccessTime);

					Interlocked.Increment(ref consideredCount);
					_cleanupRefsConsidered.Record(consideredCount, Array.Empty<KeyValuePair<string, object?>>());

					if (lastAccessTime < cutoffTime)
					{
						_logger.LogInformation(
							"Attempting to delete object {Namespace} {Bucket} {Name} as it was last updated {LastAccessTime} which is older then {CutoffTime}",
							ns, bucket, name, lastAccessTime, cutoffTime);

						await DeleteRefAsync(ns, bucket, name, cancellationToken);

						Interlocked.Increment(ref countOfDeletedRecords);

						return;
					}

					// if a object was accessed within the last two hours we will let it live even if its un-finalized as it might be written to right now
					if (lastAccessTime < DateTime.Now.AddHours(-2))
					{
						try
						{
							RefRecord refRecord = await _referencesStore.GetAsync(ns, bucket, name, IReferencesStore.FieldFlags.None, IReferencesStore.OperationFlags.BypassCache, cancellationToken);
							if (!refRecord.IsFinalized)
							{
								_logger.LogInformation("Deleting object {Namespace} {Bucket} {Name} as it is not finalized. Was last accessed at {LastAccessTime}", ns, bucket, name, lastAccessTime);

								await DeleteRefAsync(ns, bucket, name, cancellationToken);
								return;
							}
						}
						catch (RefNotFoundException)
						{
							// ignore refs that can not be found, will be cleaned up later
						}
					}
				});

			await cleanupTrackedStateTask;

			TimeSpan cleanupDuration = DateTime.Now - cleanupStart;
			_logger.LogInformation(
				"Finished cleaning refs. Refs considered: {ConsideredCount} Refs Deleted: {DeletedCount}. Cleanup took: {CleanupDuration}", consideredCount, countOfDeletedRecords, cleanupDuration);

			return countOfDeletedRecords;
		}

		private async Task<bool> DeleteRefAsync(NamespaceId ns, BucketId bucket, RefId name, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.StartActiveSpan("gc.ref")
				.SetAttribute("operation.name", "gc.ref")
				.SetAttribute("resource.name", $"{ns}:{bucket}.{name}")
				.SetAttribute("namespace", ns.ToString());
			// delete the old record from the ref refs

			bool storeDelete = false;
			try
			{
				if (_cloudDDCSettings.CurrentValue.EnableBucketStatsTracking)
				{
					List<BlobId> blobs = new List<BlobId>();
					try
					{
						blobs = await _objectService.GetReferencedBlobsAsync(ns, bucket, name, ignoreMissingBlobs: true, cancellationToken: cancellationToken);
					}
					catch (RefNotFoundException)
					{
						// if the ref is already deleted its not possible for us to cleanup the bucket list
						_logger.LogWarning("Ref {Name} {Bucket} {Namespace} not found when cleaning up stats tracking, unable to remove blob tracking thus stats will be incorrect.", name, bucket, ns);
					}
					await _blobIndex.RemoveBlobFromBucketListAsync(ns, bucket, name, blobs, cancellationToken);
				}

				storeDelete = await _refService.DeleteAsync(ns, bucket, name, cancellationToken);
				if (storeDelete && _settings.CurrentValue.WriteDeleteToReplicationLog)
				{
					// insert a delete event into the transaction log
					await _replicationLog.InsertDeleteEventAsync(ns, bucket, name, null);
				}
			}
			catch (Exception e)
			{
				_logger.LogWarning(e, "Exception when attempting to delete record {Bucket} {Name} in {Namespace}",
					bucket, name, ns);
			}

			return storeDelete;
		}
	}
}
