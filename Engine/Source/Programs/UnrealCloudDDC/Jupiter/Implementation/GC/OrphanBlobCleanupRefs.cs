// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.Metrics;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Implementation.Blob;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class OrphanBlobCleanupRefs : IBlobCleanup
	{
		private readonly IOptionsMonitor<GCSettings> _gcSettings;
		private readonly IBlobService _blobService;
		private readonly IRefService _refService;
		private readonly IBlobIndex _blobIndex;
		private readonly IBlockStore _blockStore;
		private readonly ILeaderElection _leaderElection;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;

		private readonly Counter<long> _consideredBlobCounter;
		private readonly Counter<long> _deletedBlobCounter;

		// ReSharper disable once UnusedMember.Global
		public OrphanBlobCleanupRefs(IOptionsMonitor<GCSettings> gcSettings, IBlobService blobService, IRefService refService, IBlobIndex blobIndex, IBlockStore blockStore, ILeaderElection leaderElection, INamespacePolicyResolver namespacePolicyResolver, Tracer tracer, ILogger<OrphanBlobCleanupRefs> logger, Meter meter)
		{
			_gcSettings = gcSettings;
			_blobService = blobService;
			_refService = refService;
			_blobIndex = blobIndex;
			_blockStore = blockStore;
			_leaderElection = leaderElection;
			_namespacePolicyResolver = namespacePolicyResolver;
			_tracer = tracer;
			_logger = logger;
			_consideredBlobCounter = meter.CreateCounter<long>("gc.blob.considered");
			_deletedBlobCounter = meter.CreateCounter<long>("gc.blob.deleted");
		}

		public bool ShouldRun()
		{
			if (!_leaderElection.IsThisInstanceLeader())
			{
				return false;
			}

			return true;
		}

		public async Task<ulong> CleanupAsync(CancellationToken cancellationToken)
		{
			if (!_leaderElection.IsThisInstanceLeader())
			{
				_logger.LogInformation("Skipped orphan blob (refs) cleanup run as this instance is not the leader");
				return 0;
			}

			List<NamespaceId> namespaces = await ListNamespaces().ToListAsync(cancellationToken: cancellationToken);
			List<NamespaceId> namespacesThatHaveBeenChecked = new List<NamespaceId>();
			// enumerate all namespaces, and check if the old blob is valid in any of them to allow for a blob store to just store them in a single pile if it wants to
			ulong countOfBlobsRemoved = 0;
			_logger.LogInformation("Started orphan blob");
			foreach (NamespaceId @namespace in namespaces)
			{
				// if we have already checked this namespace there is no need to repeat it
				if (namespacesThatHaveBeenChecked.Contains(@namespace))
				{
					continue;
				}
				if (cancellationToken.IsCancellationRequested)
				{
					break;
				}

				DateTime startTime = DateTime.Now;
				NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(@namespace);
				List<NamespaceId> namespacesThatSharePool = namespaces.Where(ns => _namespacePolicyResolver.GetPoliciesForNs(ns).StoragePool == policy.StoragePool).ToList();

				_logger.LogInformation("Running Orphan GC For StoragePool: {StoragePool}", policy.StoragePool);
				namespacesThatHaveBeenChecked.AddRange(namespacesThatSharePool);
				// only consider blobs that have been around for long enough
				DateTime cutoff = DateTime.Now.AddMinutes(-1 * _gcSettings.CurrentValue.OrphanBlobCutoffDurationMinutes);
				await Parallel.ForEachAsync(_blobService.ListObjectsAsync(@namespace, cancellationToken),
					new ParallelOptions { MaxDegreeOfParallelism = _gcSettings.CurrentValue.OrphanGCMaxParallelOperations, CancellationToken = cancellationToken },
					async (tuple, ctx) =>
					{
						(BlobId blob, DateTime lastModified) = tuple;

						if (lastModified > cutoff)
						{
							return;
						}

						if (cancellationToken.IsCancellationRequested)
						{
							return;
						}

						bool removed = await GCBlobAsync(policy.StoragePool, namespacesThatSharePool, blob, lastModified, cancellationToken);

						if (removed)
						{
							Interlocked.Increment(ref countOfBlobsRemoved);
						}
					});

				TimeSpan storagePoolGcDuration = DateTime.Now - startTime;
				_logger.LogInformation("Finished running Orphan GC For StoragePool: {StoragePool}. Took {Duration}", policy.StoragePool, storagePoolGcDuration);
			}

			_logger.LogInformation("Finished running Orphan GC");
			return countOfBlobsRemoved;
		}

		private async Task<bool> GCBlobAsync(string storagePool, List<NamespaceId> namespacesThatSharePool, BlobId blob, DateTime lastModifiedTime, CancellationToken cancellationToken)
		{
			string storagePoolName = string.IsNullOrEmpty(storagePool) ? "default" : storagePool;
			using TelemetrySpan removeBlobScope = _tracer.StartActiveSpan("gc.blob")
				.SetAttribute("operation.name", "gc.blob")
				.SetAttribute("resource.name", $"{storagePoolName}.{blob}")
				.SetAttribute("wasBlock", false.ToString());

			bool found = false;
			bool noReferences = true;

			// check all namespaces that share the same storage pool for presence of the blob
			foreach (NamespaceId blobNamespace in namespacesThatSharePool)
			{
				if (cancellationToken.IsCancellationRequested)
				{
					break;
				}

				if (found)
				{
					break;
				}

				IAsyncEnumerable<BaseBlobReference> references = _blobIndex.GetBlobReferencesAsync(blobNamespace, blob, cancellationToken);

				List<BaseBlobReference> oldReferences = new List<BaseBlobReference>();

				await foreach (BaseBlobReference baseBlobReference in references.WithCancellation(cancellationToken))
				{
					if (found)
					{
						break;
					}

					if (baseBlobReference is RefBlobReference refBlobReference)
					{
						BucketId bucket = refBlobReference.Bucket;
						RefId key = refBlobReference.Key;

						noReferences = false;
						try
						{
							bool refFound = await _refService.ExistsAsync(blobNamespace, bucket, key, cancellationToken);

							if (refFound)
							{
								found = true;
								break;
							}
							else
							{
								// this is not a valid reference so we should delete
								oldReferences.Add(refBlobReference);
							}
						}
						catch (RefNotFoundException)
						{
							// this is not a valid reference so we should delete
							oldReferences.Add(refBlobReference);
						}
					}
					else if (baseBlobReference is BlobToBlobReference blobReference)
					{
						noReferences = false;
						removeBlobScope.SetAttribute("hadBlobReference", bool.TrueString);

						BlobId referringBlob = blobReference.Blob;
						bool blobFound = await _blobService.ExistsAsync(blobNamespace, referringBlob, cancellationToken: cancellationToken);
						if (blobFound)
						{
							found = true;
							break;
						}
						else
						{
							oldReferences.Add(blobReference);
						}
					}
					else
					{
						throw new NotImplementedException($"Unknown blob reference type {baseBlobReference.GetType().Name}");
					}
				}

				if (found)
				{
					// if the object is still alive but had old references we remove the old references to keep the size of the references array more reasonable
					await _blobIndex.RemoveReferencesAsync(blobNamespace, blob, oldReferences, cancellationToken);
				}
			}

			if (cancellationToken.IsCancellationRequested)
			{
				return false;
			}

			removeBlobScope.SetAttribute("removed", (!found).ToString());

			_consideredBlobCounter.Add(1);

			// something is still referencing this blob, we should not delete it
			if (found)
			{
				return false;
			}

			_deletedBlobCounter.Add(1);
			_logger.LogInformation("GC Orphan blob {Blob} from {StoragePool} which was last modified at {LastModifiedTime}", blob, storagePoolName, lastModifiedTime);

			if (lastModifiedTime > DateTime.Now.AddHours(-6.0))
			{
				// the blob was added within the last 6 hours and then deleted, emit warning as that indicates the blob likely wasn't needed by anything. This warning can help detect bugs were blobs that should be referenced are getting deleted for some reason
				_logger.LogWarning("Recently added blob {Blob} from {StoragePool} was deleted. Indicates blob upload without anything that referenced it which is safe but can be a sign of an issue", blob, storagePoolName);
			}

			if (noReferences)
			{
				// this blob had no references pointing to it which is not what we would expect, we would expect it to have some reference but that this reference in turn didn't exist
				_logger.LogWarning("{Blob} from {StoragePool} was deleted and had no references to it. This is safe but a indication that data is added and never referenced", blob, storagePoolName);
			}

			if (_gcSettings.CurrentValue.AllowBlockMetadataCleanup)
			{
				// delete block information if this blob is a block
				foreach (NamespaceId namespaceId in namespacesThatSharePool)
				{
					BlobId? metadata = await _blockStore.GetBlockMetadataAsync(namespaceId, blob);
					if (metadata != null)
					{
						removeBlobScope.SetAttribute("wasBlock", true.ToString());

						// this blob is a block, GC the block unique tracking
						await _blockStore.DeleteBlockAsync(namespaceId, metadata);
					}
				}
			}

			// delete the blob itself
			try
			{
				await _blobService.DeleteObjectAsync(namespacesThatSharePool, blob, cancellationToken);
			}
			catch (BlobNotFoundException)
			{
				// ignore blob not found exceptions, if it didn't exist it has been removed so we are happy either way
			}
			catch (Exception e)
			{
				_logger.LogWarning("Failed to delete blob {Blob} from {StoragePool} due to {Error}", blob, storagePoolName, e.Message);
			}
			
			return true;

		}

		private IAsyncEnumerable<NamespaceId> ListNamespaces()
		{
			return _refService.GetNamespacesAsync(CancellationToken.None);
		}
	}
}
