// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class RefCleanupState
	{
		public RefCleanupState(IRefCleanup refCleanup)
		{
			RefCleanup = refCleanup;
		}

		public IRefCleanup RefCleanup { get; }
		public Task? RunningCleanupTask { get; set; } = null;
	}

	public class RefCleanupService : PollingService<RefCleanupState>
	{
		private readonly IOptionsMonitor<GCSettings> _settings;
		private readonly ILeaderElection _leaderElection;
		private volatile bool _alreadyPolling;
		private readonly ILogger _logger;

		public RefCleanupService(IOptionsMonitor<GCSettings> settings, IRefCleanup refCleanup, ILeaderElection leaderElection, ILogger<RefCleanupService> logger) : base(serviceName: nameof(RefCleanupService), settings.CurrentValue.RefCleanupPollFrequency, new RefCleanupState(refCleanup), logger, startAtRandomTime: false)
		{
			_settings = settings;
			_leaderElection = leaderElection;
			_logger = logger;
		}

		protected override bool ShouldStartPolling()
		{
			return _settings.CurrentValue.CleanOldRefRecords;
		}

		public override async Task<bool> OnPollAsync(RefCleanupState state, CancellationToken cancellationToken)
		{
			if (_alreadyPolling)
			{
				return false;
			}

			_alreadyPolling = true;
			try
			{
				if (!_leaderElection.IsThisInstanceLeader())
				{
					_logger.LogInformation("Skipped ref cleanup run as this instance was not the leader");
					return false;
				}

				if (!state.RunningCleanupTask?.IsCompleted ?? false)
				{
					return false;
				}

				if (state.RunningCleanupTask != null)
				{
					await state.RunningCleanupTask;
				}
				state.RunningCleanupTask = DoCleanupAsync(state, cancellationToken);

				return true;

			}
			finally
			{
				_alreadyPolling = false;
			}
		}

		private async Task DoCleanupAsync(RefCleanupState state, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Attempting to run Refs Cleanup. ");
			try
			{
				int countOfRemovedRecords = await state.RefCleanup.Cleanup(cancellationToken);
				_logger.LogInformation("Ran Refs Cleanup. Deleted {CountRefRecords}", countOfRemovedRecords);
			}
			catch (Exception e)
			{
				_logger.LogError("Error running Refs Cleanup. {Exception}", e);
			}
		}
	}
}
