// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common.Implementation;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class TempCleanupService : PollingService<object>
	{
		private readonly IOptionsMonitor<GCSettings> _gcSettings;
		private readonly IOptionsMonitor<BufferedPayloadOptions> _bufferedPayloadOptions;
		private volatile bool _alreadyPolling;
		private readonly ILogger _logger;

		public TempCleanupService(IOptionsMonitor<GCSettings> gcSettings, IOptionsMonitor<BufferedPayloadOptions> bufferedPayloadOptions, ILogger<TempCleanupService> logger) : base(serviceName: nameof(TempCleanupService), gcSettings.CurrentValue.TempCleanupPollFrequency, new object(), logger)
		{
			_gcSettings = gcSettings;
			_bufferedPayloadOptions = bufferedPayloadOptions;
			_logger = logger;
		}

		protected override bool ShouldStartPolling()
		{
			return _gcSettings.CurrentValue.CleanTempFiles;
		}

		public override async Task<bool> OnPollAsync(object state, CancellationToken cancellationToken)
		{
			if (_alreadyPolling)
			{
				return false;
			}

			_alreadyPolling = true;
			try
			{
				await Task.Run(() => Cleanup(cancellationToken), cancellationToken);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception running temp file cleanup");
				return false;
			}
			finally
			{
				_alreadyPolling = false;
			}
		}

		private void Cleanup(CancellationToken cancellationToken)
		{
			_logger.LogInformation("Beginning temp file cleanup");
			
			IEnumerable<FileInfo> fileInfos = GetObjectsOlderThan(DateTime.Now - _gcSettings.CurrentValue.TempCleanupMaximumAge);

			ulong countOfBlobsRemoved = 0;
			
			foreach (FileInfo fi in fileInfos)
			{
				if (cancellationToken.IsCancellationRequested)
				{
					break;
				}
				
				try
				{
					fi.Delete();
					countOfBlobsRemoved++;
				}
				catch (FileNotFoundException)
				{
					// Already deleted
				}
				catch (DirectoryNotFoundException)
				{
					// Already deleted
				}
				catch (IOException)
				{
					// in use
				}
			}
			
			_logger.LogInformation("Removed {Count} temp files", countOfBlobsRemoved);
		}

		private IEnumerable<FileInfo> GetObjectsOlderThan(DateTime cutoff)
		{
			DirectoryInfo di = new DirectoryInfo(_bufferedPayloadOptions.CurrentValue.FilesystemTempPayloadRoot);
			
			if (!di.Exists)
			{
				return Array.Empty<FileInfo>();
			}
			
			return di.EnumerateFiles("*", SearchOption.AllDirectories).Where(x => x.LastWriteTime < cutoff);
		}
	}
}

