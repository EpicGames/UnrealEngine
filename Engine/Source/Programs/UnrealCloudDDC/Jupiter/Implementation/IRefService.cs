// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Serialization;
using Jupiter.Common;

namespace Jupiter.Implementation
{
	public interface IRefService
	{
		Task<(RefRecord, BlobContents?)> GetAsync(NamespaceId ns, BucketId bucket, RefId key, string[]? fields = null, bool consistentRead = false, bool doLastAccessTracking = true, CancellationToken cancellationToken = default);

		Task<(ContentId[], BlobId[])> PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, CbObject payload, Action<BlobId>? onBlobFound = null, bool allowOverwrite = false, CancellationToken cancellationToken = default);
		Task<(ContentId[], BlobId[])> FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, Action<BlobId>? onBlobFound = null, CancellationToken cancellationToken = default);

		IAsyncEnumerable<NamespaceId> GetNamespacesAsync(CancellationToken cancellationToken = default);
		IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, CancellationToken cancellationToken = default);

		Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken = default);
		Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default);
		Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken = default);

		Task<bool> ExistsAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken = default);
		Task<List<BlobId>> GetReferencedBlobsAsync(NamespaceId ns, BucketId bucket, RefId key, bool ignoreMissingBlobs, CancellationToken cancellationToken = default);
		Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default);
	}

	public class ObjectHashMismatchException : Exception
	{
		public ObjectHashMismatchException(NamespaceId ns, BucketId bucket, RefId name, BlobId suppliedHash, BlobId actualHash) : base($"Object {name} in bucket {bucket} and namespace {ns} did not reference hash {suppliedHash} was referencing {actualHash}")
		{
		}
	}
}
