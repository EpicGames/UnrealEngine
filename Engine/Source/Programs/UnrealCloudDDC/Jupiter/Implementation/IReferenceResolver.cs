// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Serialization;
using Jupiter.Common;
using Jupiter.Utils;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public abstract class Attachment
	{
		public abstract IoHash AsIoHash();
	}

	public class BlobAttachment : Attachment
	{
		public BlobId Identifier { get; }

		public BlobAttachment(BlobId blobIdentifier)
		{
			Identifier = blobIdentifier;
		}

		public override IoHash AsIoHash()
		{
			return Identifier.AsIoHash();
		}
	}

	public class ObjectAttachment : Attachment
	{
		public BlobId Identifier { get; }

		public ObjectAttachment(BlobId blobIdentifier)
		{
			Identifier = blobIdentifier;
		}

		public override IoHash AsIoHash()
		{
			return Identifier.AsIoHash();
		}
	}

	public class ContentIdAttachment : Attachment
	{
		public ContentId Identifier { get; }
		public BlobId[] ReferencedBlobs { get; }

		public ContentIdAttachment(ContentId contentId, BlobId[] referencedBlobs)
		{
			Identifier = contentId;
			ReferencedBlobs = referencedBlobs;
		}

		public override IoHash AsIoHash()
		{
			return Identifier.AsIoHash();
		}
	}

	public interface IReferenceResolver
	{
		/// <summary>
		/// Returns the blobs referenced from the cb object and any children
		/// These blobs are guaranteed to exist, if any blob is missing
		/// PartialReferenceResolveException or ReferenceIsMissingBlobsException are raised
		/// </summary>
		/// <param name="ns">The namespace to check</param>
		/// <param name="cb">The compact binary object to resolve references for</param>
		/// <param name="ignoreMissingBlobs">Set to true to always returned the blobs found, ignoring anything that is missing rather then throwing</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		IAsyncEnumerable<BlobId> GetReferencedBlobsAsync(NamespaceId ns, CbObject cb, bool ignoreMissingBlobs = false, CancellationToken cancellationToken = default);

		/// <summary>
		/// Returns which attachments exist in the cb object or any children
		/// These attachments are not guaranteed to reference blobs that actually exist
		/// </summary>
		/// <param name="ns">The namespace to check</param>
		/// <param name="cb">The compact binary object to resolve references for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		IAsyncEnumerable<Attachment> GetAttachmentsAsync(NamespaceId ns, CbObject cb, CancellationToken cancellationToken);

		/// <summary>
		/// Resolve a blob id that might or might not be a content id into the blob it would reference
		/// </summary>
		/// <param name="ns">The namespace used</param>
		/// <param name="blobId">The blob id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<BlobId[]?> ResolveIdAsync(NamespaceId ns, BlobId  blobId, CancellationToken cancellationToken);
	}

	public class ReferenceResolver : IReferenceResolver
	{
		private readonly IBlobService _blobStore;
		private readonly IContentIdStore _contentIdStore;
		private readonly IOptionsMonitor<ReferenceResolverSettings> _settings;
		private readonly Tracer _tracer;

		public ReferenceResolver(IBlobService blobStore, IContentIdStore contentIdStore, IOptionsMonitor<ReferenceResolverSettings> settings, Tracer tracer)
		{
			_blobStore = blobStore;
			_contentIdStore = contentIdStore;
			_settings = settings;
			_tracer = tracer;
		}

		public async IAsyncEnumerable<Attachment> GetAttachmentsAsync(NamespaceId ns, CbObject cb, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			if (_settings.CurrentValue.UseV2ReferenceResolving)
			{
				await foreach (Attachment attachment in GetAttachmentsV2Async(ns, cb, cancellationToken))
				{
					yield return attachment;
				}
			}
			else
			{
				await foreach (Attachment attachment in GetAttachmentsOldAsync(ns, cb, cancellationToken))
				{
					yield return attachment;
				}
			}
		}

		public async Task<BlobId[]?> ResolveIdAsync(NamespaceId ns, BlobId blobId, CancellationToken cancellationToken)
		{
			try
			{
				(ContentId _, BlobId[]? resolvedBlobs) = await ResolveContentIdAsync(ns, ContentId.FromBlobIdentifier(blobId), cancellationToken);
				bool wasContentId = !(resolvedBlobs is { Length: 1 } && resolvedBlobs[0].Equals(blobId));
				return wasContentId ? resolvedBlobs : new[] { blobId };
			}
			catch (BlobNotFoundException)
			{
				return null;
			}
		}

		private async IAsyncEnumerable<Attachment> GetAttachmentsV2Async(NamespaceId ns, CbObject cb, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			Queue<CbObject> objectsToVisit = new Queue<CbObject>();
			objectsToVisit.Enqueue(cb);
			List<BlobId> unresolvedBlobReferences = new List<BlobId>();

			while (objectsToVisit.Count != 0)
			{
				if (objectsToVisit.TryDequeue(out CbObject? parent))
				{
					List<CbField> fields = new List<CbField>();
					parent.IterateAttachments(field => fields.Add(field));

					foreach (CbField[] chunkedFields in fields.Chunk(_settings.CurrentValue.ChunkSize))
					{
						ConcurrentBag<Attachment> foundAttachments = new ConcurrentBag<Attachment>();
						await Parallel.ForEachAsync(chunkedFields, cancellationToken, async (field, token) =>
						{
							IoHash attachmentHash = field.AsAttachment();

							if (field.IsBinaryAttachment())
							{
								ContentId? contentId = null;
								BlobId[]? resolvedBlobs = null;
								BlobId? blobIdentifier = null;
								bool wasContentId = false;
								try
								{
									(contentId, resolvedBlobs) = await ResolveContentIdAsync(ns, ContentId.FromIoHash(attachmentHash), cancellationToken);
									blobIdentifier = contentId.AsBlobIdentifier();
									wasContentId = !(resolvedBlobs is { Length: 1 } && resolvedBlobs[0].Equals(blobIdentifier));
								}
								catch (InvalidContentIdException)
								{
									resolvedBlobs = null;
								}
								catch (BlobNotFoundException e)
								{
									unresolvedBlobReferences.Add(e.Blob);
								}

								if (wasContentId && resolvedBlobs != null)
								{
									foundAttachments.Add(new ContentIdAttachment(contentId!, resolvedBlobs));
								}
								else
								{
									foundAttachments.Add(new BlobAttachment(blobIdentifier!));
								}
							}
							else if (field.IsObjectAttachment())
							{

								BlobId blobIdentifier = BlobId.FromIoHash(attachmentHash);
								foundAttachments.Add(new ObjectAttachment(blobIdentifier));
								
								try
								{
									CbObject childBinaryObject = await ParseCompactBinaryAttachmentAsync(ns, blobIdentifier, cancellationToken);
									objectsToVisit.Enqueue(childBinaryObject);
								}
								catch (BlobNotFoundException e)
								{
									unresolvedBlobReferences.Add(e.Blob);
								}
							}
							else
							{
								throw new NotImplementedException($"Unknown attachment type for field {field}");
							}
						});

						// using a parallel foreach above and consuming the results here means we do not stream the attachments as they are found instead we chunk the fields into pieces and stream the attachments for each piece while still going wide within the chunk

						foreach (Attachment attachment in foundAttachments)
						{
							yield return attachment;
						}	
					}
				}
			}

			if (unresolvedBlobReferences.Count != 0)
			{
				throw new ReferenceIsMissingBlobsException(unresolvedBlobReferences);
			}
		}
		
		private async IAsyncEnumerable<Attachment> GetAttachmentsOldAsync(NamespaceId ns, CbObject cb, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			Queue<CbObject> objectsToVisit = new Queue<CbObject>();
			objectsToVisit.Enqueue(cb);
			List<BlobId> unresolvedBlobReferences = new List<BlobId>();

			List<Task<CbObject>> pendingCompactBinaryAttachments = new();
			List<Task<(ContentId, BlobId[]?)>> pendingContentIdResolves = new();

			const int MaxParallelAttachmentTasks = 64;

			while (pendingCompactBinaryAttachments.Count != 0 || pendingContentIdResolves.Count != 0 || objectsToVisit.Count != 0)
			{
				List<Attachment> attachments = new List<Attachment>();

				if (objectsToVisit.TryDequeue(out CbObject? parent))
				{
					List<CbField> fields = new List<CbField>();

					parent.IterateAttachments(field => fields.Add(field));

					foreach (CbField field in fields)
					{
						IoHash attachmentHash = field.AsAttachment();

						BlobId blobIdentifier = BlobId.FromIoHash(attachmentHash);
						ContentId contentId = ContentId.FromIoHash(attachmentHash);

						if (field.IsBinaryAttachment())
						{
							Task<(ContentId, BlobId[]?)> resolveContentId = ResolveContentIdAsync(ns, contentId, cancellationToken);
							pendingContentIdResolves.Add(resolveContentId);
						}
						else if (field.IsObjectAttachment())
						{
							attachments.Add(new ObjectAttachment(blobIdentifier));
							pendingCompactBinaryAttachments.Add(ParseCompactBinaryAttachmentAsync(ns, blobIdentifier, cancellationToken));
						}
						else
						{
							throw new NotImplementedException($"Unknown attachment type for field {field}");
						}

						if (pendingContentIdResolves.Count + pendingCompactBinaryAttachments.Count > MaxParallelAttachmentTasks)
						{
							// wait for one of our lists of pending tasks to be finished to avoid spawning more tasks
							// also avoiding waiting on a empty list as that is not allowed
							if (pendingContentIdResolves.Count == 0)
							{
								await Task.WhenAny(pendingCompactBinaryAttachments);
								await ProcessCompactBinaryAttachmentsAsync(pendingCompactBinaryAttachments, objectsToVisit, unresolvedBlobReferences);
							}
							else if (pendingCompactBinaryAttachments.Count == 0)
							{
								await Task.WhenAny(pendingContentIdResolves);
								await ProcessContentIdsAsync(attachments, pendingContentIdResolves, unresolvedBlobReferences);
							}
							else
							{
								await Task.WhenAny(Task.WhenAny(pendingContentIdResolves), Task.WhenAny(pendingCompactBinaryAttachments));
								
								await ProcessContentIdsAsync(attachments, pendingContentIdResolves, unresolvedBlobReferences);
								await ProcessCompactBinaryAttachmentsAsync(pendingCompactBinaryAttachments, objectsToVisit, unresolvedBlobReferences);
							}
						}
					}
				}

				await ProcessContentIdsAsync(attachments, pendingContentIdResolves, unresolvedBlobReferences);

				// check for any compact binary attachment fetches and add those to the objects we are handling
				await ProcessCompactBinaryAttachmentsAsync(pendingCompactBinaryAttachments, objectsToVisit, unresolvedBlobReferences);

				bool hasWaited = false;
				// if there are pending resolves left, wait for one of them to finish to avoid busy waiting
				if (pendingCompactBinaryAttachments.Any())
				{
					hasWaited = true;
					await Task.WhenAny(pendingCompactBinaryAttachments);
				}

				// if we did not waiting for compact binary attachment resolves we consider waiting for content id resolves if there are any
				if (!hasWaited && pendingContentIdResolves.Any())
				{
					await Task.WhenAny(pendingContentIdResolves);
				}

				foreach (Attachment attachment in attachments)
				{
					yield return attachment;
				}
			}

			if (unresolvedBlobReferences.Count != 0)
			{
				throw new ReferenceIsMissingBlobsException(unresolvedBlobReferences);
			}
		}

		private static async Task ProcessCompactBinaryAttachmentsAsync(List<Task<CbObject>> pendingCompactBinaryAttachments, Queue<CbObject> objectsToVisit, List<BlobId> unresolvedBlobReferences)
		{
			List<Task<CbObject>> finishedCompactBinaryResolves = new();
			foreach (Task<CbObject> pendingCompactBinaryAttachment in pendingCompactBinaryAttachments)
			{
				if (pendingCompactBinaryAttachment.IsCompleted)
				{
					try
					{
						CbObject childBinaryObject = await pendingCompactBinaryAttachment;
						objectsToVisit.Enqueue(childBinaryObject);
					}
					catch (BlobNotFoundException e)
					{
						unresolvedBlobReferences.Add(e.Blob);
					}
					finishedCompactBinaryResolves.Add(pendingCompactBinaryAttachment);
				}
			}

			// cleanup finished tasks
			foreach (Task<CbObject> finishedTask in finishedCompactBinaryResolves)
			{
				pendingCompactBinaryAttachments.Remove(finishedTask);
			}
		}

		private static async Task ProcessContentIdsAsync(List<Attachment> list, List<Task<(ContentId, BlobId[]?)>> pendingContentIdResolves, List<BlobId> unresolvedBlobReferences)
		{
			// check for any content id resolves to finish
			List<Task<(ContentId, BlobId[]?)>> finishedContentIdResolves = new();
			foreach (Task<(ContentId, BlobId[]?)> pendingContentIdResolve in pendingContentIdResolves)
			{
				if (pendingContentIdResolve.IsCompleted)
				{
					ContentId? contentId = null;
					BlobId[]? resolvedBlobs = null;
					BlobId? blobIdentifier = null;
					bool wasContentId = false;
					try
					{
						(contentId, resolvedBlobs) = await pendingContentIdResolve;
						blobIdentifier = contentId.AsBlobIdentifier();
						wasContentId = !(resolvedBlobs is { Length: 1 } && resolvedBlobs[0].Equals(blobIdentifier));
					}
					catch (InvalidContentIdException)
					{
						resolvedBlobs = null;
					}
					catch (BlobNotFoundException e)
					{
						unresolvedBlobReferences.Add(e.Blob);
					}

					if (wasContentId && resolvedBlobs != null)
					{
						list.Add(new ContentIdAttachment(contentId!, resolvedBlobs));
					}
					else
					{
						list.Add(new BlobAttachment(blobIdentifier!));
					}

					finishedContentIdResolves.Add(pendingContentIdResolve);
				}
			}

			// cleanup finished tasks
			foreach (Task<(ContentId, BlobId[]?)> finishedTask in finishedContentIdResolves)
			{
				pendingContentIdResolves.Remove(finishedTask);
			}
		}

		public async IAsyncEnumerable<BlobId> GetReferencedBlobsAsync(NamespaceId ns, CbObject cb, bool ignoreMissingBlobs = false, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			List<ContentId> unresolvedContentIdReferences = new List<ContentId>();
			List<BlobId> unresolvedBlobReferences = new List<BlobId>();

			// Resolve all the attachments
			await foreach (Attachment attachment in GetAttachmentsAsync(ns, cb, cancellationToken))
			{
				if (attachment is BlobAttachment blobAttachment)
				{
					(BlobId blob, bool exists) = await CheckBlobExistsAsync(ns, blobAttachment.Identifier, cancellationToken);
					if (exists)
					{
						yield return blob;
					}
					else
					{
						unresolvedBlobReferences.Add(blob);
					}
				}
				else if (attachment is ContentIdAttachment contentIdAttachment)
				{
					// If we find a content id we resolve that into the actual blobs it references
					(ContentIdAttachment _, bool exists) = await CheckContentIdExistsAsync(ns, contentIdAttachment, cancellationToken);

					if (!exists)
					{
						unresolvedContentIdReferences.Add(contentIdAttachment.Identifier);
						continue;
					}

					foreach (BlobId b in contentIdAttachment.ReferencedBlobs)
					{
						yield return b;
					}
				}
				else if (attachment is ObjectAttachment objectAttachment)
				{
					// a object just references the same blob, traversing the object attachment is done in GetAttachments
					(BlobId blob, bool exists) = await CheckBlobExistsAsync(ns, objectAttachment.Identifier, cancellationToken);
					if (exists)
					{
						yield return blob;
					}
					else
					{
						unresolvedBlobReferences.Add(blob);
					}
				}
				else
				{
					throw new NotSupportedException($"Unknown attachment type {attachment.GetType()}");
				}
			}

			// if there were any content ids we did not recognize we throw a partial reference exception
			if (!ignoreMissingBlobs && unresolvedContentIdReferences.Count != 0)
			{
				throw new PartialReferenceResolveException(unresolvedContentIdReferences);
			}
			// if there were any blobs missing we throw a partial reference exception
			if (!ignoreMissingBlobs && unresolvedBlobReferences.Count != 0)
			{
				throw new ReferenceIsMissingBlobsException(unresolvedBlobReferences);
			}
		}

		private async Task<(ContentIdAttachment, bool)> CheckContentIdExistsAsync(NamespaceId ns, ContentIdAttachment contentIdAttachment, CancellationToken cancellationToken)
		{
			bool allBlobsExist = true;
			foreach (BlobId b in contentIdAttachment.ReferencedBlobs)
			{
				(BlobId _, bool exists) = await CheckBlobExistsAsync(ns, b, cancellationToken);

				if (!exists)
				{
					allBlobsExist = false;
					break;
				}
			}

			return (contentIdAttachment, allBlobsExist);
		}

		private async Task<CbObject> ParseCompactBinaryAttachmentAsync(NamespaceId ns, BlobId blobIdentifier, CancellationToken cancellationToken)
		{
			BlobContents contents = await _blobStore.GetObjectAsync(ns, blobIdentifier, cancellationToken: cancellationToken);
			byte[] data = await contents.Stream.ToByteArrayAsync(cancellationToken);
			CbObject childBinaryObject = new CbObject(data);

			return childBinaryObject;
		}

		private async Task<(ContentId, BlobId[]?)> ResolveContentIdAsync(NamespaceId ns, ContentId contentId, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.StartActiveSpan("ReferenceResolver.ResolveContentId")
				.SetAttribute("operation.name", "ReferenceResolver.ResolveContentId")
				.SetAttribute("resource.name", contentId.ToString());
			BlobId[]? resolvedBlobs = await _contentIdStore.ResolveAsync(ns, contentId, cancellationToken: cancellationToken);
			return (contentId, resolvedBlobs);
		}

		private async Task<(BlobId, bool)> CheckBlobExistsAsync(NamespaceId ns, BlobId blob, CancellationToken cancellationToken)
		{
			return (blob, await _blobStore.ExistsAsync(ns, blob, cancellationToken: cancellationToken));
		}
	}

	public class ReferenceResolverSettings
	{
		public bool UseV2ReferenceResolving { get; set; } = true;
		public int ChunkSize { get; set; } = 64;
	}

	public class PartialReferenceResolveException : Exception
	{
		public List<ContentId> UnresolvedReferences { get; }

		public PartialReferenceResolveException(List<ContentId> unresolvedReferences) : base($"References missing: {string.Join(',', unresolvedReferences)}")
		{
			UnresolvedReferences = unresolvedReferences;
		}
	}

	public class ReferenceIsMissingBlobsException : Exception
	{
		public List<BlobId> MissingBlobs { get; }

		public ReferenceIsMissingBlobsException(List<BlobId> missingBlobs) : base($"References is missing these blobs: {string.Join(',', missingBlobs)}")
		{
			MissingBlobs = missingBlobs;
		}
	}
}
