// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	// ReSharper disable once ClassNeverInstantiated.Global
	public class LastAccessServiceReferences : IHostedService, IDisposable
	{
		private readonly ILastAccessCache<LastAccessRecord> _lastAccessCacheRecord;
		private readonly IReferencesStore _referencesStore;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;
		private Timer? _timer;
		private readonly UnrealCloudDDCSettings _settings;

		public bool Running { get; private set; }

		public LastAccessServiceReferences(IOptionsMonitor<UnrealCloudDDCSettings> settings, ILastAccessCache<LastAccessRecord> lastAccessCache, IReferencesStore referencesStore, INamespacePolicyResolver namespacePolicyResolver, Tracer tracer, ILogger<LastAccessServiceReferences> logger)
		{
			_lastAccessCacheRecord = lastAccessCache;
			_referencesStore = referencesStore;
			_namespacePolicyResolver = namespacePolicyResolver;
			_tracer = tracer;
			_settings = settings.CurrentValue;
			_logger = logger;
		}

		public Task StartAsync(CancellationToken cancellationToken)
		{
			_logger.LogInformation("Last Access Aggregation service starting.");

			_timer = new Timer(OnUpdate, null, TimeSpan.Zero,
				period: TimeSpan.FromSeconds(_settings.LastAccessRollupFrequencySeconds));
			Running = true;

			return Task.CompletedTask;
		}

		public async Task StopAsync(CancellationToken cancellationToken)
		{
			_logger.LogInformation("Last Access Aggregation service stopping.");

			_timer?.Change(Timeout.Infinite, 0);
			Running = false;

			// process the last records we have built up
			await ProcessLastAccessRecordsAsync(cancellationToken);
		}

		private void OnUpdate(object? state)
		{
			try
			{
				// call results to make sure we join the task
				ProcessLastAccessRecordsAsync(CancellationToken.None).Wait();
			}
			catch (TaskCanceledException)
			{
				// ignore cancellations
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Exception thrown while processing last access records");
				Tracer.CurrentSpan.SetStatus(Status.Error);
				Tracer.CurrentSpan.RecordException(e);
			}
		}

		internal async Task<List<(LastAccessRecord, DateTime)>> ProcessLastAccessRecordsAsync(CancellationToken cancellationToken)
		{
			_logger.LogInformation("Running Last Access Aggregation for refs");
			List<(LastAccessRecord, DateTime)> records = await _lastAccessCacheRecord.GetLastAccessedRecords();
			foreach ((LastAccessRecord record, DateTime lastAccessTime) in records)
			{
				if (!ShouldTrackLastAccess(record.Namespace))
				{
					continue;
				}

				using TelemetrySpan scope = _tracer.StartActiveSpan("lastAccess.update")
					.SetAttribute("operation.name", "lastAccess.update")
					.SetAttribute("resource.name", $"{record.Namespace}:{record.Bucket}.{record.Key}");
				_logger.LogDebug("Updating last access time to {LastAccessTime} for {Record}", lastAccessTime, record);
				await _referencesStore.UpdateLastAccessTimeAsync(record.Namespace, record.Bucket, record.Key, lastAccessTime, cancellationToken);
				// delay 10ms between each record to distribute the load more evenly for the db
				await Task.Delay(10, cancellationToken);
			}

			return records;
		}

		private bool ShouldTrackLastAccess(NamespaceId ns)
		{
			if (!_namespacePolicyResolver.GetPoliciesForNs(ns).LastAccessTracking)
			{
				return false;
			}

			NamespacePolicy.StoragePoolGCMethod gcMethod = _namespacePolicyResolver.GetPoliciesForNs(ns).GcMethod ?? NamespacePolicy.StoragePoolGCMethod.LastAccess;
			return gcMethod == NamespacePolicy.StoragePoolGCMethod.LastAccess;
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				_timer?.Dispose();

			}
		}
	}

	// ReSharper disable once ClassNeverInstantiated.Global
	public class LastAccessBackgroundService : BackgroundService
	{
		private readonly ILastAccessCache<LastAccessRecord> _lastAccessCacheRecord;
		private readonly IReferencesStore _referencesStore;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;
		private readonly UnrealCloudDDCSettings _settings;

		public LastAccessBackgroundService(IOptionsMonitor<UnrealCloudDDCSettings> settings, ILastAccessCache<LastAccessRecord> lastAccessCache, IReferencesStore referencesStore, INamespacePolicyResolver namespacePolicyResolver, Tracer tracer, ILogger<LastAccessServiceReferences> logger)
		{
			_lastAccessCacheRecord = lastAccessCache;
			_referencesStore = referencesStore;
			_namespacePolicyResolver = namespacePolicyResolver;
			_tracer = tracer;
			_settings = settings.CurrentValue;
			_logger = logger;
		}

		protected override async Task ExecuteAsync(CancellationToken stoppingToken)
		{
			while (!stoppingToken.IsCancellationRequested)
			{
				try
				{
					Task delay = Task.Delay(TimeSpan.FromSeconds(_settings.LastAccessRollupFrequencySeconds), stoppingToken);

					// run the rollout once every rollup frequency at most, most of the time this takes longer then the rollup frequency at which point we just start again
					await Task.WhenAll(delay, ProcessLastAccessRecordsAsync(stoppingToken));
				}
				catch (TaskCanceledException)
				{
					// ignore cancellations
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Exception thrown while processing last access records");
					Tracer.CurrentSpan.SetStatus(Status.Error);
					Tracer.CurrentSpan.RecordException(e);
				}
			}
		}

		private async Task<List<(LastAccessRecord, DateTime)>> ProcessLastAccessRecordsAsync(CancellationToken cancellationToken)
		{
			_logger.LogInformation("Running Last Access Aggregation for refs");
			List<(LastAccessRecord, DateTime)> records = await _lastAccessCacheRecord.GetLastAccessedRecords();
			foreach ((LastAccessRecord record, DateTime lastAccessTime) in records)
			{
				if (!ShouldTrackLastAccess(record.Namespace))
				{
					continue;
				}

				using TelemetrySpan scope = _tracer.StartActiveSpan("lastAccess.update")
					.SetAttribute("operation.name", "lastAccess.update")
					.SetAttribute("resource.name", $"{record.Namespace}:{record.Bucket}.{record.Key}");
				_logger.LogDebug("Updating last access time to {LastAccessTime} for {Record}", lastAccessTime, record);
				await _referencesStore.UpdateLastAccessTimeAsync(record.Namespace, record.Bucket, record.Key, lastAccessTime, cancellationToken);
				// delay 10ms between each record to distribute the load more evenly for the db
				await Task.Delay(10, cancellationToken);
			}

			return records;
		}

		private bool ShouldTrackLastAccess(NamespaceId ns)
		{
			if (!_namespacePolicyResolver.GetPoliciesForNs(ns).LastAccessTracking)
			{
				return false;
			}

			NamespacePolicy.StoragePoolGCMethod gcMethod = _namespacePolicyResolver.GetPoliciesForNs(ns).GcMethod ?? NamespacePolicy.StoragePoolGCMethod.LastAccess;
			return gcMethod == NamespacePolicy.StoragePoolGCMethod.LastAccess;
		}
	}
}
