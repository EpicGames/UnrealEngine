// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class LastAccessRecord
	{
		public LastAccessRecord(NamespaceId ns, BucketId bucket, RefId key)
		{
			Namespace = ns;
			Bucket = bucket;
			Key = key;
		}

		public NamespaceId Namespace { get; set; }
		public BucketId Bucket { get; set; }
		public RefId Key { get; set; }
	}

	public class LastAccessTrackerReference : LastAccessTracker<LastAccessRecord>
	{
		public LastAccessTrackerReference(IOptionsMonitor<UnrealCloudDDCSettings> settings, Tracer tracer, ILogger<LastAccessTrackerReference> logger) : base(settings, tracer, logger)
		{
		}

		protected override string BuildCacheKey(LastAccessRecord record)
		{
			return $"{record.Namespace}.{record.Bucket}.{record.Key}";
		}
	}

	public abstract class LastAccessTracker<T> : ILastAccessTracker<T>, ILastAccessCache<T>
	{
		private readonly IOptionsMonitor<UnrealCloudDDCSettings> _settings;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;

		private ConcurrentDictionary<string, LastAccessRecord> _cache = new ConcurrentDictionary<string, LastAccessRecord>();

		// we will exchange the refs dictionary when fetching the records and use a rw lock to make sure no-one is trying to add things at the same time
		private readonly ReaderWriterLock _rwLock = new ReaderWriterLock();

		protected LastAccessTracker(IOptionsMonitor<UnrealCloudDDCSettings> settings, Tracer tracer, ILogger logger)
		{
			_settings = settings;
			_tracer = tracer;
			_logger = logger;
		}

		protected abstract string BuildCacheKey(T record);

		public Task TrackUsed(T record)
		{
			if (!_settings.CurrentValue.EnableLastAccessTracking)
			{
				return Task.CompletedTask;
			}

			return Task.Run(() =>
			{
				using TelemetrySpan _ = _tracer.StartActiveSpan("lastAccessTracker.track").SetAttribute("operation.name", "lastAccessTracker.track");
				try
				{
					_rwLock.AcquireReaderLock(-1);

					string cacheKey = BuildCacheKey(record);
					_logger.LogDebug("Last Access time updated for {RefRecordCacheKey}", cacheKey);

					_cache.AddOrUpdate(cacheKey, _ => new LastAccessRecord(record, DateTime.Now),
						(_, cacheRecord) =>
						{
							cacheRecord.LastAccessTime = DateTime.Now;
							return cacheRecord;
						});
				}
				finally
				{
					_rwLock.ReleaseLock();
				}
			});
		}

		public async Task<List<(T, DateTime)>> GetLastAccessedRecords()
		{
			return await Task.Run(() =>
			{
				try
				{
					_rwLock.AcquireWriterLock(-1);

					_logger.LogDebug("Last Access Records collected");
					ConcurrentDictionary<string, LastAccessRecord> localReference = _cache;

					_cache = new ConcurrentDictionary<string, LastAccessRecord>();

					// ToArray is important here to make sure this is thread safe as just using linq queries on a concurrent dict is not thead safe
					// http://blog.i3arnon.com/2018/01/16/concurrent-dictionary-tolist/
					return localReference.ToArray().Select(pair => (pair.Value.Record, pair.Value.LastAccessTime))
						.ToList();
				}
				finally
				{
					_rwLock.ReleaseWriterLock();
				}
			});
		}

		private class LastAccessRecord
		{
			public T Record { get; }
			public DateTime LastAccessTime { get; set; }

			public LastAccessRecord(T record, in DateTime lastAccessTime)
			{
				Record = record;
				LastAccessTime = lastAccessTime;
			}
		}
	}
}
