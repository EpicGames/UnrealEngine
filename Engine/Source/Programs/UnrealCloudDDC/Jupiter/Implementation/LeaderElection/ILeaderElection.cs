// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace Jupiter.Implementation
{
	public class OnLeaderChangedEventArgs
	{
		public bool IsLeader { get; }
		public string LeaderName { get; }

		public OnLeaderChangedEventArgs(bool isLeader, string leaderName)
		{
			IsLeader = isLeader;
			LeaderName = leaderName;
		}
	}

	public interface ILeaderElection
	{
		bool IsThisInstanceLeader();

		event EventHandler<OnLeaderChangedEventArgs>? OnLeaderChanged;
	}

	public class LeaderElectionStub : ILeaderElection
	{
		private readonly bool _isLeader;

		public LeaderElectionStub(bool isLeader)
		{
			_isLeader = isLeader;

			OnLeaderChanged?.Invoke(this, new OnLeaderChangedEventArgs(isLeader, ""));
		}

		public bool IsThisInstanceLeader()
		{
			return _isLeader;
		}

		public event EventHandler<OnLeaderChangedEventArgs>? OnLeaderChanged;
	}
}
