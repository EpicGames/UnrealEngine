// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel.DataAnnotations;
using System.Net;
using System.Threading;
using System.Threading.Tasks;
using k8s;
using k8s.Autorest;
using k8s.LeaderElection;
using k8s.LeaderElection.ResourceLock;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation.LeaderElection
{
	public class KubernetesLeaderElectionSettings
	{
		// these defaults are based on recommendations from the kubernetes client - https://github.com/kubernetes/client-go/blob/master/tools/leaderelection/leaderelection.go
		public TimeSpan LeaseDuration { get; set; } = TimeSpan.FromSeconds(15);
		public TimeSpan RetryPeriod { get; set; } = TimeSpan.FromMilliseconds(2000);
		public TimeSpan RenewDeadline { get; set; } = TimeSpan.FromSeconds(10);

		public string ConfigMapName { get; set; } = "jupiter-leader-configmap";

		[Required]
		public string Namespace { get; set; } = "";
		[Required] public string PodName { get; set; } = "";

		public string PeerPodLabelSelector { get; set; } = "app.kubernetes.io/name=unreal-cloud-ddc";
	}

	public class KubernetesLeaderElectionState
	{

	}

	public class KubernetesLeaderElection : PollingService<KubernetesLeaderElectionState>, ILeaderElection
	{
		private readonly ILogger _logger;

		private readonly Kubernetes _client;
		private readonly LeaderElector _leaderElector;
		private readonly ConfigMapLock _configMapLock;
		private readonly string _identity;

		public KubernetesLeaderElection(IOptionsMonitor<KubernetesLeaderElectionSettings> leaderSettings, ILogger<KubernetesLeaderElection> logger) : base("Kubernetes Leader Election", TimeSpan.FromSeconds(1), new KubernetesLeaderElectionState(), logger, startAtRandomTime: true)
		{
			_logger = logger;

			KubernetesLeaderElectionSettings settings = leaderSettings.CurrentValue;
			// As we are determining if we are the leader we just assume we are running in a kubernetes cluster
			KubernetesClientConfiguration config = KubernetesClientConfiguration.InClusterConfig();
			_client = new Kubernetes(config);

			_identity = System.Net.Dns.GetHostName();
			_logger.LogInformation("Participating in kubernetes leadership election as {Identity} using {Resource} under {Namespace}", _identity, settings.ConfigMapName, settings.Namespace);
			_configMapLock = new ConfigMapLock(_client, settings.Namespace, settings.ConfigMapName, _identity);

			LeaderElectionConfig leaderElectionConfig = new LeaderElectionConfig(_configMapLock)
			{
				LeaseDuration = settings.LeaseDuration,
				RetryPeriod = settings.RetryPeriod,
				RenewDeadline = settings.RenewDeadline
			};

			_leaderElector = new LeaderElector(leaderElectionConfig);
			_leaderElector.OnStartedLeading += () => _logger.LogWarning("Acquired leadership from kubernetes");
			_leaderElector.OnStoppedLeading += () => _logger.LogWarning("Lost leadership in kubernetes");
			_leaderElector.OnNewLeader += OnLeaderElectorNewLeader;
		}

		private void OnLeaderElectorNewLeader(string leaderName)
		{
			_logger.LogWarning("{Instance} is the new leader", leaderName);

			bool isLeader = string.Equals(leaderName, _identity, StringComparison.OrdinalIgnoreCase);
			OnLeaderChanged?.Invoke(this, new OnLeaderChangedEventArgs(isLeader, leaderName));
		}

		public bool IsThisInstanceLeader()
		{
			return _leaderElector.IsLeader();
		}

		public event EventHandler<OnLeaderChangedEventArgs>? OnLeaderChanged;

		protected override Task OnStopping(KubernetesLeaderElectionState state)
		{
			_leaderElector?.Dispose();
			_client?.Dispose();

			return Task.CompletedTask;
		}

		public override async Task<bool> OnPollAsync(KubernetesLeaderElectionState state, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Polling kubernetes to determine leadership status");

			try
			{
				await _configMapLock.GetAsync(cancellationToken);
			}
			catch (HttpOperationException e)
			{
				if (e.Response.StatusCode != HttpStatusCode.NotFound)
				{
					_logger.LogError(e, "Failed to fetch config map lock");
				}
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Failed to fetch config map lock");
			}

			await _leaderElector.RunUntilLeadershipLostAsync(cancellationToken);

			return true;
		}
	}
}
