// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.Metrics;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Implementation.Blob;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class MetricsState
	{
		public Task? CalculateMetricsTask { get; set; } = null;
	}

	public class MetricsServiceSettings
	{
		/// <summary>
		/// Set to enable calculation of metrics in the background. Adds load to database so only enable these if you intend to use it.
		/// </summary>
		public bool Enabled { get; set; } = false;

		public TimeSpan PollFrequency { get; set; } = TimeSpan.FromHours(6);
	}

	public class MetricsService : PollingService<MetricsState>
	{
		private readonly IOptionsMonitor<MetricsServiceSettings> _settings;
		private readonly IReferencesStore _referencesStore;
		private volatile bool _alreadyPolling;

		private readonly ILogger _logger;
		private readonly IServiceProvider _provider;

		public MetricsService(IOptionsMonitor<MetricsServiceSettings> settings, IReferencesStore referencesStore, ILogger<MetricsService> logger, IServiceProvider provider) : base(serviceName: nameof(MetricsService), settings.CurrentValue.PollFrequency, new MetricsState(), logger, startAtRandomTime: false)
		{
			_settings = settings;
			_referencesStore = referencesStore;
			_logger = logger;
			_provider = provider;
		}

		protected override bool ShouldStartPolling()
		{
			return _settings.CurrentValue.Enabled;
		}

		public override async Task<bool> OnPollAsync(MetricsState state, CancellationToken cancellationToken)
		{
			if (_alreadyPolling)
			{
				return false;
			}

			_alreadyPolling = true;
			try
			{
				if (!state.CalculateMetricsTask?.IsCompleted ?? false)
				{
					return false;
				}

				if (state.CalculateMetricsTask != null)
				{
					await state.CalculateMetricsTask;
				}
				state.CalculateMetricsTask = DoCalculateMetricsAsync(state, cancellationToken);

				return true;

			}
			finally
			{
				_alreadyPolling = false;
			}
		}

		private async Task DoCalculateMetricsAsync(MetricsState _, CancellationToken cancellationToken)
		{
			MetricsCalculator calculator = ActivatorUtilities.CreateInstance<MetricsCalculator>(_provider);
			_logger.LogInformation("Attempting to calculate metrics. ");
			try
			{
				await foreach (NamespaceId ns in _referencesStore.GetNamespacesAsync(cancellationToken).WithCancellation(cancellationToken))
				{
					await foreach (BucketId bucket in _referencesStore.GetBucketsAsync(ns, cancellationToken).WithCancellation(cancellationToken))
					{
						DateTime start = DateTime.UtcNow;
						_logger.LogInformation("Calculating stats for {Namespace} {Bucket}", ns, bucket);

						await calculator.CalculateStatsForBucketAsync(ns, bucket, cancellationToken);

						TimeSpan duration = DateTime.UtcNow - start;
						_logger.LogInformation("Stats calculated for {Namespace} {Bucket} took {Duration}", ns, bucket, duration);
					}
				}
			}
			catch (Exception e)
			{
				_logger.LogError("Error calculating metrics. {Exception}", e);
			}
		}
	}

	public class MetricsCalculator
	{
		private readonly IBlobIndex _blobIndex;

		private readonly ILogger _logger;
		private readonly Tracer _tracer;
		private readonly Gauge<double> _blobSizeAvgGauge;
		private readonly Gauge<long> _blobSizeMinGauge;
		private readonly Gauge<long> _blobSizeMaxGauge;
		private readonly Gauge<long> _refsInBucketGauge;
		private readonly Gauge<long> _blobSizeCountGauge;
		private readonly Gauge<long> _blobSizeTotalGauge;

		public MetricsCalculator(IBlobIndex blobIndex, Meter meter, ILogger<MetricsService> logger, Tracer tracer)
		{
			_blobIndex = blobIndex;
			_logger = logger;
			_tracer = tracer;

			_blobSizeAvgGauge = meter.CreateGauge<double>("blobstats.bucket_size.avg");
			_blobSizeMinGauge = meter.CreateGauge<long>("blobstats.bucket_size.min");
			_blobSizeMaxGauge = meter.CreateGauge<long>("blobstats.bucket_size.max");
			_blobSizeCountGauge = meter.CreateGauge<long>("blobstats.bucket_size.count");
			_blobSizeTotalGauge = meter.CreateGauge<long>("blobstats.bucket_size.sum");
			_refsInBucketGauge = meter.CreateGauge<long>("blobstats.refs_in_bucket");
		}

		public async Task<BucketStats?> CalculateStatsForBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan removeBlobScope = _tracer.StartActiveSpan("metrics.calculate")
				.SetAttribute("operation.name", "metrics.calculate")
				.SetAttribute("resource.name", $"{ns}.{bucket}");

			KeyValuePair<string, object?>[] tags = new[] { new KeyValuePair<string, object?>("Bucket", bucket.ToString()), new KeyValuePair<string, object?>("Namespace", ns.ToString()) };

			BucketStats stats = await _blobIndex.CalculateBucketStatisticsAsync(ns, bucket, cancellationToken);

			_blobSizeAvgGauge.Record(stats.AvgSize, tags);
			_blobSizeMinGauge.Record(stats.SmallestBlobFound, tags);
			_blobSizeMaxGauge.Record(stats.LargestBlob, tags);
			_blobSizeCountGauge.Record(stats.CountOfBlobs, tags);
			_refsInBucketGauge.Record(stats.CountOfRefs, tags);
			_blobSizeTotalGauge.Record(stats.TotalSize, tags);
			_logger.LogInformation("Stats calculated for {Namespace} {Bucket}. {CountOfRefs} {CountOfBlobs} {TotalSize} {AvgSize} {MaxSize} {MinSize}",
			ns, bucket, stats.CountOfRefs, stats.CountOfBlobs, stats.TotalSize, stats.AvgSize, stats.LargestBlob, stats.SmallestBlobFound);

			return stats;
		}
	}
}
