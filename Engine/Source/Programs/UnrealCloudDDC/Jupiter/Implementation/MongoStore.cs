// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Security.Authentication;
using System.Threading.Tasks;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Driver;

namespace Jupiter.Implementation;

public class MongoStore
{
	private readonly string? _overrideDatabaseName;
	private readonly MongoClient _client;

	public MongoStore(IOptionsMonitor<MongoSettings> settings, string? overrideDatabaseName = null)
	{
		_overrideDatabaseName = overrideDatabaseName;
		MongoClientSettings mongoClientSettings = MongoClientSettings.FromUrl(
			new MongoUrl(settings.CurrentValue.ConnectionString)
		);
		if (settings.CurrentValue.RequireTls12)
		{
			mongoClientSettings.SslSettings = new SslSettings { EnabledSslProtocols = SslProtocols.None };
		}

		_client = new MongoClient(mongoClientSettings);
	}

	protected async Task CreateCollectionIfNotExistsAsync<T>()
	{
		string collectionName = GetCollectionName<T>();

		IMongoDatabase database = _client.GetDatabase(GetDatabaseName());

		// Try to avoid exceptions breaking in the debugger unnecessarily by checking for the existence of the collection beforehand.
		FilterDefinition<BsonDocument> filter = new BsonDocument("name", collectionName);
		if (await (await database.ListCollectionNamesAsync(new ListCollectionNamesOptions { Filter = filter })).AnyAsync())
		{
			return;
		}

		try
		{
			await database.CreateCollectionAsync(collectionName);
		}
		catch (MongoCommandException e)
		{
			if (e.CodeName != "NamespaceExists")
			{
				throw;
			}
		}
	}

	private string GetCollectionName<T>()
	{
		object[] attr = typeof(T).GetCustomAttributes(typeof(MongoCollectionNameAttribute), true);
		foreach (MongoCollectionNameAttribute o in attr)
		{
			return o.CollectionName;
		}

		throw new ArgumentException($"No MongoCollectionNameAttribute found on type {nameof(T)}");
	}

	protected IMongoIndexManager<T> AddIndexFor<T>()
	{
		return GetCollection<T>().Indexes;
	}

	protected IMongoCollection<T> GetCollection<T>()
	{
		string collectionName = GetCollectionName<T>();
		string dbName = GetDatabaseName();
		return _client.GetDatabase(dbName).GetCollection<T>(collectionName);
	}

	internal string GetDatabaseName()
	{
		return _overrideDatabaseName ?? "Jupiter";
	}
}

[AttributeUsage(AttributeTargets.Class)]
public sealed class MongoCollectionNameAttribute : Attribute
{
	public string CollectionName { get; }

	public MongoCollectionNameAttribute(string collectionName)
	{
		CollectionName = collectionName;
	}
}
