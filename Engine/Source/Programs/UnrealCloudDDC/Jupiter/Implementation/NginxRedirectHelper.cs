// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class NginxRedirectHelper
	{
		private readonly IOptionsMonitor<NginxSettings> _nginxSettings;

		public NginxRedirectHelper(IOptionsMonitor<NginxSettings> nginxSettings)
		{
			_nginxSettings = nginxSettings;
		}

		public IActionResult CreateActionResult(BlobContents blobContents, string contentType)
		{
			if (blobContents.LocalPath == null)
			{
				throw new Exception("No local path available for blob contents when attempting to use nginx redirect");
			}

			string s = blobContents.LocalPath.Replace("\\", "/", StringComparison.OrdinalIgnoreCase);
			// dispose of the open stream because we will just use the path to it and have nginx read and send it
			blobContents.Dispose();
			return new NginxRedirectOkResult($"{_nginxSettings.CurrentValue.NginxRootPath}/{s}", contentType);
		}

		public bool CanRedirect(HttpRequest request, BlobContents blobContents)
		{
			if (!_nginxSettings.CurrentValue.UseNginxRedirect)
			{
				return false;
			}

			if (blobContents.LocalPath == null)
			{
				return false;
			}

			if (!request.Headers.ContainsKey(_nginxSettings.CurrentValue.RequiredNginxHeader))
			{
				return false;
			}

			// suppress warning about accepting paths from user requests while this is technically based on a blob name send by the user that name has to be a hash and the actual path is built from the hash
#pragma warning disable CA3003 
			if (!File.Exists(blobContents.AbsolutePath))
			{
				return false;
			}
#pragma warning restore CA3003
			return true;
		}
	}

	public class NginxSettings
	{
		public bool UseNginxRedirect { get; set; } = false;
		public string NginxRootPath { get; set; } = "/nginx-blobs";
		public string RequiredNginxHeader { get; set; } = "X-Jupiter-XAccel-Supported";
	}

	public class NginxRedirectOkResult : OkResult
	{
		private readonly string _nginxRedirectPath;
		private readonly string _contentType;

		public NginxRedirectOkResult(string nginxRedirectPath, string contentType)
		{
			_nginxRedirectPath = nginxRedirectPath;
			_contentType = contentType;
		}

		public override Task ExecuteResultAsync(ActionContext context)
		{
			context.HttpContext.Response.Headers.ContentType = _contentType;
			context.HttpContext.Response.Headers["X-Accel-Redirect"] = _nginxRedirectPath;
			// disable buffering in nginx as this is mostly used for large objects which should be streamed
			context.HttpContext.Response.Headers["X-Accel-Buffering"] = "no";
			return base.ExecuteResultAsync(context);
		}
	}
}
