// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Net.Http.Json;
using System.Net.Mime;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.AspNet;
using Jupiter.Common;
using Jupiter.Controllers;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class CacheReferencesStore : IReferencesStore
	{
		private readonly MongoReferencesStore _mongoReferenceStore;
		private readonly UpstreamReferenceStore _upstreamReferenceStore;

		public CacheReferencesStore(IServiceProvider provider)
		{
			_mongoReferenceStore = ActivatorUtilities.CreateInstance<MongoReferencesStore>(provider, "Jupiter_Cache");
			_mongoReferenceStore.SetLastAccessTTLDuration(TimeSpan.FromDays(7.0));

			_upstreamReferenceStore = ActivatorUtilities.CreateInstance<UpstreamReferenceStore>(provider);
		}

		public async Task<RefRecord> GetAsync(NamespaceId ns, BucketId bucket, RefId key, IReferencesStore.FieldFlags fieldFlags, IReferencesStore.OperationFlags opFlags, CancellationToken cancellationToken)
		{
			try
			{
				RefRecord cachedRecord = await _mongoReferenceStore.GetAsync(ns, bucket, key, fieldFlags, opFlags, cancellationToken);
				return cachedRecord;
			}
			catch (RefNotFoundException)
			{
				// not cached, we check the upstream for it
			}

			RefRecord record = await _upstreamReferenceStore.GetAsync(ns, bucket, key, fieldFlags, opFlags, cancellationToken);
			// always overwriting local value with the upstream value so that the upstream is authoritative
			await _mongoReferenceStore.PutAsync(record.Namespace, record.Bucket, record.Name, record.BlobIdentifier, record.InlinePayload, record.IsFinalized, allowOverwrite: true, cancellationToken: cancellationToken);
			return record;
		}

		public async Task PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, byte[] blob, bool isFinalized, bool allowOverwrite = false, CancellationToken cancellationToken = default)
		{
			Task cachePut = _mongoReferenceStore.PutAsync(ns, bucket, key, blobHash, blob, isFinalized, allowOverwrite: allowOverwrite, cancellationToken: cancellationToken);
			Task upstreamPut = _upstreamReferenceStore.PutAsync(ns, bucket, key, blobHash, blob, isFinalized, allowOverwrite: allowOverwrite, cancellationToken: cancellationToken);

			await Task.WhenAll(cachePut, upstreamPut);
		}

		public async Task FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobIdentifier, CancellationToken cancellationToken)
		{
			Task cacheFinalize = _mongoReferenceStore.FinalizeAsync(ns, bucket, key, blobIdentifier, cancellationToken);
			Task upstreamFinalize = _upstreamReferenceStore.FinalizeAsync(ns, bucket, key, blobIdentifier, cancellationToken);

			await Task.WhenAll(cacheFinalize, upstreamFinalize);
		}

		public async Task<DateTime?> GetLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			try
			{
				DateTime? lastAccessTimeCache = await _mongoReferenceStore.GetLastAccessTimeAsync(ns, bucket, key, cancellationToken);
				return lastAccessTimeCache;
			}
			catch (RefNotFoundException)
			{
				// not cached, we check the upstream for it
			}
			DateTime? lastAccessTime = await _upstreamReferenceStore.GetLastAccessTimeAsync(ns, bucket, key, cancellationToken);
			if (lastAccessTime.HasValue)
			{
				await _mongoReferenceStore.UpdateLastAccessTimeAsync(ns, bucket, key, lastAccessTime.Value, cancellationToken);
			}

			return lastAccessTime;
		}

		public async Task UpdateLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime newLastAccessTime, CancellationToken cancellationToken)
		{
			Task cacheUpdateLastAccess = _mongoReferenceStore.UpdateLastAccessTimeAsync(ns, bucket, key, newLastAccessTime, cancellationToken);
			Task upstreamUpdateLastAccess = _upstreamReferenceStore.UpdateLastAccessTimeAsync(ns, bucket, key, newLastAccessTime, cancellationToken);

			await Task.WhenAll(cacheUpdateLastAccess, upstreamUpdateLastAccess);
		}

		public Task CleanupTrackedStateAsync()
		{
			// nothing to cleanup
			return Task.CompletedTask;
		}

		public IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsAsync(CancellationToken cancellationToken)
		{
			throw new NotImplementedException("GetRecords not supported on a cached reference store");
		}

		public IAsyncEnumerable<(NamespaceId, BucketId, RefId)> GetRecordsWithoutAccessTimeAsync(CancellationToken cancellationToken)
		{
			throw new NotImplementedException("GetRecordsWithoutAccessTimeAsync not supported on a cached reference store");
		}

		public IAsyncEnumerable<NamespaceId> GetNamespacesAsync(CancellationToken cancellationToken)
		{
			throw new NotImplementedException("GetNamespaces not supported on a cached reference store");
		}

		public IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			throw new NotImplementedException("GetBuckets not supported on a cached reference store");
		}

		public Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			throw new NotImplementedException("Deletes are not supported on a cached reference store");
		}

		public Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			throw new NotImplementedException("DropNamespace is not supported on a cached reference store");

		}

		public Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
		{
			throw new NotImplementedException("DeleteBucket is not supported on a cached reference store");
		}

		public Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException("Update TTL is not supported on a cached reference store");
		}
	}

	class UpstreamReferenceStore : RelayStore, IReferencesStore
	{
		public UpstreamReferenceStore(IOptionsMonitor<UpstreamRelaySettings> settings, IHttpClientFactory httpClientFactory, IServiceCredentials serviceCredentials) : base(settings, httpClientFactory, serviceCredentials)
		{
		}

		public async Task<RefRecord> GetAsync(NamespaceId ns, BucketId bucket, RefId key, IReferencesStore.FieldFlags fieldFlags, IReferencesStore.OperationFlags opFlags, CancellationToken cancellationToken)
		{
			using HttpRequestMessage getObjectRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/refs/{ns}/{bucket}/{key}/metadata", UriKind.Relative));
			getObjectRequest.Headers.Add("Accept", MediaTypeNames.Application.Json);
			HttpResponseMessage response = await HttpClient.SendAsync(getObjectRequest, cancellationToken);
			if (response.StatusCode == HttpStatusCode.NotFound)
			{
				throw new RefNotFoundException(ns, bucket, key);
			}

			response.EnsureSuccessStatusCode();

			RefMetadataResponse? metadataResponse = await response.Content.ReadFromJsonAsync<RefMetadataResponse>(cancellationToken);
			if (metadataResponse == null)
			{
				throw new Exception("Unable to deserialize metadata response");
			}
			return new RefRecord(metadataResponse.Ns, metadataResponse.Bucket, metadataResponse.Name, metadataResponse.LastAccess, metadataResponse.InlinePayload, metadataResponse.PayloadIdentifier, metadataResponse.IsFinalized);
		}

		public async Task PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, byte[] blob, bool isFinalized, bool allowOverwrite = false, CancellationToken cancellationToken = default)
		{
			using HttpRequestMessage putObjectRequest = await BuildHttpRequestAsync(HttpMethod.Put, new Uri($"api/v1/refs/{ns}/{bucket}/{key}", UriKind.Relative));
			putObjectRequest.Headers.Add("Accept", MediaTypeNames.Application.Json);
			putObjectRequest.Headers.Add(CommonHeaders.HashHeaderName, blobHash.ToString());
			putObjectRequest.Content = new ByteArrayContent(blob);
			// a blob sent to the reference store is always by definition a compact binary, so we upload it as such to avoid any automatic conversions that we would get with octet-stream
			putObjectRequest.Content.Headers.ContentType = new MediaTypeHeaderValue(CustomMediaTypeNames.UnrealCompactBinary);

			HttpResponseMessage response = await HttpClient.SendAsync(putObjectRequest, cancellationToken);
			if (response.StatusCode == HttpStatusCode.BadRequest)
			{
				ProblemDetails? problem = await response.Content.ReadFromJsonAsync<ProblemDetails>(cancellationToken);
				if (problem == null)
				{
					throw new Exception("Unable to deserialize problem details when bad request was encountered");
				}
				throw new Exception("Upstream returned 400 (BadRequest) with error: " + problem.Title);
			}
			response.EnsureSuccessStatusCode();

			// if this put returns needs, we cant really do anything about it. we should be calling put in the upstream blob store as the operation continues and we should be able to catch the missing blobs during the finalize call
		}

		public async Task FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobIdentifier, CancellationToken cancellationToken)
		{
			using HttpRequestMessage putObjectRequest = await BuildHttpRequestAsync(HttpMethod.Post, new Uri($"api/v1/refs/{ns}/{bucket}/{key}/finalize/{blobIdentifier}", UriKind.Relative));
			putObjectRequest.Headers.Add("Accept", MediaTypeNames.Application.Json);

			HttpResponseMessage response = await HttpClient.SendAsync(putObjectRequest, cancellationToken);

			if (response.StatusCode == HttpStatusCode.BadRequest)
			{
				ProblemDetails? problem = await response.Content.ReadFromJsonAsync<ProblemDetails>(cancellationToken);
				if (problem == null)
				{
					throw new Exception("Unable to deserialize problem details when bad request was encountered during finalize");
				}
				throw new Exception("Upstream returned 400 (BadRequest) with error: " + problem.Title);
			}
			response.EnsureSuccessStatusCode();

			PutObjectResponse? putObjectResponse = await response.Content.ReadFromJsonAsync<PutObjectResponse>(cancellationToken);
			if (putObjectResponse == null)
			{
				throw new Exception("Unable to deserialize put object response when finalizing ref");
			}

			if (putObjectResponse.Needs.Length != 0)
			{
				// TODO: we assume here that all the objects that are missing are content ids, that may not be true, but as needs only contains a list with mixed hashes we can not detect the difference
				throw new PartialReferenceResolveException(putObjectResponse.Needs.Select(hash => new ContentId(hash.HashData)).ToList());
			}
		}

		public async Task<DateTime?> GetLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			try
			{
				RefRecord foo = await GetAsync(ns, bucket, key, IReferencesStore.FieldFlags.None, IReferencesStore.OperationFlags.None, cancellationToken);
				return foo.LastAccess;
			}
			catch (RefNotFoundException)
			{
				return null;
			}
		}

		public async Task UpdateLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime newLastAccessTime, CancellationToken cancellationToken)
		{
			// there is no endpoint to update last access time externally, but fetching a object will in turn update its last access time, though to a different time then newLastAccessTime
			await GetAsync(ns, bucket, key, IReferencesStore.FieldFlags.None, IReferencesStore.OperationFlags.None, cancellationToken);
		}

		public Task CleanupTrackedStateAsync()
		{
			// nothing to clean up
			return Task.CompletedTask;
		}

		public IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsAsync(CancellationToken cancellationToken)
		{
			throw new NotImplementedException("GetRecords is not supported on a upstream reference store");
		}

		public IAsyncEnumerable<(NamespaceId, BucketId, RefId)> GetRecordsWithoutAccessTimeAsync(CancellationToken cancellationToken)
		{
			throw new NotImplementedException("GetRecordsWithoutAccessTimeAsync is not supported on a upstream reference store");
		}

		public IAsyncEnumerable<NamespaceId> GetNamespacesAsync(CancellationToken cancellationToken)
		{
			throw new NotImplementedException("GetNamespaces is not supported on a upstream reference store");
		}

		public IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			throw new NotImplementedException("GetBuckets is not supported on a upstream reference store");
		}

		public Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			throw new NotImplementedException("Delete is not supported on a upstream reference store");
		}

		public Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			throw new NotImplementedException("DropNamespace is not supported on a upstream reference store");
		}

		public Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
		{
			throw new NotImplementedException("DeleteBucket is not supported on a upstream reference store");
		}

		public Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException("UpdateTTL is not supported on a upstream reference store");
		}
	}
}
