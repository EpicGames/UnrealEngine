// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Jupiter.Common;
using Jupiter.DataStore;

namespace Jupiter.Implementation;

public class DataStoreReferencesStore : IReferencesStore
{
	private readonly INamespacePolicyResolver _namespacePolicyResolver;
	private readonly DataStoreTable<DataStoreRefRecord> _refTable;

	public DataStoreReferencesStore(IDataStoreTableService tableService, INamespacePolicyResolver namespacePolicyResolver)
	{
		_namespacePolicyResolver = namespacePolicyResolver;
		_refTable = tableService.GetTableByType<DataStoreRefRecord>();
	}

	public async Task<RefRecord> GetAsync(NamespaceId ns, BucketId bucket, RefId key, IReferencesStore.FieldFlags fieldFlags, IReferencesStore.OperationFlags opFlags, CancellationToken cancellationToken = default)
	{
		DataStorePartitionValue? value = await _refTable.FindAsync(GetKeyspace(ns), bucket.ToString(), DataStoreRefRecord.GetPartitionKey(key), cancellationToken);

		if (value == null)
		{
			throw new RefNotFoundException(ns, bucket, key);
		}
		value.AsObject(out DataStoreRefRecord record);
		return record.ToRefRecord(ns, bucket, key, value.IsFinalized, fieldFlags);
	}

	public async Task PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, byte[] blob, bool isFinalized, bool allowOverwrite = false, CancellationToken cancellationToken = default)
	{
		NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
		bool allowOverwrites = policy.AllowOverwritesOfRefs || allowOverwrite;

		KeyspaceId keyspace = GetKeyspace(ns);
		DataStorePartitionValue? partitionValue = await _refTable.FindAsync(keyspace, bucket.ToString(), DataStoreRefRecord.GetPartitionKey(key), cancellationToken);
		if (!allowOverwrites)
		{
			if (partitionValue != null)
			{
				partitionValue.AsObject(out DataStoreRefRecord oldRecord);
				if (!oldRecord.BlobHash.Equals(blobHash.AsIoHash()))
				{
					// blob was not the same, e.g. we attempted to change the value, this is not allowed
					throw new RefAlreadyExistsException(ns, bucket, key, oldRecord.ToRefRecord(ns, bucket, key, isFinalized: partitionValue.IsFinalized));
				}
			}
		}

		await _refTable.InsertAsync(keyspace, bucket.ToString(), DataStoreRefRecord.GetPartitionKey(key), new DataStoreRefRecord(blobHash, blob), isFinalized, cancellationToken);
	}

	public async Task FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobIdentifier, CancellationToken cancellationToken = default)
	{
		KeyspaceId keyspace = GetKeyspace(ns);
		DataStorePartitionValue? value = await _refTable.FindAsync(keyspace, bucket.ToString(), DataStoreRefRecord.GetPartitionKey(key), cancellationToken);
		if (value == null)
		{
			throw new Exception("value not available when finalizing");
		}

		value.AsObject(out DataStoreRefRecord record);
		await _refTable.InsertAsync(keyspace, bucket.ToString(), DataStoreRefRecord.GetPartitionKey(key), record, true, cancellationToken);
	}

	public Task<DateTime?> GetLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken = default)
	{
		// we do not track last access time
		return Task.FromResult((DateTime?)null);
	}

	public Task UpdateLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime newLastAccessTime, CancellationToken cancellationToken = default)
	{
		// no op - we do not track last access time
		return Task.CompletedTask;
	}

	public async IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
	{
		foreach (KeyspaceId keyspace in _refTable.GetKeyspacesAsync(cancellationToken))
		{
			NamespaceId ns = keyspace.ToNamespaceId();
			foreach (string prefix in _refTable.GetPrefixesForNamespaceAsync(GetKeyspace(ns), cancellationToken))
			{
				BucketId bucket = new BucketId(prefix);
				await foreach ((DataStorePartitionKey key, DataStorePartitionValue _) in _refTable.EnumerateRecordsAsync(GetKeyspace(ns), prefix, cancellationToken))
				{
					yield return (ns, bucket, new RefId(key.Bytes), DateTime.MinValue);
				}
			}
		}
	}

	public async IAsyncEnumerable<(NamespaceId, BucketId, RefId)> GetRecordsWithoutAccessTimeAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
	{
		foreach (KeyspaceId keyspace in _refTable.GetKeyspacesAsync(cancellationToken))
		{
			NamespaceId ns = keyspace.ToNamespaceId();
			foreach (string prefix in _refTable.GetPrefixesForNamespaceAsync(GetKeyspace(ns), cancellationToken))
			{
				BucketId bucket = new BucketId(prefix);
				await foreach((DataStorePartitionKey key, DataStorePartitionValue _) in _refTable.EnumerateRecordsAsync(keyspace, prefix, cancellationToken))
				{
					yield return (ns, bucket, new RefId(key.Bytes));
				}
			}
		}
	}

	public IAsyncEnumerable<NamespaceId> GetNamespacesAsync(CancellationToken cancellationToken = default)
	{
		return _refTable.GetKeyspacesAsync(cancellationToken).Select(k => k.ToNamespaceId()).ToAsyncEnumerable();
	}

	public IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, CancellationToken cancellationToken = default)
	{
		return _refTable.GetPrefixesForNamespaceAsync(GetKeyspace(ns), cancellationToken).Select(p => new BucketId(p)).ToAsyncEnumerable();
	}

	public async Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken = default)
	{
		await _refTable.DeleteAsync(GetKeyspace(ns), bucket.ToString(), DataStoreRefRecord.GetPartitionKey(key), cancellationToken);

		return true;
	}

	public async Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default)
	{
		return await _refTable.DropKeyspaceAsync(GetKeyspace(ns), cancellationToken);
	}

	public Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken = default)
	{
		return _refTable.DropPrefixAsync(GetKeyspace(ns), bucket.ToString(), cancellationToken);
	}

	public Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task CleanupTrackedStateAsync()
	{
		return Task.CompletedTask;
	}

	private static KeyspaceId GetKeyspace(NamespaceId ns)
	{
		return KeyspaceId.FromNamespace(ns);
	}
}

internal record struct DataStoreRefRecord : IDataStoreTableObject<DataStoreRefRecord>
{
	public IoHash BlobHash { get; set; }

	public byte[] Blob { get; set; }

	public DataStoreRefRecord(BlobId blobHash, byte[] blob)
	{
		BlobHash = blobHash.AsIoHash();
		Blob = blob;
	}

	public static DataStorePartitionKey GetPartitionKey(RefId refId)
	{
		return new DataStorePartitionKey(StringUtils.ToHashFromHexString(refId.ToString()));
	}

	public static DataStorePartitionKey? MaybeGetPartitionKey(RefId? refId)
	{
		if (refId == null)
		{
			return null;
		}
		return new DataStorePartitionKey(StringUtils.ToHashFromHexString(refId.ToString()!));
	}

	public RefRecord ToRefRecord(NamespaceId ns, BucketId bucket, RefId key, bool isFinalized, IReferencesStore.FieldFlags fieldFlags = IReferencesStore.FieldFlags.All)
	{
		RefRecord record = new RefRecord(ns, bucket,key, DateTime.Now, Blob, BlobId.FromIoHash(BlobHash), isFinalized: isFinalized);

		if (!fieldFlags.HasFlag(IReferencesStore.FieldFlags.IncludePayload))
		{
			record.InlinePayload = null;
		}
		return record;
	}

	public static byte[] Serialize(DataStoreRefRecord self)
	{
		byte[] payload = new byte[20 + 4 + self.Blob.Length];
		MemoryWriter writer = new MemoryWriter(payload);

		writer.WriteFixedLengthBytes(self.BlobHash.ToByteArray().AsSpan()[..20]);
		writer.WriteInt32(self.Blob.Length);
		writer.WriteFixedLengthBytes(self.Blob);

		return payload;
	}

	public static void Deserialize(byte[] payload, out DataStoreRefRecord record)
	{
		MemoryReader reader = new MemoryReader(payload);
		BlobId hash = new BlobId(reader.ReadFixedLengthBytes(20).ToArray());
		int blobLength = reader.ReadInt32();
		Debug.Assert(reader.RemainingMemory.Length == blobLength);
		byte[] blob = reader.ReadFixedLengthBytes(blobLength).ToArray();

		record = new DataStoreRefRecord(hash, blob);
	}

	public static bool HasSortKey()
	{
		return false;
	}

	public static byte[] GetSortKey(DataStoreRefRecord self)
	{
		return null!;
	}
}