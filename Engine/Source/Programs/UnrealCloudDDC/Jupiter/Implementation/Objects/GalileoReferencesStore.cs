// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace Jupiter.Implementation
{
	/// <summary>
	/// Temporary storage layer for coordinating data between scylla and data store
	/// </summary>
	public class GalileoReferencesStore : IReferencesStore
	{
		private readonly ILogger<GalileoReferencesStore> _logger;
		private readonly ScyllaReferencesStore _scyllaReferenceStore;
		private readonly DataStoreReferencesStore _datastoreReferenceStore;

		public GalileoReferencesStore(IServiceProvider provider, ILogger<GalileoReferencesStore> logger)
		{
			_logger = logger;
			_scyllaReferenceStore = ActivatorUtilities.CreateInstance<ScyllaReferencesStore>(provider);
			_datastoreReferenceStore = ActivatorUtilities.CreateInstance<DataStoreReferencesStore>(provider);
		}

		public async Task<RefRecord> GetAsync(NamespaceId ns, BucketId bucket, RefId key, IReferencesStore.FieldFlags fieldFlags, IReferencesStore.OperationFlags opFlags, CancellationToken cancellationToken = default)
		{
			// fetch the datastore record but ignore it, just making sure its present
			Task datastoreGet = _datastoreReferenceStore.GetAsync(ns, bucket, key, fieldFlags, opFlags, cancellationToken);

			RefRecord refRecord = await _scyllaReferenceStore.GetAsync(ns, bucket, key, fieldFlags, opFlags, cancellationToken);

			try
			{
				// TODO: Do more to verify if the two stores are in sync
				await datastoreGet;
			}
			catch (RefNotFoundException)
			{
				// ignore missing refs
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Unhandled exception from datastore reference store, this will be ignored.");
			}

			return refRecord;
		}

		public async Task PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, byte[] blob, bool isFinalized, bool allowOverwrite = false, CancellationToken cancellationToken = default)
		{
			// forward all writes to datastore but keep everything in scylla as well so that we can read from there
			Task datastorePut = _datastoreReferenceStore.PutAsync(ns, bucket, key, blobHash, blob, isFinalized, allowOverwrite, cancellationToken);

			await _scyllaReferenceStore.PutAsync(ns, bucket, key, blobHash, blob, isFinalized, allowOverwrite, cancellationToken);

			try
			{
				await datastorePut;
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Unhandled exception from datastore reference store, this will be ignored.");
			}
		}

		public async Task FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobIdentifier, CancellationToken cancellationToken = default)
		{
			// forward all writes to datastore but keep everything in scylla as well so that we can read from there
			Task datastoreFinalize = _datastoreReferenceStore.FinalizeAsync(ns, bucket, key, blobIdentifier, cancellationToken);

			await _scyllaReferenceStore.FinalizeAsync(ns, bucket, key, blobIdentifier, cancellationToken);

			try
			{
				await datastoreFinalize;
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Unhandled exception from datastore reference store, this will be ignored.");
			}
		}

		public Task<DateTime?> GetLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.GetLastAccessTimeAsync(ns, bucket, key, cancellationToken);
		}

		public Task UpdateLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime newLastAccessTime, CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.UpdateLastAccessTimeAsync(ns, bucket, key, newLastAccessTime, cancellationToken);
		}

		public IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsAsync(CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.GetRecordsAsync(cancellationToken);
		}

		public IAsyncEnumerable<(NamespaceId, BucketId, RefId)> GetRecordsWithoutAccessTimeAsync(CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.GetRecordsWithoutAccessTimeAsync(cancellationToken);
		}

		public IAsyncEnumerable<NamespaceId> GetNamespacesAsync(CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.GetNamespacesAsync(cancellationToken);
		}

		public IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.GetBucketsAsync(ns, cancellationToken);
		}

		public Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.DeleteAsync(ns, bucket, key, cancellationToken);
		}

		public Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.DropNamespaceAsync(ns, cancellationToken);
		}

		public Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.DeleteBucketAsync(ns, bucket, cancellationToken);
		}

		public Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default)
		{
			return _scyllaReferenceStore.UpdateTTL(ns, bucket, refId, ttl, cancellationToken);
		}

		public Task CleanupTrackedStateAsync()
		{
			return _scyllaReferenceStore.CleanupTrackedStateAsync();
		}
	}
}
