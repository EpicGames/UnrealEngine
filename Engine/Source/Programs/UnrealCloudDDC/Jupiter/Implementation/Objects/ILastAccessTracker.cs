// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;

namespace Jupiter.Implementation
{
	public interface ILastAccessTracker<T>
	{
		Task TrackUsed(T record);
	}

	public interface ILastAccessCache<T>
	{
		Task<List<(T, DateTime)>> GetLastAccessedRecords();
	}
}
