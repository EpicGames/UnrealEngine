// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation
{
	public interface IReferencesStore
	{
		Task<RefRecord> GetAsync(NamespaceId ns, BucketId bucket, RefId key, FieldFlags fieldFlags, OperationFlags opFlags, CancellationToken cancellationToken = default);

		[Flags]
		public enum FieldFlags
		{
			None = 0,
			IncludePayload = 1,
			All = IncludePayload
		}

		[Flags]
		public enum OperationFlags
		{
			None = 0,
			BypassCache = 1
		}

		Task PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, byte[] blob, bool isFinalized, bool allowOverwrite = false, CancellationToken cancellationToken = default);
		Task FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobIdentifier, CancellationToken cancellationToken = default);

		Task<DateTime?> GetLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken = default);
		Task UpdateLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime newLastAccessTime, CancellationToken cancellationToken = default);
		IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsAsync(CancellationToken cancellationToken = default);

		IAsyncEnumerable<(NamespaceId, BucketId, RefId)> GetRecordsWithoutAccessTimeAsync(CancellationToken cancellationToken = default);

		IAsyncEnumerable<NamespaceId> GetNamespacesAsync(CancellationToken cancellationToken = default);
		IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, CancellationToken cancellationToken = default);
		Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken = default);
		Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken = default);
		Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken = default);
		Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default);
		Task CleanupTrackedStateAsync();
	}

	public class RefRecord
	{
		public RefRecord(NamespaceId ns, BucketId bucket, RefId name, DateTime lastAccess, byte[]? inlinePayload, BlobId blobIdentifier, bool isFinalized)
		{
			Namespace = ns;
			Bucket = bucket;
			Name = name;
			LastAccess = lastAccess;
			InlinePayload = inlinePayload;
			BlobIdentifier = blobIdentifier;
			IsFinalized = isFinalized;
		}

		public NamespaceId Namespace { get; }
		public BucketId Bucket { get; }
		public RefId Name { get; }
		public DateTime LastAccess { get; }
		public byte[]? InlinePayload { get; set; }
		public BlobId BlobIdentifier { get; set; }
		public bool IsFinalized { get; }
	}

	public class RefNotFoundException : Exception
	{
		public RefNotFoundException(NamespaceId ns, BucketId bucket, RefId key) : base($"Object not found {key} in bucket {bucket} namespace {ns}")
		{
			Namespace = ns;
			Bucket = bucket;
			Key = key;
		}

		public NamespaceId Namespace { get; }
		public BucketId Bucket { get; }
		public RefId Key { get; }
	}

	public class RefAlreadyExistsException : Exception
	{
		public RefAlreadyExistsException(NamespaceId ns, BucketId bucket, RefId key, RefRecord oldObject) : base($"Object already found {key} in bucket {bucket} namespace {ns}")
		{
			Namespace = ns;
			Bucket = bucket;
			Key = key;
			OldObject = oldObject;
		}

		public RefRecord OldObject { get; set; }

		public NamespaceId Namespace { get; }
		public BucketId Bucket { get; }
		public RefId Key { get; }
	}
}
