// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation.Objects
{
	public class MemoryCachedReferencesStore : IReferencesStore
	{
		private readonly IReferencesStore _actualStore;
		private readonly ConcurrentDictionary<NamespaceId, MemoryCache> _referenceCaches = new ConcurrentDictionary<NamespaceId, MemoryCache>();
		private readonly IOptionsMonitor<MemoryCacheReferencesSettings> _options;
		private readonly Tracer _tracer;

		public MemoryCachedReferencesStore(IReferencesStore actualStore, IOptionsMonitor<MemoryCacheReferencesSettings> options, Tracer tracer)
		{
			_actualStore = actualStore;
			_options = options;
			_tracer = tracer;
		}

		private void AddCacheEntry(NamespaceId ns, BucketId bucket, RefId key, RefRecord record)
		{
			MemoryCache cache = GetCacheForNamespace(ns);

			CachedReferenceEntry cachedEntry = new CachedReferenceEntry(record);

			using ICacheEntry entry = cache.CreateEntry(new CachedReferenceKey(bucket, key));
			entry.Value = cachedEntry;
			entry.Size = cachedEntry.Size;

			if (_options.CurrentValue.EnableSlidingExpiry)
			{
				entry.SlidingExpiration = TimeSpan.FromMinutes(_options.CurrentValue.SlidingExpirationMinutes);
			}
		}

		private MemoryCache GetCacheForNamespace(NamespaceId ns)
		{
			return _referenceCaches.GetOrAdd(ns, id => new MemoryCache(_options.CurrentValue));
		}

		public async Task<RefRecord> GetAsync(NamespaceId ns, BucketId bucket, RefId key, IReferencesStore.FieldFlags fieldFlags, IReferencesStore.OperationFlags opFlags, CancellationToken cancellationToken)
		{
			if (opFlags.HasFlag(IReferencesStore.OperationFlags.BypassCache))
			{
				return await _actualStore.GetAsync(ns, bucket, key, IReferencesStore.FieldFlags.All, opFlags, cancellationToken);
			}

			using TelemetrySpan scope = _tracer.StartActiveSpan("Ref.get")
				.SetAttribute("operation.name", "Ref.get")
				.SetAttribute("resource.name", $"{bucket}.{key}");

			MemoryCache cache = GetCacheForNamespace(ns);

			if (cache.TryGetValue(new CachedReferenceKey(bucket, key), out CachedReferenceEntry? cachedResult))
			{
				scope.SetAttribute("Found", true);
				scope.SetAttribute("BlobIdentifier", cachedResult!.BlobIdentifier.ToString());
				RefRecord record = cachedResult.ToRefRecord(fieldFlags);
				if (record.IsFinalized)
				{
					// only return finalized records
					return record;
				}
			}

			scope.SetAttribute("Found", false);
			RefRecord objectRecord = await _actualStore.GetAsync(ns, bucket, key, IReferencesStore.FieldFlags.All, opFlags, cancellationToken);
			AddCacheEntry(ns, bucket, key, objectRecord);

			return objectRecord;
		}

		public Task PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, byte[] blob, bool isFinalized, bool allowOverwrite = false, CancellationToken cancellationToken = default)
		{
			RefRecord objectRecord = new RefRecord(ns, bucket, key, DateTime.Now, blob, blobHash, isFinalized);
			AddCacheEntry(ns, bucket, key, objectRecord);

			return _actualStore.PutAsync(ns, bucket, key, blobHash, blob, isFinalized, allowOverwrite: allowOverwrite, cancellationToken: cancellationToken);
		}

		public Task FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobIdentifier, CancellationToken cancellationToken)
		{
			FinalizeCacheEntry(ns, bucket, key);
			return _actualStore.FinalizeAsync(ns, bucket, key, blobIdentifier, cancellationToken);
		}

		public Task<DateTime?> GetLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			return _actualStore.GetLastAccessTimeAsync(ns, bucket, key, cancellationToken);
		}

		private void FinalizeCacheEntry(NamespaceId ns, BucketId bucket, RefId key)
		{
			MemoryCache cache = GetCacheForNamespace(ns);

			if (cache.TryGetValue(new CachedReferenceKey(bucket, key), out CachedReferenceEntry? result))
			{
				result!.IsFinalized = true;
			}
		}

		public Task UpdateLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime newLastAccessTime, CancellationToken cancellationToken)
		{
			return _actualStore.UpdateLastAccessTimeAsync(ns, bucket, key, newLastAccessTime, cancellationToken);
		}

		public IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsAsync(CancellationToken cancellationToken)
		{
			return _actualStore.GetRecordsAsync(cancellationToken);
		}

		public IAsyncEnumerable<(NamespaceId, BucketId, RefId)> GetRecordsWithoutAccessTimeAsync(CancellationToken cancellationToken)
		{
			return _actualStore.GetRecordsWithoutAccessTimeAsync(cancellationToken);
		}

		public IAsyncEnumerable<NamespaceId> GetNamespacesAsync(CancellationToken cancellationToken)
		{
			return _actualStore.GetNamespacesAsync(cancellationToken);
		}

		public IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			return _actualStore.GetBucketsAsync(ns, cancellationToken);
		}

		public Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			MemoryCache cache = GetCacheForNamespace(ns);
			cache.Remove(new CachedReferenceKey(bucket, key));
			return _actualStore.DeleteAsync(ns, bucket, key, cancellationToken);
		}

		public Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			_referenceCaches.TryRemove(ns, out _);
			return _actualStore.DropNamespaceAsync(ns, cancellationToken);
		}

		public Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
		{
			// we do not track enough information to be able to drop a bucket, so we have to drop the entire namespace cache to remove the bucket
			// this should be okay as deleting buckets is a extremely uncommon operation
			_referenceCaches.TryRemove(ns, out _);
			return _actualStore.DeleteBucketAsync(ns, bucket, cancellationToken);
		}

		public Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default)
		{
			return _actualStore.UpdateTTL(ns, bucket, refId, ttl, cancellationToken);
		}

		public void Clear()
		{
			_referenceCaches.Clear();
		}

		public Task CleanupTrackedStateAsync()
		{
			return _actualStore.CleanupTrackedStateAsync();
		}

		public IReferencesStore GetUnderlyingStore()
		{
			return _actualStore;
		}
	}

	class CachedReferenceKey : IEquatable<CachedReferenceKey>
	{
		private readonly BucketId _bucket;
		private readonly RefId _key;

		public CachedReferenceKey(BucketId bucket, RefId key)
		{
			_bucket = bucket;
			_key = key;
		}

		public bool Equals(CachedReferenceKey? other)
		{
			if (ReferenceEquals(null, other))
			{
				return false;
			}

			if (ReferenceEquals(this, other))
			{
				return true;
			}

			return _bucket.Equals(other._bucket) && _key.Equals(other._key);
		}

		public override bool Equals(object? obj)
		{
			if (ReferenceEquals(null, obj))
			{
				return false;
			}

			if (ReferenceEquals(this, obj))
			{
				return true;
			}

			if (obj.GetType() != GetType())
			{
				return false;
			}

			return Equals((CachedReferenceKey)obj);
		}

		public override int GetHashCode()
		{
			return HashCode.Combine(_bucket, _key);
		}
	}

	class CachedReferenceEntry
	{
		private int GetSize()
		{
			return Namespace.Text.Text.Length + Bucket.ToString().Length + 20 + Blob?.Length ?? 0 + 20 + 4;
		}

		public CachedReferenceEntry(RefRecord record)
		{
			Namespace = record.Namespace;
			Bucket = record.Bucket;
			Name = record.Name;
			Blob = record.InlinePayload;
			BlobIdentifier = record.BlobIdentifier;
			IsFinalized = record.IsFinalized;
			Size = GetSize();
		}

		public NamespaceId Namespace { get; }
		public BucketId Bucket { get; }
		public RefId Name { get; }
		public byte[]? Blob { get; }
		public BlobId BlobIdentifier { get; }
		public int Size { get; }
		public bool IsFinalized { get; set; }

		public RefRecord ToRefRecord(IReferencesStore.FieldFlags fieldFlags)
		{
			return new RefRecord(Namespace, Bucket, Name, DateTime.Now, (fieldFlags & IReferencesStore.FieldFlags.IncludePayload) != 0 ? Blob : null, BlobIdentifier, IsFinalized);
		}
	}
}
