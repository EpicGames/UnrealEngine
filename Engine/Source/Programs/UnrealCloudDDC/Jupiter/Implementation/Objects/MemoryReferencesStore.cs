// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation
{
	public class MemoryReferencesStore : IReferencesStore
	{
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly ConcurrentDictionary<string, MemoryStoreObject> _objects = new ConcurrentDictionary<string, MemoryStoreObject>();
		private readonly HashSet<NamespaceId> _namespaces = new HashSet<NamespaceId>();

		public MemoryReferencesStore(INamespacePolicyResolver namespacePolicyResolver)
		{
			_namespacePolicyResolver = namespacePolicyResolver;
		}

		public Task<RefRecord> GetAsync(NamespaceId ns, BucketId bucket, RefId key, IReferencesStore.FieldFlags fieldFlags, IReferencesStore.OperationFlags opFlags, CancellationToken cancellationToken)
		{
			if (_objects.TryGetValue(BuildKey(ns, bucket, key), out MemoryStoreObject? o))
			{
				return Task.FromResult(o.ToRefRecord(fieldFlags));
			}

			throw new RefNotFoundException(ns, bucket, key);
		}

		public Task PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, byte[] blob, bool isFinalized, bool allowOverwrite = false, CancellationToken cancellationToken = default)
		{
			lock (_namespaces)
			{
				_namespaces.Add(ns);
			}

			NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
			bool allowOverwrites = policy.AllowOverwritesOfRefs || allowOverwrite;

			MemoryStoreObject o = _objects.AddOrUpdate(BuildKey(ns, bucket, key),
				s => new MemoryStoreObject(ns, bucket, key, blobHash, blob, isFinalized),
				(s, o) =>
				{
					if (allowOverwrites)
					{
						return new MemoryStoreObject(ns, bucket, key, blobHash, blob, isFinalized);
					}
					else
					{
						// check to see if we are trying to mutate the ref
						if (!o.BlobHash.Equals(blobHash))
						{
							throw new RefAlreadyExistsException(ns, bucket, key, o.ToRefRecord(IReferencesStore.FieldFlags.All));
						}

						return o;
					}
				});

			return Task.FromResult(o);
		}

		public Task FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobIdentifier, CancellationToken cancellationToken)
		{
			if (!_objects.TryGetValue(BuildKey(ns, bucket, key), out MemoryStoreObject? o))
			{
				throw new RefNotFoundException(ns, bucket, key);
			}

			o.FinalizeObject();
			return Task.CompletedTask;
		}

		public Task<DateTime?> GetLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			DateTime? lastAccessTime = null;
			if (_objects.TryGetValue(BuildKey(ns, bucket, key), out MemoryStoreObject? o))
			{
				lastAccessTime = o.LastAccessTime;
			}

			return Task.FromResult(lastAccessTime);
		}

		public Task UpdateLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime lastAccessTime, CancellationToken cancellationToken)
		{
			if (!_objects.TryGetValue(BuildKey(ns, bucket, key), out MemoryStoreObject? o))
			{
				throw new RefNotFoundException(ns, bucket, key);
			}

			o.SetLastAccessTime(lastAccessTime);
			return Task.CompletedTask;
		}

		public async IAsyncEnumerable<(NamespaceId, BucketId, RefId)> GetRecordsWithoutAccessTimeAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			await foreach ((NamespaceId namespaceId, BucketId bucketId, RefId refId, DateTime _) in GetRecordsAsync(cancellationToken))
			{
				await Task.CompletedTask;
				yield return (namespaceId, bucketId, refId);
			}
		}

		public async IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			foreach (MemoryStoreObject o in _objects.Values.OrderBy(o => o.LastAccessTime))
			{
				await Task.CompletedTask;
				yield return (o.Namespace, o.Bucket, o.Name, o.LastAccessTime);
			}
		}

		public IAsyncEnumerable<NamespaceId> GetNamespacesAsync(CancellationToken cancellationToken)
		{
			return _namespaces.ToAsyncEnumerable();
		}

		public async IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			HashSet<BucketId> buckets = new HashSet<BucketId>();
			foreach (MemoryStoreObject o in _objects.Values.Where(o => o.Namespace == ns))
			{
				buckets.Add(o.Bucket);
			}
			await Task.CompletedTask;

			foreach (BucketId bucket in buckets)
			{
				yield return bucket;
			}
		}

		public Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			if (!_objects.TryRemove(BuildKey(ns, bucket, key), out MemoryStoreObject? _))
			{
				throw new RefNotFoundException(ns, bucket, key);
			}

			return Task.FromResult(true);
		}

		public Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			lock (_namespaces)
			{
				_namespaces.Remove(ns);
			}

			List<string> objectToRemove = new List<string>();

			foreach (MemoryStoreObject o in _objects.Values)
			{
				if (o.Namespace == ns)
				{
					objectToRemove.Add(BuildKey(o.Namespace, o.Bucket, o.Name));
				}
			}

			long removedCount = 0L;
			foreach (string key in objectToRemove)
			{
				if (_objects.TryRemove(key, out _))
				{
					removedCount++;
				}
			}

			return Task.FromResult(removedCount);
		}

		public Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
		{
			List<string> objectToRemove = new List<string>();

			foreach (MemoryStoreObject o in _objects.Values)
			{
				if (o.Namespace == ns && o.Bucket == bucket)
				{
					objectToRemove.Add(BuildKey(o.Namespace, o.Bucket, o.Name));
				}
			}

			long removedCount = 0L;
			foreach (string key in objectToRemove)
			{
				if (_objects.TryRemove(key, out _))
				{
					removedCount++;
				}
			}

			return Task.FromResult(removedCount);
		}

		public Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default)
		{
			// we do not support ttl in memory stores
			return Task.CompletedTask;
		}

		public Task CleanupTrackedStateAsync()
		{
			// nothing to cleanup
			return Task.CompletedTask;
		}

		private static string BuildKey(NamespaceId ns, BucketId bucket, RefId name)
		{
			return $"{ns}.{bucket}.{name}";
		}
	}

	public class MemoryStoreObject
	{
		public MemoryStoreObject(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, byte[] blob, bool isFinalized)
		{
			Namespace = ns;
			Bucket = bucket;
			Name = key;
			BlobHash = blobHash;
			Blob = blob;
			IsFinalized = isFinalized;
			LastAccessTime = DateTime.Now;
		}

		public NamespaceId Namespace { get; }
		public BucketId Bucket { get; }
		public RefId Name { get; }
		public byte[] Blob { get; }
		public BlobId BlobHash { get; }

		public DateTime LastAccessTime { get; private set; }
		public bool IsFinalized { get; private set; }

		public void FinalizeObject()
		{
			IsFinalized = true;
		}
		public void SetLastAccessTime(DateTime lastAccessTime)
		{
			LastAccessTime = lastAccessTime;
		}

		public RefRecord ToRefRecord(IReferencesStore.FieldFlags fieldFlags)
		{
			bool includePayload = (fieldFlags & IReferencesStore.FieldFlags.IncludePayload) != 0;
			return new RefRecord(Namespace, Bucket, Name, LastAccessTime, includePayload ? Blob : null, BlobHash, IsFinalized);
		}
	}
}
