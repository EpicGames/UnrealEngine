// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Options;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace Jupiter.Implementation
{
	public class MongoReferencesStore : MongoStore, IReferencesStore
	{
		private readonly INamespacePolicyResolver _namespacePolicyResolver;

		public MongoReferencesStore(IOptionsMonitor<MongoSettings> settings, INamespacePolicyResolver namespacePolicyResolver, string? overrideDatabaseName = null) : base(settings, overrideDatabaseName)
		{
			_namespacePolicyResolver = namespacePolicyResolver;
			CreateCollectionIfNotExistsAsync<MongoReferencesModelV0>().Wait();
			CreateCollectionIfNotExistsAsync<MongoNamespacesModelV0>().Wait();

			IndexKeysDefinitionBuilder<MongoReferencesModelV0> indexKeysDefinitionBuilder = Builders<MongoReferencesModelV0>.IndexKeys;
			CreateIndexModel<MongoReferencesModelV0> indexModelClusteredKey = new CreateIndexModel<MongoReferencesModelV0>(
				indexKeysDefinitionBuilder.Combine(
					indexKeysDefinitionBuilder.Ascending(m => m.Ns),
					indexKeysDefinitionBuilder.Ascending(m => m.Bucket),
					indexKeysDefinitionBuilder.Ascending(m => m.Key)
				), new CreateIndexOptions { Name = "CompoundIndex" }
			);

			CreateIndexModel<MongoReferencesModelV0> indexModelNamespace = new CreateIndexModel<MongoReferencesModelV0>(
				indexKeysDefinitionBuilder.Ascending(m => m.Ns),
				new CreateIndexOptions { Name = "NamespaceIndex" }
			);

			AddIndexFor<MongoReferencesModelV0>().CreateMany(new[] {
				indexModelClusteredKey,
				indexModelNamespace
			});

			CreateIndexModel<MongoReferencesModelV0> indexTTL = new CreateIndexModel<MongoReferencesModelV0>(
				indexKeysDefinitionBuilder.Ascending(m => m.ExpireAt), new CreateIndexOptions { Name = "ExpireAtTTL", ExpireAfter = TimeSpan.Zero }
			);

			AddIndexFor<MongoReferencesModelV0>().CreateOne(indexTTL);
		}

		public async Task<RefRecord> GetAsync(NamespaceId ns, BucketId bucket, RefId key, IReferencesStore.FieldFlags fieldFlags, IReferencesStore.OperationFlags opFlags, CancellationToken cancellationToken)
		{
			bool includePayload = (fieldFlags & IReferencesStore.FieldFlags.IncludePayload) != 0;
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();
			IAsyncCursor<MongoReferencesModelV0>? cursor = await collection.FindAsync(m => m.Ns == ns.ToString() && m.Bucket == bucket.ToString() && m.Key == key.ToString(), cancellationToken: cancellationToken);
			MongoReferencesModelV0? model = await cursor.FirstOrDefaultAsync(cancellationToken);
			if (model == null)
			{
				throw new RefNotFoundException(ns, bucket, key);
			}

			if (!includePayload)
			{
				// TODO: We should actually use this information to use a projection and not fetch the actual blob
				model.InlineBlob = null;
			}

			return model.ToRefRecord();
		}

		public async Task PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, byte[]? blob, bool isFinalized, bool allowOverwrite = false, CancellationToken cancellationToken = default)
		{
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();

			Task addNamespaceTask = AddNamespaceIfNotExistAsync(ns, cancellationToken);
			MongoReferencesModelV0 model = new MongoReferencesModelV0(ns, bucket, key, blobHash, blob, isFinalized, DateTime.Now);

			NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
			NamespacePolicy.StoragePoolGCMethod gcMethod = policy.GcMethod ?? NamespacePolicy.StoragePoolGCMethod.LastAccess;
			if (gcMethod == NamespacePolicy.StoragePoolGCMethod.TTL)
			{
				model.ExpireAt = DateTime.UtcNow.Add(policy.DefaultTTL);
			}

			
			bool allowOverwrites = policy.AllowOverwritesOfRefs || allowOverwrite;
			FilterDefinition<MongoReferencesModelV0> filter = Builders<MongoReferencesModelV0>.Filter.Where(m => m.Ns == ns.ToString() && m.Bucket == bucket.ToString() && m.Key == key.ToString());
			FindOneAndReplaceOptions<MongoReferencesModelV0, MongoReferencesModelV0> options = new FindOneAndReplaceOptions<MongoReferencesModelV0, MongoReferencesModelV0>
			{
				IsUpsert = allowOverwrites
			};
			MongoReferencesModelV0? found = await collection.FindOneAndReplaceAsync(filter, model, options, cancellationToken);
			if (found != null && !allowOverwrites)
			{
				// non-null return value means the old document already existed, so this is a potential overwrite
				if (!blobHash.Equals(new BlobId(found.BlobIdentifier)))
				{
					// blob was not the same, e.g. we attempted to change the value, this is not allowed
					throw new RefAlreadyExistsException(ns, bucket, key, found.ToRefRecord());
				}
			}
			else
			{
				// model not found, insert it
				FindOneAndReplaceOptions<MongoReferencesModelV0, MongoReferencesModelV0> insertOptions = new FindOneAndReplaceOptions<MongoReferencesModelV0, MongoReferencesModelV0>
				{
					IsUpsert = true
				};
				await collection.FindOneAndReplaceAsync(filter, model, insertOptions, cancellationToken);
			}

			await addNamespaceTask;
		}

		public async Task FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobIdentifier, CancellationToken cancellationToken)
		{
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();

			UpdateResult _ = await collection.UpdateOneAsync(
				model => model.Ns == ns.ToString() && model.Bucket == bucket.ToString() && model.Key == key.ToString(),
				Builders<MongoReferencesModelV0>.Update.Set(model => model.IsFinalized, true),
				cancellationToken: cancellationToken
			);
		}

		public async Task<DateTime?> GetLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();
			IAsyncCursor<MongoReferencesModelV0>? cursor = await collection.FindAsync(m => m.Ns == ns.ToString() && m.Bucket == bucket.ToString() && m.Key == key.ToString(), cancellationToken: cancellationToken);
			MongoReferencesModelV0? model = await cursor.FirstOrDefaultAsync(cancellationToken);

			return model?.LastAccessTime;
		}

		public async Task UpdateLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime newLastAccessTime, CancellationToken cancellationToken)
		{
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();

			UpdateResult _ = await collection.UpdateOneAsync(
				model => model.Ns == ns.ToString() && model.Bucket == bucket.ToString() && model.Key == key.ToString(),
				Builders<MongoReferencesModelV0>.Update.Set(model => model.LastAccessTime, newLastAccessTime), 
				cancellationToken: cancellationToken
			);
		}

		public async IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();
			IAsyncCursor<MongoReferencesModelV0>? cursor = await collection.FindAsync(FilterDefinition<MongoReferencesModelV0>.Empty, cancellationToken: cancellationToken);

			while (await cursor.MoveNextAsync(cancellationToken))
			{
				foreach (MongoReferencesModelV0 model in cursor.Current)
				{
					yield return (new NamespaceId(model.Ns), new BucketId(model.Bucket), new RefId(model.Key), model.LastAccessTime);
				}
			}
		}

		public async IAsyncEnumerable<(NamespaceId, BucketId, RefId)> GetRecordsWithoutAccessTimeAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			await foreach ((NamespaceId ns, BucketId bucket, RefId key, DateTime _) in GetRecordsAsync(cancellationToken))
			{
				yield return (ns, bucket, key);
			}
		}

		public async Task AddNamespaceIfNotExistAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			FilterDefinition<MongoNamespacesModelV0> filter = Builders<MongoNamespacesModelV0>.Filter.Where(m => m.Ns == ns.ToString());
			FindOneAndReplaceOptions<MongoNamespacesModelV0, MongoNamespacesModelV0> options = new FindOneAndReplaceOptions<MongoNamespacesModelV0, MongoNamespacesModelV0>
			{
				IsUpsert = true
			};

			IMongoCollection<MongoNamespacesModelV0> collection = GetCollection<MongoNamespacesModelV0>();
			await collection.FindOneAndReplaceAsync(filter, new MongoNamespacesModelV0(ns), options, cancellationToken);
		}

		public async IAsyncEnumerable<NamespaceId> GetNamespacesAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			IMongoCollection<MongoNamespacesModelV0> collection = GetCollection<MongoNamespacesModelV0>();

			IAsyncCursor<MongoNamespacesModelV0> cursor = await collection.FindAsync(FilterDefinition<MongoNamespacesModelV0>.Empty, cancellationToken: cancellationToken);
			while (await cursor.MoveNextAsync(cancellationToken))
			{
				foreach (MongoNamespacesModelV0? document in cursor.Current)
				{
					yield return new NamespaceId(document.Ns);
				}
			}
		}

		public async IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();

			IAsyncCursor<MongoReferencesModelV0> cursor = await collection.FindAsync(m => m.Ns == ns.ToString(), cancellationToken: cancellationToken);

			HashSet<BucketId> buckets = new HashSet<BucketId>();
			while (await cursor.MoveNextAsync(cancellationToken))
			{
				foreach (MongoReferencesModelV0? document in cursor.Current)
				{
					buckets.Add(new BucketId(document.Bucket));
				}
			}

			foreach (BucketId bucket in buckets)
			{
				yield return bucket;
			}
		}

		public async Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();

			DeleteResult result = await collection.DeleteOneAsync(model =>
				model.Ns == ns.ToString() && model.Bucket == bucket.ToString() && model.Key == key.ToString(),
				cancellationToken);

			return result.DeletedCount != 0;
		}

		public async Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();

			DeleteResult result = await collection.DeleteManyAsync(model =>
				model.Ns == ns.ToString(), cancellationToken);

			long deletedCount = 0;
			if (result.IsAcknowledged)
			{
				deletedCount = result.DeletedCount;
			}

			IMongoCollection<MongoNamespacesModelV0> namespaceCollection = GetCollection<MongoNamespacesModelV0>();
			await namespaceCollection.DeleteOneAsync(m => m.Ns == ns.ToString(), cancellationToken);
			return deletedCount;

		}

		public async Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
		{
			IMongoCollection<MongoReferencesModelV0> collection = GetCollection<MongoReferencesModelV0>();

			DeleteResult result = await collection.DeleteManyAsync(model =>
				model.Ns == ns.ToString() && model.Bucket == bucket.ToString(), cancellationToken);

			if (result.IsAcknowledged)
			{
				return result.DeletedCount;
			}

			// failed to delete
			return 0L;
		}

		public Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException();
		}

		public void SetLastAccessTTLDuration(TimeSpan duration)
		{
			IndexKeysDefinitionBuilder<MongoReferencesModelV0> indexKeysDefinitionBuilder = Builders<MongoReferencesModelV0>.IndexKeys;
			CreateIndexModel<MongoReferencesModelV0> indexTTL = new CreateIndexModel<MongoReferencesModelV0>(
				indexKeysDefinitionBuilder.Ascending(m => m.LastAccessTime), new CreateIndexOptions { Name = "LastAccessTTL", ExpireAfter = duration }
			);

			AddIndexFor<MongoReferencesModelV0>().CreateOne(indexTTL);
		}

		public Task CleanupTrackedStateAsync()
		{
			// nothing to clean up
			return Task.CompletedTask;
		}
	}

	// we do versioning by writing a discriminator into object
	[BsonDiscriminator("ref.v0")]
	[BsonIgnoreExtraElements]
	[MongoCollectionName("References")]
	public class MongoReferencesModelV0
	{
		[BsonConstructor]
		public MongoReferencesModelV0(string ns, string bucket, string key, string blobIdentifier, byte[] inlineBlob, bool isFinalized, DateTime lastAccessTime)
		{
			Ns = ns;
			Bucket = bucket;
			Key = key;
			BlobIdentifier = blobIdentifier;
			InlineBlob = inlineBlob;
			IsFinalized = isFinalized;
			LastAccessTime = lastAccessTime;
		}

		public MongoReferencesModelV0(NamespaceId ns, BucketId bucket, RefId key, BlobId blobIdentifier, byte[]? blob, bool isFinalized, DateTime lastAccessTime)
		{
			Ns = ns.ToString();
			Bucket = bucket.ToString();
			Key = key.ToString();
			InlineBlob = blob;
			BlobIdentifier = blobIdentifier.ToString();
			IsFinalized = isFinalized;
			LastAccessTime = lastAccessTime;
		}

		[BsonRequired]
		public string Ns { get; set; }

		[BsonRequired]
		public string Bucket { get; set; }

		[BsonRequired]
		public string Key { get; set; }

		public string BlobIdentifier { get; set; }

		public bool IsFinalized { get; set; }

		[BsonRequired]
		public DateTime LastAccessTime { get; set; }

		public byte[]? InlineBlob { get; set; }
		public DateTime ExpireAt { get; set; } = DateTime.MaxValue;

		public RefRecord ToRefRecord()
		{
			return new RefRecord(new NamespaceId(Ns), new BucketId(Bucket), new RefId(Key),
				LastAccessTime,
				InlineBlob, new BlobId(BlobIdentifier), IsFinalized);
		}
	}

	[MongoCollectionName("Namespaces")]
	[BsonDiscriminator("namespace.v0")]
	[BsonIgnoreExtraElements]
	public class MongoNamespacesModelV0
	{
		[BsonRequired]
		public string Ns { get; set; }

		[BsonConstructor]
		public MongoNamespacesModelV0(string ns)
		{
			Ns = ns;
		}

		public MongoNamespacesModelV0(NamespaceId ns)
		{
			Ns = ns.ToString();
		}
	}
}
