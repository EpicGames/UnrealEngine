// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Threading;
using System.Threading.Tasks;
using Cassandra;
using Cassandra.Mapping;
using Jupiter.Common;
using Jupiter.Common.Utils;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class ScyllaReferencesStore : IReferencesStore
	{
		private readonly ISession _session;
		private readonly IMapper _mapper;
		private readonly IScyllaSessionManager _scyllaSessionManager;
		private readonly IOptionsMonitor<ScyllaSettings> _settings;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;
		private readonly PreparedStatement _getObjectsStatement;
		private readonly PreparedStatement _getObjectsLastAccessStatement;
		private readonly PreparedStatement _getNamespacesStatement;
		private readonly PreparedStatement _getNamespacesOldStatement;
		private readonly PreparedStatement _getObjectsForPartitionRangeStatement;
		private readonly PreparedStatement _getObjectsBucketsStatement;
		private readonly PreparedStatement _getObjectsLastAccessForPartitionRangeStatement;

		private readonly ConcurrentDictionary<NamespaceId, ConcurrentDictionary<BucketId, DateTime>> _addedBuckets = new ConcurrentDictionary<NamespaceId, ConcurrentDictionary<BucketId, DateTime>>();

		public ScyllaReferencesStore(IScyllaSessionManager scyllaSessionManager, IOptionsMonitor<ScyllaSettings> settings, INamespacePolicyResolver namespacePolicyResolver, Tracer tracer, ILogger<ScyllaReferencesStore> logger)
		{
			_session = scyllaSessionManager.GetSessionForReplicatedKeyspace();
			_scyllaSessionManager = scyllaSessionManager;
			_settings = settings;
			_namespacePolicyResolver = namespacePolicyResolver;
			_tracer = tracer;
			_logger = logger;

			_mapper = new Mapper(_session);

			if (!_settings.CurrentValue.AvoidSchemaChanges)
			{
				_session.Execute(new SimpleStatement(@"CREATE TABLE IF NOT EXISTS objects (
					namespace text, 
					bucket text, 
					name text, 
					payload_hash blob_identifier, 
					inline_payload blob, 
					is_finalized boolean,
					last_access_time timestamp,
					PRIMARY KEY ((namespace, bucket, name))
				);"
				));

				_session.Execute(new SimpleStatement(@"CREATE TABLE IF NOT EXISTS object_last_access_v2 (
					namespace text, 
					bucket text, 
					name text, 
					last_access_time timestamp,
					PRIMARY KEY ((namespace, bucket, name))
				);"
				));

				_session.Execute(new SimpleStatement(@"CREATE TABLE IF NOT EXISTS buckets_v2 (
					namespace text, 
					bucket text, 
					PRIMARY KEY ((namespace), bucket)
				);"
				));

				_session.Execute(new SimpleStatement(@"CREATE TABLE IF NOT EXISTS buckets (
					namespace text, 
					bucket set<text>, 
					PRIMARY KEY (namespace)
				);"
				));
			}

			// BYPASS CACHE is a scylla specific extension to disable populating the cache, should be ignored by other cassandra dbs
			string cqlOptions = scyllaSessionManager.IsScylla ? "BYPASS CACHE" : "";
			_getObjectsStatement = _session.Prepare($"SELECT namespace, bucket, name, last_access_time FROM objects {cqlOptions}");
			_getObjectsLastAccessStatement = _session.Prepare($"SELECT namespace, bucket, name, last_access_time FROM object_last_access_v2 {cqlOptions}");
			_getNamespacesStatement = scyllaSessionManager.IsAWSKeyspace ? _session.Prepare("SELECT namespace FROM buckets_v2") :  _session.Prepare("SELECT DISTINCT namespace FROM buckets_v2");
			_getNamespacesOldStatement = scyllaSessionManager.IsAWSKeyspace ? _session.Prepare("SELECT namespace FROM buckets") : _session.Prepare("SELECT DISTINCT namespace FROM buckets");

			_getObjectsBucketsStatement = _session.Prepare("select namespace, bucket FROM objects");
			_getObjectsForPartitionRangeStatement = _session.Prepare($"SELECT namespace, bucket, name, last_access_time FROM objects WHERE token(namespace, bucket, name) >= ? AND token(namespace, bucket, name) <= ? {cqlOptions}");
			_getObjectsLastAccessForPartitionRangeStatement = _session.Prepare($"SELECT namespace, bucket, name, last_access_time FROM object_last_access_v2 WHERE token(namespace, bucket, name) >= ? AND token(namespace, bucket, name) <= ? {cqlOptions}");
		}

		public async Task<RefRecord> GetAsync(NamespaceId ns, BucketId bucket, RefId name, IReferencesStore.FieldFlags fieldFlags, IReferencesStore.OperationFlags opFlags, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get").SetAttribute("resource.name", $"{ns}.{bucket}.{name}");
			scope.SetAttribute("BypassCache", false);

			try
			{
				ScyllaObject? o;
				bool includePayload = (fieldFlags & IReferencesStore.FieldFlags.IncludePayload) != 0;
				if (includePayload)
				{
					o = await _mapper.SingleOrDefaultAsync<ScyllaObject>("WHERE namespace = ? AND bucket = ? AND name = ?", ns.ToString(),
						bucket.ToString(), name.ToString());
				}
				else
				{
					string cqlOptions = "";
					if (_scyllaSessionManager.IsScylla && opFlags.HasFlag(IReferencesStore.OperationFlags.BypassCache))
					{
						// BYPASS CACHE is a scylla specific extension to disable populating the cache, should be ignored by other cassandra dbs
						cqlOptions = "BYPASS CACHE";
						scope.SetAttribute("BypassCache", true);
					}

					// fetch everything except for the inline blob which is quite large
					o = await _mapper.SingleOrDefaultAsync<ScyllaObject>(
						$"SELECT namespace, bucket, name , payload_hash, is_finalized, last_access_time FROM objects WHERE namespace = ? AND bucket = ? AND name = ? {cqlOptions}",
						ns.ToString(), bucket.ToString(), name.ToString());
				}

				if (o == null)
				{
					throw new RefNotFoundException(ns, bucket, name);
				}

				if (o.IsRequiredFieldIsMissing())
				{
					_logger.LogWarning("Partial object found {Namespace} {Bucket} {Name} cleaning up stale data. Will be ignored.", ns, bucket, name);
					await DeleteAsync(ns, bucket, name, cancellationToken);
					throw new RefNotFoundException(ns, bucket, name);
				}

				return new RefRecord(new NamespaceId(o.Namespace!), new BucketId(o.Bucket!), new RefId(o.Name!), o.LastAccessTime, o.InlinePayload,
					o.PayloadHash!.AsBlobIdentifier(), o.IsFinalized!.Value);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task PutAsync(NamespaceId ns, BucketId bucket, RefId name, BlobId blobHash, byte[] blob, bool isFinalized, bool allowOverwrite = false, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.put").SetAttribute("resource.name", $"{ns}.{bucket}.{name}");

			// add the bucket in parallel with the actual object
			Task addBucketTask = AddBucketAsync(ns, bucket);

			int? ttl = null;
			NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
			NamespacePolicy.StoragePoolGCMethod gcMethod = policy.GcMethod ?? NamespacePolicy.StoragePoolGCMethod.LastAccess;
			if (gcMethod == NamespacePolicy.StoragePoolGCMethod.TTL)
			{
				ttl = (int)policy.DefaultTTL.TotalSeconds;
			}

			try
			{
				Task? insertLastAccess = gcMethod == NamespacePolicy.StoragePoolGCMethod.LastAccess
					? _mapper.InsertAsync<ScyllaObjectLastAccess>(new ScyllaObjectLastAccess(ns, bucket, name, DateTime.Now))
					: null;

				bool allowOverwrites = policy.AllowOverwritesOfRefs || allowOverwrite;
				if (!allowOverwrites)
				{
					bool insertRef = false;
					try
					{
						RefRecord oldRecord = await GetAsync(ns, bucket, name, IReferencesStore.FieldFlags.IncludePayload,
							IReferencesStore.OperationFlags.None, cancellationToken);

						if (!oldRecord.BlobIdentifier.Equals(blobHash))
						{
							// blob was not the same, e.g. we attempted to change the value, this is not allowed
							throw new RefAlreadyExistsException(ns, bucket, name, oldRecord);
						}

						// new record is the same as the old, this is fine there is no need to write anything
					}
					catch (RefNotFoundException)
					{
						insertRef = true;
					}

					if (insertRef)
					{
						await _mapper.InsertAsync<ScyllaObject>(new ScyllaObject(ns, bucket, name, blob, blobHash, isFinalized), ttl: ttl,
							insertNulls: false);
					}
				}
				else
				{
					// if we allow overwrites we do the insert without the "if not exists" check which is much faster
					await _mapper.InsertAsync<ScyllaObject>(new ScyllaObject(ns, bucket, name, blob, blobHash, isFinalized), ttl: ttl,
						insertNulls: false);
				}

				if (insertLastAccess != null)
				{
					await insertLastAccess;
				}

				await addBucketTask;
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task FinalizeAsync(NamespaceId ns, BucketId bucket, RefId name, BlobId blobIdentifier, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.finalize").SetAttribute("resource.name", $"{ns}.{bucket}.{name}");

			int? ttl = null;
			NamespacePolicy policy = _namespacePolicyResolver.GetPoliciesForNs(ns);
			NamespacePolicy.StoragePoolGCMethod gcMethod = policy.GcMethod ?? NamespacePolicy.StoragePoolGCMethod.LastAccess;
			if (gcMethod == NamespacePolicy.StoragePoolGCMethod.TTL)
			{
				ttl = (int)policy.DefaultTTL.TotalSeconds;
			}

			try
			{
				if (ttl != null)
				{
					await _mapper.UpdateAsync<ScyllaObject>("USING TTL ? SET is_finalized=true WHERE namespace=? AND bucket=? AND name=?", ttl.Value, ns.ToString(), bucket.ToString(), name.ToString());
				}
				else
				{
					await _mapper.UpdateAsync<ScyllaObject>("SET is_finalized=true WHERE namespace=? AND bucket=? AND name=?", ns.ToString(), bucket.ToString(), name.ToString());
				}
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task<DateTime?> GetLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_last_access_time").SetAttribute("resource.name", $"{ns}.{bucket}.{key}");
			
			try
			{
				ScyllaObjectLastAccess? lastAccessRecord = await _mapper.SingleOrDefaultAsync<ScyllaObjectLastAccess>("WHERE namespace = ? AND bucket = ? AND name = ?", ns.ToString(), bucket.ToString(), key.ToString());
				return lastAccessRecord?.LastAccessTime;
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task UpdateLastAccessTimeAsync(NamespaceId ns, BucketId bucket, RefId name, DateTime lastAccessTime, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.update_last_access_time");

			try
			{
				Task? updateObjectLastAccessTask = null;
				updateObjectLastAccessTask = _mapper.InsertAsync<ScyllaObjectLastAccess>(new ScyllaObjectLastAccess(ns, bucket, name, lastAccessTime));

				if (_settings.CurrentValue.UpdateLegacyLastAccessField)
				{
					await _mapper.UpdateAsync<ScyllaObject>("SET last_access_time = ? WHERE namespace = ? AND bucket = ? AND name = ?", lastAccessTime, ns.ToString(), bucket.ToString(), name.ToString());
				}

				await updateObjectLastAccessTask;
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_records");

			if (_settings.CurrentValue.UsePerShardScanning)
			{
				IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> enumerable = GetRecordsPerShardAsync();

				await foreach ((NamespaceId, BucketId, RefId, DateTime) record in enumerable)
				{
					yield return (record.Item1, record.Item2, record.Item3, record.Item4);
				}
			}
			else
			{
				PreparedStatement getObjectStatement = _settings.CurrentValue.ListObjectsFromLastAccessTable
					? _getObjectsLastAccessStatement
					: _getObjectsStatement;
				const int MaxRetryAttempts = 3;
				RowSet rowSet = await _session.ExecuteAsync(getObjectStatement.Bind());

				do
				{
					int countOfRows = rowSet.GetAvailableWithoutFetching();
					IEnumerable<Row> localRows = rowSet.Take(countOfRows);
					Task prefetchTask = rowSet.FetchMoreResultsAsync();

					foreach (Row row in localRows)
					{
						string ns = row.GetValue<string>("namespace");
						string bucket = row.GetValue<string>("bucket");
						string name = row.GetValue<string>("name");
						DateTime? lastAccessTime = row.GetValue<DateTime?>("last_access_time");

						// skip any names that are not conformant to io hash
						if (name.Length != 40)
						{
							continue;
						}

						// if last access time is missing we treat it as being very old
						lastAccessTime ??= DateTime.MinValue;
						yield return (new NamespaceId(ns), new BucketId(bucket), new RefId(name), lastAccessTime.Value);
					}

					int retryAttempts = 0;
					Exception? timeoutException = null;
					while (retryAttempts < MaxRetryAttempts)
					{
						try
						{
							await prefetchTask;
							timeoutException = null;
							break;
						}
						catch (ReadTimeoutException e)
						{
							retryAttempts += 1;
							_logger.LogWarning(
								"Cassandra read timeouts, waiting a while and then retrying. Attempt {Attempts} .",
								retryAttempts);
							// wait 10 seconds and try again as the Db is under heavy load right now
							await Task.Delay(TimeSpan.FromSeconds(10), cancellationToken);
							timeoutException = e;
						}
					}

					if (timeoutException != null)
					{
						_logger.LogWarning("Cassandra read timeouts, attempted {Attempts} attempts now we give up.",
							retryAttempts);
						// we have failed to many times, rethrow the exception and abort to avoid stalling here for ever
						throw new RemoteServiceException("Error communicating with Scylla", timeoutException);
					}
				} while (!rowSet.IsFullyFetched);
			}
		}

		public async IAsyncEnumerable<(NamespaceId, BucketId, RefId)> GetRecordsWithoutAccessTimeAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_records_no_access_time");

			if (_settings.CurrentValue.UsePerShardScanning)
			{
				IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> enumerable = GetRecordsPerShardAsync(false);

				await foreach ((NamespaceId, BucketId, RefId, DateTime) record in enumerable)
				{
					yield return (record.Item1, record.Item2, record.Item3);
				}
			}
			else
			{
				// use the object table for listing here as its used by the consistency check
				PreparedStatement getObjectStatement = _getObjectsStatement;
				const int MaxRetryAttempts = 3;
				RowSet rowSet = await _session.ExecuteAsync(getObjectStatement.Bind());

				do
				{
					int countOfRows = rowSet.GetAvailableWithoutFetching();
					IEnumerable<Row> localRows = rowSet.Take(countOfRows);
					Task prefetchTask = rowSet.FetchMoreResultsAsync();

					foreach (Row row in localRows)
					{
						string ns = row.GetValue<string>("namespace");
						string bucket = row.GetValue<string>("bucket");
						string name = row.GetValue<string>("name");

						// skip any names that are not conformant to io hash
						if (name.Length != 40)
						{
							continue;
						}

						yield return (new NamespaceId(ns), new BucketId(bucket), new RefId(name));
					}

					int retryAttempts = 0;
					Exception? timeoutException = null;
					while (retryAttempts < MaxRetryAttempts)
					{
						try
						{
							await prefetchTask;
							timeoutException = null;
							break;
						}
						catch (ReadTimeoutException e)
						{
							retryAttempts += 1;
							_logger.LogWarning(
								"Cassandra read timeouts, waiting a while and then retrying. Attempt {Attempts} .",
								retryAttempts);
							// wait 10 seconds and try again as the Db is under heavy load right now
							await Task.Delay(TimeSpan.FromSeconds(10), cancellationToken);
							timeoutException = e;
						}
					}

					if (timeoutException != null)
					{
						_logger.LogWarning("Cassandra read timeouts, attempted {Attempts} attempts now we give up.",
							retryAttempts);
						// we have failed to many times, rethrow the exception and abort to avoid stalling here for ever
						throw new RemoteServiceException("Error communicating with Scylla", timeoutException);
					}
				} while (!rowSet.IsFullyFetched);
			}
		}
		
		/// <summary>
		/// This implements a more efficient scanning where we fetch objects based on which shard it is in. It scans the entire database and thus returns all namespaces.
		/// See https://www.scylladb.com/2017/03/28/parallel-efficient-full-table-scan-scylla/
		/// </summary>
		/// <returns></returns>
		private async IAsyncEnumerable<(NamespaceId, BucketId, RefId, DateTime)> GetRecordsPerShardAsync(bool? forceUseLastAccessTable = null)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_records_per_shard");

			bool useLastAccessTable = forceUseLastAccessTable ?? _settings.CurrentValue.ListObjectsFromLastAccessTable;
			PreparedStatement getObjectStatement = useLastAccessTable
				? _getObjectsLastAccessForPartitionRangeStatement
				: _getObjectsForPartitionRangeStatement;

			scope.SetAttribute("TableUsed", useLastAccessTable ? "LastAccess" : "Objects");

			// generate a list of all the primary key ranges that exist on the cluster
			List<(long, long)> tableRanges = ScyllaUtils.GetTableRanges(_settings.CurrentValue.CountOfNodes, _settings.CurrentValue.CountOfCoresPerNode, 3).ToList();

			// randomly shuffle this list so that we do not scan them in the same order, means that we will eventually visit all ranges even if the process running this is restarted before we have finished
			List<int> tableRangeIndices = Enumerable.Range(0, tableRanges.Count).ToList();
			tableRangeIndices.Shuffle();

			if (_settings.CurrentValue.AllowParallelRecordFetch)
			{
				ConcurrentQueue<(NamespaceId, BucketId, RefId, DateTime)> foundRecords = new ConcurrentQueue<(NamespaceId, BucketId, RefId, DateTime)>();

				Task scanTask = Parallel.ForEachAsync(tableRangeIndices, new ParallelOptions { MaxDegreeOfParallelism = (int)_settings.CurrentValue.CountOfNodes },
					async (index, token) =>
					{
						(long, long) range = tableRanges[index];
						BoundStatement? statement = getObjectStatement.Bind(range.Item1, range.Item2);
						statement.SetPageSize(5000); // increase page size as there seems to be issues fetching multiple pages when token scanning
						RowSet rowSet = await _session.ExecuteAsync(statement);
						foreach (Row row in rowSet)
						{
							if (token.IsCancellationRequested)
							{
								return;
							}
							string ns = row.GetValue<string>("namespace");
							string bucket = row.GetValue<string>("bucket");
							string name = row.GetValue<string>("name");
							DateTime? lastAccessTime = row.GetValue<DateTime?>("last_access_time");

							// skip any names that are not conformant to io hash
							if (name.Length != 40)
							{
								continue;
							}

							// if last access time is missing we treat it as being very old
							lastAccessTime ??= DateTime.MinValue;
							foundRecords.Enqueue((new NamespaceId(ns), new BucketId(bucket), new RefId(name), lastAccessTime.Value));
						}
					});

				while (!scanTask.IsCompleted)
				{
					while (foundRecords.TryDequeue(out (NamespaceId, BucketId, RefId, DateTime) foundRecord))
					{
						yield return foundRecord;
					}

					await Task.Delay(10);
				}

				await scanTask;
			}
			else
			{
				foreach (int index in tableRangeIndices)
				{
					(long, long) range = tableRanges[index];
					RowSet rowSet = await _session.ExecuteAsync(getObjectStatement.Bind(range.Item1, range.Item2));
					foreach (Row row in rowSet)
					{
						string ns = row.GetValue<string>("namespace");
						string bucket = row.GetValue<string>("bucket");
						string name = row.GetValue<string>("name");
						DateTime? lastAccessTime = row.GetValue<DateTime?>("last_access_time");

						// skip any names that are not conformant to io hash
						if (name.Length != 40)
						{
							continue;
						}

						// if last access time is missing we treat it as being very old
						lastAccessTime ??= DateTime.MinValue;
						yield return (new NamespaceId(ns), new BucketId(bucket), new RefId(name), lastAccessTime.Value);
					}
				}
			}
		}

		public async IAsyncEnumerable<NamespaceId> GetNamespacesAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_namespaces");

			{
				RowSet rowSet;

				try
				{
					rowSet = await _session.ExecuteAsync(_getNamespacesStatement.Bind());
				}
				catch (QueryExecutionException e)
				{
					throw new RemoteServiceException("Error communicating with Scylla", e);
				}

				if (_scyllaSessionManager.IsAWSKeyspace)
				{
					// keyspaces do not support the distinct keyword so we filter here on the client instead

					List<NamespaceId> namespaces = new List<NamespaceId>();
					foreach (Row row in rowSet)
					{
						if (rowSet.GetAvailableWithoutFetching() == 0)
						{
							await rowSet.FetchMoreResultsAsync();
						}

						NamespaceId ns = new NamespaceId(row.GetValue<string>(0));
						if (!namespaces.Contains(ns))
						{
							namespaces.Add(ns);
						}
					}

					foreach (NamespaceId namespaceId in namespaces)
					{
						yield return namespaceId;
					}
				}
				else
				{
					foreach (Row row in rowSet)
					{
						if (rowSet.GetAvailableWithoutFetching() == 0)
						{
							await rowSet.FetchMoreResultsAsync();
						}

						yield return new NamespaceId(row.GetValue<string>(0));
					}
				}
			}

			if (_settings.CurrentValue.ListObjectsFromOldNamespaceTable)
			{
				// this will likely generate duplicates from the statements above but that is not a huge issue
				using TelemetrySpan _ = _tracer.BuildScyllaSpan("scylla.get_old_namespaces");

				RowSet rowSet;

				try
				{
					rowSet = await _session.ExecuteAsync(_getNamespacesOldStatement.Bind());
				}
				catch (QueryExecutionException e)
				{
					throw new RemoteServiceException("Error communicating with Scylla", e);
				}

				foreach (Row row in rowSet)
				{
					if (rowSet.GetAvailableWithoutFetching() == 0)
					{
						await rowSet.FetchMoreResultsAsync();
					}

					yield return new NamespaceId(row.GetValue<string>(0));
				}
			}
		}

		public async IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			IEnumerable<ScyllaBucket> buckets;
			
			try
			{
				buckets = await _mapper.FetchAsync<ScyllaBucket>("WHERE namespace=?", ns.ToString());
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
			
			foreach (ScyllaBucket scyllaBucket in buckets)
			{
				if (scyllaBucket.Bucket == null)
				{
					continue;
				}

				yield return new BucketId(scyllaBucket.Bucket);
			}
		}

		public async Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.delete_record").SetAttribute("resource.name", $"{ns}.{bucket}.{key}");

			try
			{
				Task? lastAccessDeleteTask = _mapper.DeleteAsync<ScyllaObjectLastAccess>("WHERE namespace=? AND bucket=? AND name=?", ns.ToString(), bucket.ToString(), key.ToString());

				await _mapper.DeleteAsync<ScyllaObject>("WHERE namespace=? AND bucket=? AND name=?", ns.ToString(), bucket.ToString(), key.ToString());

				await lastAccessDeleteTask;

				return true;
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.delete_namespace");

			try
			{
				RowSet rowSet = await _session.ExecuteAsync(new SimpleStatement("SELECT bucket, name FROM objects WHERE namespace = ? ALLOW FILTERING;", ns.ToString()));
				long deletedCount = 0;
				foreach (Row row in rowSet)
				{
					string bucket = row.GetValue<string>("bucket");
					string name = row.GetValue<string>("name");

					await DeleteAsync(ns, new BucketId(bucket), new RefId(name), cancellationToken);

					deletedCount++;
				}

				// remove the tracking in the buckets table as well
				await _session.ExecuteAsync(new SimpleStatement("DELETE FROM buckets WHERE namespace = ?", ns.ToString()));
				await _session.ExecuteAsync(new SimpleStatement("DELETE FROM buckets_v2 WHERE namespace = ?", ns.ToString()));

				return deletedCount;
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.delete_bucket");

			try
			{
				RowSet rowSet = await _session.ExecuteAsync(new SimpleStatement("SELECT name FROM objects WHERE namespace = ? AND bucket = ? ALLOW FILTERING;", ns.ToString(), bucket.ToString()));
				long deletedCount = 0;
				foreach (Row row in rowSet)
				{
					string name = row.GetValue<string>("name");

					await DeleteAsync(ns, bucket, new RefId(name), cancellationToken);
					deletedCount++;
				}

				// remove the tracking in the buckets table as well
				await _mapper.DeleteAsync<ScyllaBucket>(new ScyllaBucket(ns, bucket));

				return deletedCount;
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId key, uint ttl, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.update_ttl");

			try
			{
				ScyllaObject? o = await _mapper.SingleOrDefaultAsync<ScyllaObject>("WHERE namespace=? AND bucket=? AND name=?", ns.ToString(), bucket.ToString(), key.ToString());
				if (o != null)
				{
					await _mapper.InsertAsync<ScyllaObject>(o, false, (int)ttl);
				}
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		private async Task AddBucketAsync(NamespaceId ns, BucketId bucket)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.add_bucket");

			ConcurrentDictionary<BucketId, DateTime> addedBuckets = _addedBuckets.GetOrAdd(ns, id => new ConcurrentDictionary<BucketId, DateTime>());

			DateTime now = DateTime.UtcNow;
			if (addedBuckets.TryGetValue(bucket, out DateTime added))
			{
				if (added > now - TimeSpan.FromHours(1))
				{
					return;
				}
			}

			try
			{
				Task addTask = _mapper.InsertAsync<ScyllaBucket>(new ScyllaBucket(ns, bucket));
				await addTask;
				addedBuckets.AddOrUpdate(bucket, now, (_, _) => now);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task CleanupTrackedStateAsync()
		{
			try
			{
				_logger.LogInformation("Starting internal scylla cleanup. ");
				DateTime cleanupStart = DateTime.Now;
				ConcurrentDictionary<NamespaceId, ConcurrentDictionary<BucketId, bool>> namespacesWithRefs = new();

				// build lookup of which namespaces and buckets are actually references by live objects
				{
					RowSet rowSet = await _session.ExecuteAsync(_getObjectsBucketsStatement.Bind());

					foreach (Row row in rowSet)
					{
						if (rowSet.GetAvailableWithoutFetching() == 0)
						{
							await rowSet.FetchMoreResultsAsync();
						}

						NamespaceId ns = new NamespaceId(row.GetValue<string>(0));
						BucketId bucket = new BucketId(row.GetValue<string>(1));

						ConcurrentDictionary<BucketId, bool> bucketTracking =
							namespacesWithRefs.GetOrAdd(ns, new ConcurrentDictionary<BucketId, bool>());
						bucketTracking.TryAdd(bucket, true);
					}
				}

				// enumerate our cached namespaces and buckets and determine if any of them are old
				await foreach (NamespaceId ns in GetNamespacesAsync(CancellationToken.None))
				{
					bool refFound = false;
					if (!namespacesWithRefs.TryGetValue(ns, out ConcurrentDictionary<BucketId, bool>? foundBuckets))
					{
						// namespace does not exist, still need to processes all buckets to remove their tracking
						foundBuckets = new ConcurrentDictionary<BucketId, bool>();
					}

					await foreach (BucketId bucket in GetBucketsAsync(ns, CancellationToken.None))
					{
						// if the bucket had a ref we should not remove its tracking
						if (foundBuckets.ContainsKey(bucket))
						{
							refFound = true;
							continue;
						}

						_logger.LogWarning("Namespace {Namespace} Bucket {Bucket} has no refs. Should be removed from tracking.", ns, bucket);

						if (_settings.CurrentValue.CleanupOldTrackedState)
						{
							await _session.ExecuteAsync(new SimpleStatement("DELETE FROM buckets_v2 WHERE namespace = ? and bucket = ?",
								ns.ToString(), bucket.ToString()));
						}
					}

					if (!refFound)
					{
						_logger.LogWarning("Namespace {Namespace} has no refs. Should be removed from tracking.", ns);

						if (_settings.CurrentValue.CleanupOldTrackedState)
						{
							await _session.ExecuteAsync(new SimpleStatement("DELETE FROM buckets_v2 WHERE namespace = ?", ns.ToString()));
						}
					}
				}

				TimeSpan cleanupDuration = DateTime.Now - cleanupStart;
				_logger.LogInformation("Finished internal scylla cleanup. Cleanup took: {CleanupDuration}", cleanupDuration);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}
	}

	public class ScyllaBlobIdentifier
	{
		public ScyllaBlobIdentifier()
		{
			Hash = null;
		}

		public ScyllaBlobIdentifier(ContentHash hash)
		{
			Hash = hash.HashData;
		}

		public byte[]? Hash { get; set; }

		public BlobId AsBlobIdentifier()
		{
			return new BlobId(Hash!);
		}
	}

	public class ScyllaObjectReference
	{
		// used by the cassandra mapper
		public ScyllaObjectReference()
		{
			Bucket = null!;
			Key = null!;
		}

		public ScyllaObjectReference(BucketId bucket, RefId key)
		{
			Bucket = bucket.ToString();
			Key = key.ToString();
		}

		public string Bucket { get; set; }
		public string Key { get; set; }

		public (BucketId, RefId) AsTuple()
		{
			return (new BucketId(Bucket), new RefId(Key));
		}
	}

	[Cassandra.Mapping.Attributes.Table("objects")]
	public class ScyllaObject
	{
		public ScyllaObject()
		{

		}

		public ScyllaObject(NamespaceId ns, BucketId bucket, RefId name, byte[] payload, BlobId payloadHash, bool isFinalized)
		{
			Namespace = ns.ToString();
			Bucket = bucket.ToString();
			Name = name.ToString();
			InlinePayload = payload;
			PayloadHash = new ScyllaBlobIdentifier(payloadHash);

			IsFinalized = isFinalized;

			LastAccessTime = DateTime.Now;
		}

		[Cassandra.Mapping.Attributes.PartitionKey(0)]
		public string? Namespace { get; set; }

		[Cassandra.Mapping.Attributes.PartitionKey(1)]
		public string? Bucket { get; set; }

		[Cassandra.Mapping.Attributes.PartitionKey(2)]
		public string? Name { get; set; }

		[Cassandra.Mapping.Attributes.Column("payload_hash")]
		public ScyllaBlobIdentifier? PayloadHash { get; set; }

		[Cassandra.Mapping.Attributes.Column("inline_payload")]
		public byte[]? InlinePayload { get; set; }

		[Cassandra.Mapping.Attributes.Column("is_finalized")]
		public bool? IsFinalized { get; set; }
		[Cassandra.Mapping.Attributes.Column("last_access_time")]
		public DateTime LastAccessTime { get; set; }

		public bool IsRequiredFieldIsMissing()
		{
			if (string.IsNullOrEmpty(Namespace))
			{
				return true;
			}

			if (string.IsNullOrEmpty(Bucket))
			{
				return true;
			}

			if (string.IsNullOrEmpty(Name))
			{
				return true;
			}

			if (PayloadHash == null)
			{
				return true;
			}

			if (!IsFinalized.HasValue)
			{
				return true;
			}

			return false;
		}
	}

	[Cassandra.Mapping.Attributes.Table("buckets_v2")]
	public class ScyllaBucket
	{
		public ScyllaBucket()
		{
		}

		public ScyllaBucket(NamespaceId ns, BucketId bucket)
		{
			Namespace = ns.ToString();
			Bucket = bucket.ToString();
		}

		[Cassandra.Mapping.Attributes.PartitionKey]
		public string? Namespace { get; set; }

		[Cassandra.Mapping.Attributes.ClusteringKey]
		public string? Bucket { get; set; }
	}

	[Cassandra.Mapping.Attributes.Table("object_last_access_v2")]
	public class ScyllaObjectLastAccess
	{
		public ScyllaObjectLastAccess()
		{

		}

		public ScyllaObjectLastAccess(NamespaceId ns, BucketId bucket, RefId name, DateTime lastAccessTime)
		{
			Namespace = ns.ToString();
			Bucket = bucket.ToString();
			Name = name.ToString();

			LastAccessTime = lastAccessTime;
		}

		[Cassandra.Mapping.Attributes.PartitionKey(0)]
		public string? Namespace { get; set; }

		[Cassandra.Mapping.Attributes.PartitionKey(1)]
		public string? Bucket { get; set; }

		[Cassandra.Mapping.Attributes.PartitionKey(2)]
		public string? Name { get; set; }

		[Cassandra.Mapping.Attributes.Column("last_access_time")]
		public DateTime LastAccessTime { get; set; }
	}
}
