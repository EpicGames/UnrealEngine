// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using Cassandra;

namespace Jupiter.Implementation;

public class ScyllaSettings : IValidatableObject
{
	public string ConnectionString { get; set; } = "Contact Points=localhost,scylla;Default Keyspace=jupiter";
	public int MaxSnapshotsPerNamespace { get; set; } = 10;

	/// <summary>
	/// Set to override the replication strategy used to create keyspace
	/// Note that this only applies when creating the keyspace
	/// To modify a existing keyspace see https://docs.datastax.com/en/dse/6.7/dse-admin/datastax_enterprise/operations/opsChangeKSStrategy.html
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
	public Dictionary<string, string>? KeyspaceReplicationStrategy { get; set; }

	/// <summary>
	/// Set to override the replication strategy used to create the local keyspace
	/// Note that this only applies when creating the keyspace
	/// To modify a existing keyspace see https://docs.datastax.com/en/dse/6.7/dse-admin/datastax_enterprise/operations/opsChangeKSStrategy.html
	/// </summary>
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
	public Dictionary<string, string>? LocalKeyspaceReplicationStrategy { get; set; }

	/// <summary>
	/// Used to configure the load balancing policy to stick to this specified datacenter
	/// </summary>
	[Required]
	public string LocalDatacenterName { get; set; } = null!;

	[Required]
	public string LocalKeyspaceSuffix { get; set; } = null!;

	/// <summary>
	/// Max number of connections opened, so the max amount of scylla nodes connected to.
	/// See https://docs.datastax.com/en/developer/nodejs-driver/4.6/features/connection-pooling/
	/// </summary>
	public int MaxConnectionForLocalHost { get; set; } = 8;

	/// <summary>
	/// Fixed maximum amount of requests in flight for a connection before a new connection is used (thus max requests per node)
	/// </summary>
	public int MaxRequestsPerConnection { get; set; } = 4096;

	/// <summary>
	/// The time for a replication log event to live in the incremental state before being deleted, assumption is that the snapshot will have processed the event within this time
	/// </summary>
	public TimeSpan ReplicationLogTimeToLive { get; set; } = TimeSpan.FromDays(7);

	/// <summary>
	/// Whether a Cosmos DB Cassandra layer from Azure should be used.
	/// </summary>
	public bool UseAzureCosmosDB { get; set; } = false;

	/// <summary>
	/// Whether a AWS Keyspace is used, adjusts requests to be compatible with this flavor of Cassandra.
	/// </summary>
	public bool UseAWSKeyspace { get; set; } = false;

	/// <summary>
	/// Set to false to disable use of SSL, this is only used when connecting to a CosmosDB instance or a AWS Keyspace instance
	/// </summary>
	public bool UseSSL { get; set; } = true;

	/// <summary>
	/// Toggle to enable reading objects from the object_last_access_time_v2 table
	/// </summary>
	public bool ListObjectsFromLastAccessTable { get; set; } = true;

	/// <summary>
	/// Set to update the last access field in the object table, this is legacy and causes extra compaction work on the database
	/// </summary>
	public bool UpdateLegacyLastAccessField { get; set; } = false;

	/// <summary>
	/// Read timeout in milliseconds
	/// Set to -1 to get the default timeout, set to 0 to disable timeouts
	/// </summary>
	public int ReadTimeout { get; set; } = -1; // use the default timeout value by default

	/// <summary>
	/// Enable to scan database per shard, this requires setting more information about your cluster
	/// </summary>
	public bool UsePerShardScanning { get; set; } = false;

	/// <summary>
	/// Count of cores each nodes has, only used when UsePerShardScanning is set
	/// </summary>
	[Range(1, 128)]
	public uint CountOfCoresPerNode { get; set; } = 1;

	/// <summary>
	/// Count of nodes in the DC, only used when UsePerShardScanning is set
	/// </summary>
	[Range(1, 64)]
	public uint CountOfNodes { get; set; } = 1;

	/// <summary>
	/// Enable to migrate data from blob_index table to blob_index_v2 table
	/// </summary>
	public bool MigrateFromOldBlobIndex { get; set; } = false;

	/// <summary>
	/// Enable to use list namespaces from the old table, this can set for a period while migration from old to new table happens
	/// </summary>
	public bool ListObjectsFromOldNamespaceTable { get; set; } = false;

	/// <summary>
	/// Set to disable any modifications to cassandra schema from Unreal Cloud DDC
	/// </summary>
	public bool AvoidSchemaChanges { get; set; } = false;

	/// <summary>
	/// Runs the record fetching in parallel (up to CountOfNodes) - experimental change that only applies when UsePerShardScanning is active 
	/// </summary>
	public bool AllowParallelRecordFetch { get; set; } = false;

	/// <summary>
	/// Can be used to disable the cleanup of unused namespaces and buckets
	/// </summary>
	public bool CleanupOldTrackedState { get; set; } = true;

	/// <summary>
	/// Used to override the consistency level for requests
	/// </summary>
	public ConsistencyLevel DefaultConsistencyLevel { get; set; } = ConsistencyLevel.LocalOne;

	/// <summary>
	/// When enabled uses a more consistent read level for content ids which solves intermittent issues as the cost of worse performance
	/// </summary>
	public bool UseConsistentContentIdResolve { get; set; } = false;

	public IEnumerable<ValidationResult> Validate(ValidationContext validationContext)
	{
		List<ValidationResult> results = new List<ValidationResult>();

		if (UsePerShardScanning)
		{
			Validator.TryValidateProperty(CountOfNodes, new ValidationContext(this, null, null) { MemberName = nameof(CountOfNodes) }, results);
			Validator.TryValidateProperty(CountOfCoresPerNode, new ValidationContext(this, null, null) { MemberName = nameof(CountOfCoresPerNode) }, results);
		}
		return results;
	}
}
