// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class PeerStatus
	{
		public PeerStatus(PeerSettings peerSettings)
		{
			Endpoints = peerSettings.Endpoints;
		}

		public List<PeerEndpoints> Endpoints { get; }
		public int Latency { get; set; }
		public bool Reachable { get; set; }
	}

	public interface IPeerStatusService
	{
		/// <summary>
		/// Lists all known regions
		/// </summary>
		/// <returns></returns>
		IEnumerable<string> GetRegions();

		/// <summary>
		/// Returns all the information about a particular region
		/// </summary>
		/// <param name="regionName"></param>
		/// <returns></returns>
		PeerStatus? GetPeerStatus(string regionName);

		/// <summary>
		/// Returns possible peers sorted by latency to them
		/// </summary>
		/// <param name="peerNames"></param>
		/// <returns></returns>
		List<(int, string)> GetPeersByLatency(IEnumerable<string> peerNames);
	}

	public class PeerStatusServiceState
	{
	}

	public class PeerStatusService : PollingService<PeerStatusServiceState>, IPeerStatusService
	{
		private readonly IOptionsMonitor<ClusterSettings> _clusterSettings;
		private readonly IOptionsMonitor<JupiterSettings> _jupiterSettings;
		private readonly IHttpClientFactory _clientFactory;
		private readonly Dictionary<string, PeerStatus> _peers = new Dictionary<string, PeerStatus>(StringComparer.InvariantCultureIgnoreCase);
		private volatile bool _alreadyPolling = false;
		private readonly ILogger _logger;

		public IEnumerable<string> GetRegions()
		{
			return _clusterSettings.CurrentValue.Peers.Select(settings => settings.Name);
		}

		public PeerStatus? GetPeerStatus(string regionName)
		{
			if (_peers.TryGetValue(regionName, out PeerStatus? peerStatus))
			{
				return peerStatus;
			}

			return null;
		}

		public List<(int, string)> GetPeersByLatency(IEnumerable<string> peerNames)
		{
			List<(int, string)> peers = new();
			foreach (string peerName in peerNames)
			{
				// skip the local site
				if (string.Equals(peerName, _jupiterSettings.CurrentValue.CurrentSite, StringComparison.OrdinalIgnoreCase))
				{
					continue;
				}

				PeerStatus? peerStatus = GetPeerStatus(peerName);
				if (peerStatus == null)
				{
					continue;
				}

				peers.Add((peerStatus.Latency, peerName));
			}

			peers.SortBy(pair => pair.Item1);
			return peers;
		}

		public PeerStatusService(IOptionsMonitor<ClusterSettings> clusterSettings, IOptionsMonitor<JupiterSettings> jupiterSettings, IHttpClientFactory clientFactory, ILogger<PeerStatusService> logger) : base("PeerStatus", TimeSpan.FromMinutes(15), new PeerStatusServiceState(), logger)
		{
			_clusterSettings = clusterSettings;
			_jupiterSettings = jupiterSettings;
			_clientFactory = clientFactory;
			_logger = logger;

			foreach (PeerSettings peerSettings in clusterSettings.CurrentValue.Peers)
			{
				// skip the local site
				if (string.Equals(peerSettings.Name, jupiterSettings.CurrentValue.CurrentSite, StringComparison.OrdinalIgnoreCase))
				{
					continue;
				}

				_peers[peerSettings.Name] = new PeerStatus(peerSettings)
				{
					Latency = int.MaxValue
				};
			}
		}

		public override async Task<bool> OnPollAsync(PeerStatusServiceState state, CancellationToken cancellationToken)
		{
			if (_alreadyPolling)
			{
				return false;
			}

			_alreadyPolling = true;

			await UpdatePeerStatusAsync(cancellationToken);
			return true;
		}

		public async Task UpdatePeerStatusAsync(CancellationToken cancellationToken)
		{
			foreach (PeerSettings peerSettings in _clusterSettings.CurrentValue.Peers)
			{
				// skip the local site
				if (string.Equals(peerSettings.Name, _jupiterSettings.CurrentValue.CurrentSite, StringComparison.OrdinalIgnoreCase))
				{
					continue;
				}

				PeerStatus status = _peers[peerSettings.Name];

				int bestLatency = int.MaxValue;
				bool reachable = false;

				await Parallel.ForEachAsync(peerSettings.Endpoints, cancellationToken, async (endpoint, token) =>
				{
					int latency = await MeasureLatencyAsync(endpoint);
					bestLatency = Math.Min(latency, bestLatency);

					if (latency != int.MaxValue)
					{
						reachable = true;
					}
				});

				status.Reachable = reachable;
				status.Latency = bestLatency;
			}
		}

		private async Task<int> MeasureLatencyAsync(PeerEndpoints endpoint)
		{
			Stopwatch stopwatch = Stopwatch.StartNew();
			using HttpClient client = _clientFactory.CreateClient();

			// treat any connection that takes more then 5 seconds to establish as timed out
			client.Timeout = TimeSpan.FromSeconds(5);

			string url = endpoint.Url.ToString();
			if (!url.EndsWith("/", StringComparison.OrdinalIgnoreCase))
			{
				url += "/";
			}

			Uri uri = new Uri(url + "health/live");
			try
			{
				HttpResponseMessage result = await client.GetAsync(uri);
				// ignore error responses as they may not have reached the actual instance
				if (!result.IsSuccessStatusCode)
				{
					_logger.LogWarning("Non-success status code ({StatusCode}) when attempting to measure latency to {Endpoint}", result.StatusCode, uri);
					return int.MaxValue;
				}

				return (int)stopwatch.ElapsedMilliseconds;
			}
			catch (Exception e)
			{
				_logger.LogWarning(e, "Exception when attempting to measure latency from {Endpoint}", uri);
				// error reaching the endpoint is just considered to max latency
				return int.MaxValue;
			}
		}
	}
}
