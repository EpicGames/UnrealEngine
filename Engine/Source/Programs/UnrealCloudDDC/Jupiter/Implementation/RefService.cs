// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Serialization;
using Jupiter.Common;
using Jupiter.Implementation.Blob;
using Jupiter.Utils;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class ObjectService : IRefService
	{
		private readonly IHttpContextAccessor _httpContextAccessor;
		private readonly IReferencesStore _referencesStore;
		private readonly IBlobService _blobService;
		private readonly IReferenceResolver _referenceResolver;
		private readonly IReplicationLog _replicationLog;
		private readonly IBlobIndex _blobIndex;
		private readonly INamespacePolicyResolver _namespacePolicyResolver;
		private readonly ILastAccessTracker<LastAccessRecord> _lastAccessTracker;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;
		private readonly IOptionsMonitor<UnrealCloudDDCSettings> _cloudDDCSettings;

		public ObjectService(IHttpContextAccessor httpContextAccessor, IReferencesStore referencesStore, IBlobService blobService, IReferenceResolver referenceResolver, IReplicationLog replicationLog, IBlobIndex blobIndex, INamespacePolicyResolver namespacePolicyResolver, ILastAccessTracker<LastAccessRecord> lastAccessTracker, Tracer tracer, ILogger<ObjectService> logger, IOptionsMonitor<UnrealCloudDDCSettings> cloudDDCSettings)
		{
			_httpContextAccessor = httpContextAccessor;
			_referencesStore = referencesStore;
			_blobService = blobService;
			_referenceResolver = referenceResolver;
			_replicationLog = replicationLog;
			_blobIndex = blobIndex;
			_namespacePolicyResolver = namespacePolicyResolver;
			_lastAccessTracker = lastAccessTracker;
			_tracer = tracer;
			_logger = logger;
			_cloudDDCSettings = cloudDDCSettings;
		}

		public async Task<(RefRecord, BlobContents?)> GetAsync(NamespaceId ns, BucketId bucket, RefId key, string[]? fields = null, bool consistentRead = false, bool doLastAccessTracking = true, CancellationToken cancellationToken = default)
		{
			// if no field filtering is being used we assume everything is needed
			IReferencesStore.FieldFlags flags = IReferencesStore.FieldFlags.All;
			if (fields != null)
			{
				// empty array means fetch all fields
				if (fields.Length == 0)
				{
					flags = IReferencesStore.FieldFlags.All;
				}
				else
				{
					flags = fields.Contains("payload")
						? IReferencesStore.FieldFlags.IncludePayload
						: IReferencesStore.FieldFlags.None;
				}
			}

			IReferencesStore.OperationFlags opFlags = IReferencesStore.OperationFlags.None;
			if (consistentRead)
			{
				opFlags |= IReferencesStore.OperationFlags.BypassCache;
			}

			IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();

			RefRecord o;
			{
				using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope("ref.get", "Fetching Ref from DB");

				o = await _referencesStore.GetAsync(ns, bucket, key, flags, opFlags, cancellationToken);
			}

			if (doLastAccessTracking)
			{
				NamespacePolicy.StoragePoolGCMethod gcPolicy = _namespacePolicyResolver.GetPoliciesForNs(ns).GcMethod ?? NamespacePolicy.StoragePoolGCMethod.LastAccess;
				if (gcPolicy == NamespacePolicy.StoragePoolGCMethod.LastAccess)
				{
					// we do not wait for the last access tracking as it does not matter when it completes
					Task lastAccessTask = _lastAccessTracker.TrackUsed(new LastAccessRecord(ns, bucket, key)).ContinueWith((task, _) =>
					{
						if (task.Exception != null)
						{
							_logger.LogError(task.Exception, "Exception when tracking last access record");
						}
					}, null, TaskScheduler.Current);
				}
			}

			BlobContents? blobContents = null;
			if ((flags & IReferencesStore.FieldFlags.IncludePayload) != 0)
			{
				if (o.InlinePayload != null && o.InlinePayload.Length != 0)
				{
#pragma warning disable CA2000 // Dispose objects before losing scope , ownership is transfered to caller
					blobContents = new BlobContents(o.InlinePayload);
#pragma warning restore CA2000 // Dispose objects before losing scope
				}
				else
				{
					using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope("blob.get", "Downloading blob from store");

					blobContents = await _blobService.GetObjectAsync(ns, o.BlobIdentifier, cancellationToken: cancellationToken);
				}
			}

			return (o, blobContents);
		}

		public async Task<(ContentId[], BlobId[])> PutAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, CbObject payload, Action<BlobId>? onBlobFound = null, bool allowOverwrite = false, CancellationToken cancellationToken = default)
		{
			IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();
			using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope("ref.put", "Inserting ref");

			bool hasReferences = HasAttachments(payload);

			// if we have no references we are always finalized, e.g. there are no referenced blobs to upload
			bool isFinalized = !hasReferences;

			// if inlining is enabled and the blob is small we inline it, use EnableForceSubmitRefBlobToBlobStore(false) to disable the backup submission into the blob store
			byte[] blobPayloadBytes = payload.GetView().ToArray();
			bool putIntoBlobStore = _cloudDDCSettings.CurrentValue.EnableForceSubmitRefBlobToBlobStore;
			bool inlineBlob = _cloudDDCSettings.CurrentValue.EnableInlineSmallBlobs;
			if (inlineBlob && blobPayloadBytes.LongLength > _cloudDDCSettings.CurrentValue.InlineBlobMaxSize)
			{
				// do not inline large blobs, instead put them into the blob store
				blobPayloadBytes = Array.Empty<byte>();
				putIntoBlobStore = true;
			}
			else if (!inlineBlob)
			{
				// if we are never inlining the blob then remove the payload bytes from the ref entry and just put them in the blob store
				blobPayloadBytes = Array.Empty<byte>();
				putIntoBlobStore = true;
			}

			Task objectStorePut = _referencesStore.PutAsync(ns, bucket, key, blobHash, blobPayloadBytes, isFinalized, cancellationToken: cancellationToken, allowOverwrite: allowOverwrite);

			Task<BlobId>? blobStorePut = null;
			if (putIntoBlobStore)
			{
				blobStorePut = _blobService.PutObjectAsync(ns, payload.GetView().ToArray(), blobHash, bucketHint: bucket, bypassCache: false, cancellationToken: cancellationToken);
			}

			await objectStorePut;
			if (blobStorePut != null)
			{
				await blobStorePut;
			}

			return await DoFinalizeAsync(ns, bucket, key, blobHash, payload, onBlobFound, cancellationToken);
		}

		private bool HasAttachments(CbObject payload)
		{
			bool FieldHasAttachments(CbField field)
			{
				if (field.IsObject())
				{
					bool hasAttachment = HasAttachments(field.AsObject());
					if (hasAttachment)
					{
						return true;
					}
				}

				if (field.IsArray())
				{
					foreach (CbField subField in field.AsArray())
					{
						bool hasAttachment = FieldHasAttachments(subField);
						if (hasAttachment)
						{
							return true;
						}
					}
				}

				return field.IsAttachment();
			}

			return payload.Any(FieldHasAttachments);
		}

		public async Task<(ContentId[], BlobId[])> FinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, Action<BlobId>? onBlobFound = null, CancellationToken cancellationToken = default)
		{
			// finalize is intended to verify the state of the object in the db, so we bypass any caches to make sure the object we are working on is not stale
			(RefRecord o, BlobContents? blob) = await GetAsync(ns, bucket, key, consistentRead: true, cancellationToken: cancellationToken);
			if (blob == null)
			{
				throw new InvalidOperationException("No blob when attempting to finalize");
			}

			byte[] blobContents = await blob.Stream.ToByteArrayAsync(cancellationToken);
			CbObject payload = new CbObject(blobContents);

			if (!o.BlobIdentifier.Equals(blobHash))
			{
				throw new ObjectHashMismatchException(ns, bucket, key, blobHash, o.BlobIdentifier);
			}

			return await DoFinalizeAsync(ns, bucket, key, blobHash, payload, onBlobFound, cancellationToken);
		}

		private async Task<(ContentId[], BlobId[])> DoFinalizeAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blobHash, CbObject payload, Action<BlobId>? onBlobFound = null, CancellationToken cancellationToken = default)
		{
			IServerTiming? serverTiming = _httpContextAccessor.HttpContext?.RequestServices.GetService<IServerTiming>();
			using ServerTimingMetricScoped? serverTimingScope = serverTiming?.CreateServerTimingMetricScope("ref.finalize", "Finalizing the ref");

			Task addRefToBlobsTask = _blobIndex.AddRefToBlobsAsync(ns, bucket, key, new[] { blobHash }, cancellationToken);
			Task addToBucketListTask = _cloudDDCSettings.CurrentValue.EnableBucketStatsTracking
				? _blobIndex.AddBlobToBucketListAsync(ns, bucket, key, blobHash, (long)payload.GetView().Length, cancellationToken)
				: Task.CompletedTask;
			ContentId[] missingReferences = Array.Empty<ContentId>();
			BlobId[] missingBlobs = Array.Empty<BlobId>();

			bool hasReferences = HasAttachments(payload);
			if (hasReferences)
			{
				using TelemetrySpan _ = _tracer.StartActiveSpan("ObjectService.ResolveReferences").SetAttribute("operation.name", "ObjectService.ResolveReferences");
				try
				{
					IAsyncEnumerable<BlobId> references = _referenceResolver.GetReferencedBlobsAsync(ns, payload, cancellationToken: cancellationToken);

					await Parallel.ForEachAsync(references, new ParallelOptions {CancellationToken = cancellationToken, MaxDegreeOfParallelism = _cloudDDCSettings.CurrentValue.RefFinalizeResolveMaxParallel},async (blobId, token) =>
					{
						Task blobIndexAddTask = _blobIndex.AddRefToBlobsAsync(ns, bucket, key, new BlobId[] { blobId }, cancellationToken);

						onBlobFound?.Invoke(blobId);
						if (_cloudDDCSettings.CurrentValue.EnableBucketStatsTracking)
						{
							// if a blob is missing its not an error, the finalize will report this as missing and it will be uploaded and finalize ran again
							try
							{
								BlobMetadata result = await _blobService.GetObjectMetadataAsync(ns, blobId, cancellationToken);
								await _blobIndex.AddBlobToBucketListAsync(ns, bucket, key, blobId, result.Length, cancellationToken);
							}
							catch (BlobNotFoundException)
							{
							}
						}

						await blobIndexAddTask;
					});
				}
				catch (PartialReferenceResolveException e)
				{
					missingReferences = e.UnresolvedReferences.ToArray();
				}
				catch (ReferenceIsMissingBlobsException e)
				{
					missingBlobs = e.MissingBlobs.ToArray();
				}
			}

			await Task.WhenAll(addRefToBlobsTask, addToBucketListTask);

			if (missingReferences.Length == 0 && missingBlobs.Length == 0)
			{
				await _referencesStore.FinalizeAsync(ns, bucket, key, blobHash, cancellationToken);
				await _replicationLog.InsertAddEventAsync(ns, bucket, key, blobHash);
			}

			return (missingReferences, missingBlobs);
		}

		public IAsyncEnumerable<NamespaceId> GetNamespacesAsync(CancellationToken cancellationToken)
		{
			return _referencesStore.GetNamespacesAsync(cancellationToken);
		}

		public IAsyncEnumerable<BucketId> GetBucketsAsync(NamespaceId ns, CancellationToken cancellationToken = default)
		{
			return _referencesStore.GetBucketsAsync(ns, cancellationToken);
		}

		public async Task<bool> DeleteAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			RefRecord refRecord = await _referencesStore.GetAsync(ns, bucket, key, IReferencesStore.FieldFlags.None, IReferencesStore.OperationFlags.BypassCache, cancellationToken);
			await _blobIndex.RemoveReferencesAsync(ns, refRecord.BlobIdentifier, new List<BaseBlobReference> { new RefBlobReference(refRecord.BlobIdentifier, bucket, key) }, cancellationToken);
			return await _referencesStore.DeleteAsync(ns, bucket, key, cancellationToken);
		}

		public Task<long> DropNamespaceAsync(NamespaceId ns, CancellationToken cancellationToken)
		{
			return _referencesStore.DropNamespaceAsync(ns, cancellationToken);
		}

		public Task<long> DeleteBucketAsync(NamespaceId ns, BucketId bucket, CancellationToken cancellationToken)
		{
			return _referencesStore.DeleteBucketAsync(ns, bucket, cancellationToken);
		}

		public async Task<bool> ExistsAsync(NamespaceId ns, BucketId bucket, RefId key, CancellationToken cancellationToken)
		{
			try
			{
				(RefRecord, BlobContents?) _ = await GetAsync(ns, bucket, key, new string[] { "name" }, doLastAccessTracking: false, consistentRead: true, cancellationToken: cancellationToken);
			}
			catch (NamespaceNotFoundException)
			{
				return false;
			}
			catch (BlobNotFoundException)
			{
				return false;
			}
			catch (RefNotFoundException)
			{
				return false;
			}
			catch (PartialReferenceResolveException)
			{
				return false;
			}
			catch (ReferenceIsMissingBlobsException)
			{
				return false;
			}

			return true;
		}

		public async Task<List<BlobId>> GetReferencedBlobsAsync(NamespaceId ns, BucketId bucket, RefId name, bool ignoreMissingBlobs = false, CancellationToken cancellationToken = default)
		{
			byte[] blob;
			RefRecord o = await _referencesStore.GetAsync(ns, bucket, name, IReferencesStore.FieldFlags.IncludePayload, IReferencesStore.OperationFlags.None, cancellationToken);
			if (o.InlinePayload != null && o.InlinePayload.Length != 0)
			{
				blob = o.InlinePayload;
			}
			else
			{
				BlobContents blobContents = await _blobService.GetObjectAsync(ns, o.BlobIdentifier, cancellationToken: cancellationToken);
				blob = await blobContents.Stream.ToByteArrayAsync(cancellationToken);
			}

			CbObject cbObject = new CbObject(blob);

			List<BlobId> referencedBlobs = await _referenceResolver.GetReferencedBlobsAsync(ns, cbObject, ignoreMissingBlobs, cancellationToken).ToListAsync(cancellationToken);
			return referencedBlobs;
		}

		public Task UpdateTTL(NamespaceId ns, BucketId bucket, RefId refId, uint ttl, CancellationToken cancellationToken = default)
		{
			return _referencesStore.UpdateTTL(ns, bucket, refId, ttl, cancellationToken);
		}
	}
}
