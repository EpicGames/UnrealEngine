// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net.Http;
using System.Threading.Tasks;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public abstract class RelayStore
	{
		private readonly IOptionsMonitor<UpstreamRelaySettings> _settings;
		private readonly IServiceCredentials _serviceCredentials;
		private readonly HttpClient _httpClient;

		protected HttpClient HttpClient => _httpClient;

		protected RelayStore(IOptionsMonitor<UpstreamRelaySettings> settings, IHttpClientFactory httpClientFactory, IServiceCredentials serviceCredentials)
		{
			_settings = settings;
			_serviceCredentials = serviceCredentials;

			_httpClient = httpClientFactory.CreateClient();
			_httpClient.BaseAddress = new Uri(_settings.CurrentValue.ConnectionString);
		}

		protected async Task<HttpRequestMessage> BuildHttpRequestAsync(HttpMethod method, Uri uri)
		{
			string? token = await _serviceCredentials.GetTokenAsync();
			HttpRequestMessage request = new HttpRequestMessage(method, uri);
			if (!string.IsNullOrEmpty(token))
			{
				request.Headers.Add("Authorization", $"{_serviceCredentials.GetAuthenticationScheme()} {token}");
			}

			return request;
		}
	}
}
