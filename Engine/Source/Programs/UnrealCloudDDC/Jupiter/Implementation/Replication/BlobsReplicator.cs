// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.Metrics;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Runtime.CompilerServices;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Controllers;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class BlobsReplicator : IReplicator
	{
		private readonly string _name;
		private readonly ILogger _logger;
		private readonly ManualResetEvent _replicationFinishedEvent = new ManualResetEvent(true);
		private readonly ReplicatorSettings _replicatorSettings;
		private readonly IBlobService _blobService;
		private readonly IReplicationLog _replicationLog;
		private readonly IServiceCredentials _serviceCredentials;
		private readonly Tracer _tracer;
		private readonly BufferedPayloadFactory _bufferedPayloadFactory;
		private readonly HttpClient _httpClient;
		private readonly NamespaceId _namespace;
		private BlobsState _replicationState;
		private bool _replicationRunning;
		private CancellationTokenSource _replicationTokenSource = new CancellationTokenSource();
		private bool _disposed = false;

		private static Histogram<long>? s_replicatedCounter;
		private static Histogram<long>? s_replicationBehindCounter;
		private static Counter<long>? s_replicationAttempts;

		private static readonly JsonSerializerOptions DefaultSerializerSettings = ConfigureJsonOptions();
		public BlobsReplicator(ReplicatorSettings replicatorSettings, ClusterSettings clusterSettings, IBlobService blobService, IHttpClientFactory httpClientFactory, IReplicationLog replicationLog, IServiceCredentials serviceCredentials, Tracer tracer, BufferedPayloadFactory bufferedPayloadFactory,ILogger<BlobsReplicator> logger, Meter meter)
		{
			_name = replicatorSettings.ReplicatorName;
			_namespace = new NamespaceId(replicatorSettings.NamespaceToReplicate);
			_replicatorSettings = replicatorSettings;
			_blobService = blobService;
			_replicationLog = replicationLog;
			_serviceCredentials = serviceCredentials;
			_tracer = tracer;
			_bufferedPayloadFactory = bufferedPayloadFactory;
			_logger = logger;

			_httpClient = httpClientFactory.CreateClient();

			// Prefer to look up the base address using the site name
			if (replicatorSettings.SourceSite != null)
			{
				foreach (PeerSettings peerSettings in clusterSettings.Peers)
				{
					if (peerSettings.Name == replicatorSettings.SourceSite && peerSettings.Endpoints.Count > 0)
					{
						_httpClient.BaseAddress = peerSettings.Endpoints.First().Url;
						break;
					}
				}

				if (_httpClient.BaseAddress == null)
				{
					throw new Exception($"No replicator source base address could be determined for peer {replicatorSettings.SourceSite}");
				}
			}
			else
			{
				_httpClient.BaseAddress = new Uri(replicatorSettings.ConnectionString);
			}

			ReplicatorState? replicatorState = _replicationLog.GetReplicatorStateAsync(_namespace, _name).Result;
			if (replicatorState == null)
			{
				_replicationState = new BlobsState();
			}
			else
			{
				_replicationState = new BlobsState()
				{
					LastBucket = replicatorState.LastBucket,
				};
			}

			Info = new ReplicatorInfo(replicatorSettings.ReplicatorName, _namespace, _replicationState);

			if (s_replicatedCounter == null)
			{
				s_replicatedCounter = meter.CreateHistogram<long>("blob-replication.active");
			}
			if (s_replicationBehindCounter == null)
			{
				s_replicationBehindCounter = meter.CreateHistogram<long>("blob-replication.queue-length");
			}
			if (s_replicationAttempts == null)
			{
				s_replicationAttempts = meter.CreateCounter<long>("blob-replication.attempts");
			}
		}

		private static JsonSerializerOptions ConfigureJsonOptions()
		{
			JsonSerializerOptions options = new JsonSerializerOptions();
			BaseStartup.ConfigureJsonOptions(options);
			return options;
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (_disposed)
			{
				return;
			}

			if (disposing)
			{
				_replicationFinishedEvent.WaitOne();
				_replicationFinishedEvent.Dispose();
				_replicationTokenSource.Dispose();

				_httpClient.Dispose();
			}

			_disposed = true;
		}

		/// <summary>
		/// Updates the state of the current replicator using the provided new state.
		/// 
		/// If an expected old state is provided, then ensures that the existing state matches prior to saving the new state. This allows the admin
		/// endpoints to rewind or fast-forward the replicator's state without worrying about being overwritten.
		///
		/// If an existing state does not exist at all, then the new state will be inserted regardless of the provided expected old state.
		/// </summary>
		/// <param name="newState">New state to persist</param>
		/// <param name="expectedOldState">Optional old state that is expected to exist before persisting the new state</param>
		private async Task<bool> SaveStateAsync(BlobsState newState, BlobsState? expectedOldState = null)
		{
			if (expectedOldState != null)
			{
				ReplicatorState? existingState = await _replicationLog.GetReplicatorStateAsync(Info.NamespaceToReplicate, _name);

				if (existingState != null && existingState.LastBucket != expectedOldState.LastBucket)
				{
					_logger.LogWarning("Existing replicator state for {Name} does not match expected value ({Expected} != {Actual}). Skipping persistence of new state.", _name, expectedOldState.LastBucket, existingState.LastBucket);
					return false;
				}
			}
			
			await _replicationLog.UpdateReplicatorStateAsync(Info.NamespaceToReplicate, _name, new ReplicatorState { LastBucket = newState.LastBucket });
			return true;
		}

		public async Task<bool> TriggerNewReplicationsAsync()
		{
			if (_replicationRunning)
			{
				_logger.LogDebug("Skipping replication of replicator: {Name} as it was already running.", _name);
				return false;
			}

			bool hasRun;
			int countOfReplicationsDone = 0;

			try
			{
				// read the state again to allow it to be modified by the admin controller / other instances of jupiter connected to the same filesystem
				ReplicatorState? replicatorState = await _replicationLog.GetReplicatorStateAsync(_namespace, _name);
				if (replicatorState == null)
				{
					_replicationState = new BlobsState();
				}
				else
				{
					_replicationState = new BlobsState()
					{
						LastBucket = replicatorState.LastBucket,
					};
				}

				_logger.LogDebug("Read Replication state for replicator: {Name}. {LastBucket} {LastEvent}.", _name, _replicationState.LastBucket, _replicationState.LastEvent);

				LogReplicationHeartbeat();
				
				_logger.LogDebug("Replicator: {Name} is starting a run.", _name);

				if (!_replicationTokenSource.TryReset())
				{
					_replicationTokenSource.Dispose();
					_replicationTokenSource = new CancellationTokenSource();
				}
				
				_replicationRunning = true;
				_replicationFinishedEvent.Reset();
				CancellationToken replicationToken = _replicationTokenSource.Token;

				NamespaceId ns = _namespace;

				Info.CountOfRunningReplications = 0;

				string? lastBucket = _replicationState.LastBucket;
				if (lastBucket == null)
				{
					// if we have not run start from a bucket 7 days ago and replicate from there
					lastBucket = DateTime.UtcNow.AddDays(-7).ToReplicationBucket().ToReplicationBucketIdentifier();
				}

				try
				{
					countOfReplicationsDone += await ReplicateIncrementallyAsync(ns, lastBucket, replicationToken);
				}
				catch (ToOldTimeBucketException)
				{
					_logger.LogError("Encountered old bucket in use, resetting to replicating only the last weeks data. Regions will likely be inconsistent.");
					string bucket = DateTime.UtcNow.AddDays(-7).ToReplicationBucket().ToReplicationBucketIdentifier();
					_replicationState.LastBucket = bucket;
					await SaveStateAsync(_replicationState);
				}

				hasRun = countOfReplicationsDone != 0;
			}
			finally
			{
				_replicationRunning = false;
				_replicationFinishedEvent.Set();
			}

			_logger.LogDebug("Replicator: {Name} finished its replication run. Replications completed: {ReplicationsDone} .", _name, countOfReplicationsDone);

			return hasRun;
		}

		private void LogReplicationHeartbeat()
		{
			s_replicationAttempts?.Add(1, new KeyValuePair<string, object?>("replicator", _name), new KeyValuePair<string, object?>("namespace", _namespace));
		}

		private async Task<HttpRequestMessage> BuildHttpRequestAsync(HttpMethod httpMethod, Uri uri)
		{
			string? token = await _serviceCredentials.GetTokenAsync();
			HttpRequestMessage request = new HttpRequestMessage(httpMethod, uri);
			if (!string.IsNullOrEmpty(token))
			{
				request.Headers.Add("Authorization", $"{_serviceCredentials.GetAuthenticationScheme()} {token}");
			}

			return request;
		}

		private async Task<int> ReplicateIncrementallyAsync(NamespaceId ns, string lastBucket, CancellationToken replicationToken)
		{
			int countOfReplicationsDone = 0;

			_logger.LogInformation("{Name} Starting blob replication maxParallelism: {MaxParallelism} Last Bucket {LastBucket}", _name, _replicatorSettings.MaxParallelReplications, lastBucket);

			if (replicationToken.IsCancellationRequested)
			{
				return countOfReplicationsDone;
			}

			// log the metrics when we start / have nothing to do to make sure there is data in these most of the time
			int timeBucketsBehindStart = (int)(DateTime.UtcNow - lastBucket.FromReplicationBucketIdentifier()).TotalMinutes / 5;
			s_replicationBehindCounter?.Record(timeBucketsBehindStart, new KeyValuePair<string, object?>("replicator", _name), new KeyValuePair<string, object?>("namespace", _namespace));
			s_replicatedCounter?.Record(countOfReplicationsDone, new KeyValuePair<string, object?>("replicator", _name), new KeyValuePair<string, object?>("namespace", _namespace));

			foreach (string refBucket in GetRefBuckets(lastBucket, replicationToken))
			{
				if (replicationToken.IsCancellationRequested)
				{
					break;
				}

				LogReplicationHeartbeat();

				await Parallel.ForEachAsync(GetBlobEventsAsync(ns, refBucket, replicationToken),
					new ParallelOptions { MaxDegreeOfParallelism = _replicatorSettings.MaxParallelReplications, CancellationToken = replicationToken },
					async (BlobReplicationLogEvent @event, CancellationToken ctx) =>
				{
					if (ctx.IsCancellationRequested)
					{
						return;
					}

					using TelemetrySpan scope = _tracer.StartActiveSpan("replicator.replicate_blob")
						.SetAttribute("operation.name", "replicator.replicate_blob")
						.SetAttribute("resource.name", $"{ns}.{@event.Blob}")
						.SetAttribute("namespace", ns.ToString())
						.SetAttribute("time-bucket", refBucket);

					_logger.LogDebug("{Name} New transaction to replicate found. Blob: {Namespace} {Blob} ({Bucket}) in {TimeBucket} ({TimeDate}).", _name, @event.Namespace, @event.Blob, @event.BucketHint.HasValue ? @event.BucketHint.Value : "Unknown-Bucket", @event.TimeBucket, @event.Timestamp);

					Interlocked.Increment(ref countOfReplicationsDone);

					try
					{
						bool blobWasReplicated = false;
						// we do not need to replicate delete events
						if (@event.Op != BlobReplicationLogEvent.OpType.Deleted)
						{
							blobWasReplicated = await ReplicateBlobAsync(@event.Namespace, @event.Blob, @event.BucketHint, ctx);
						}

						if (blobWasReplicated)
						{
							await AddToBlobReplicationLogAsync(@event.Namespace, @event.Blob, @event.BucketHint);
						}
					}
					catch (BlobNotFoundException)
					{
						_logger.LogWarning("{Name} Failed to replicate {@Op} in {Namespace} because blob was not present in remote store. Skipping.", _name, @event, Info.NamespaceToReplicate);
					}
				});

				// finished replicating bucket
				DateTime timestamp = refBucket.FromReplicationBucketIdentifier();

				_logger.LogDebug("{Name} finished enumeration of bucket: {EventBucket}", _name, refBucket);

				// if the bucket was recently created we do not store that we replicate it, thus we will replicate it again to avoid risks with missing recent blobs that gets added after we queries the time bucket
				bool isRecentBucket = timestamp > DateTime.UtcNow.AddMinutes(-10);

				if (isRecentBucket)
				{
					_logger.LogInformation("Reached recent bucket {BucketName} when replicating {Name}, will run replication again to ensure consistency as this bucket may change again", refBucket, _name);
				}
				else
				{
					// we have replicated everything up to a point and can persist this in the state
					BlobsState previousState = new BlobsState() { LastBucket = _replicationState.LastBucket };
					_replicationState.LastBucket = refBucket;

					if (!await SaveStateAsync(_replicationState, expectedOldState: previousState))
					{
						_logger.LogWarning("Replication state for {Name} has changed during processing. Aborting.", _name);
						break;
					}
				}

				Info.LastRun = DateTime.Now;
				int timeBucketsBehind = (int)(DateTime.UtcNow - timestamp).TotalMinutes / 5;
				s_replicationBehindCounter?.Record(timeBucketsBehind, new KeyValuePair<string, object?>("replicator", _name), new KeyValuePair<string, object?>("namespace", _namespace));
				s_replicatedCounter?.Record(countOfReplicationsDone, new KeyValuePair<string, object?>("replicator", _name), new KeyValuePair<string, object?>("namespace", _namespace));

				_logger.LogInformation("{Name} replicated all events up to {Time} . Bucket: {EventBucket}", _name, timestamp, refBucket);
			}

			return countOfReplicationsDone;
		}

		private async IAsyncEnumerable<BlobReplicationLogEvent> GetBlobEventsAsync(NamespaceId ns, string refBucket, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			if (cancellationToken.IsCancellationRequested)
			{
				yield break;
			}

			HttpResponseMessage? response = null;
			Exception? lastException = null;
			const int RetryAttempts = 3;
			for (int i = 0; i < RetryAttempts; i++)
			{
				using HttpRequestMessage request = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"/api/v1/replication-log/blobs/{ns}/{refBucket}", UriKind.Relative));

				try
				{
					response = await _httpClient.SendAsync(request, cancellationToken);
					break;
				}
				catch (HttpRequestException e)
				{
					response = null;

					if (e.StatusCode == HttpStatusCode.TooManyRequests)
					{
						continue;
					}
					// rethrow unknown exceptions
					if (e.InnerException is not IOException)
					{
						throw;
					}

					lastException = e;
				}
				catch (InvalidOperationException)
				{
					_logger.LogError("Invalid operation when calling requesting blobs for {Namespace} {RefBucket}", ns, refBucket);
					throw;
				}
			}

			if (response == null)
			{
				throw new Exception("Ref response never set", lastException);
			}

			string body = await response.Content.ReadAsStringAsync(cancellationToken);
			if (cancellationToken.IsCancellationRequested)
			{
				yield break;
			}

			if (response.StatusCode == HttpStatusCode.BadRequest)
			{
				ProblemDetails? problemDetails = JsonSerializer.Deserialize<ProblemDetails>(body, DefaultSerializerSettings);
				if (problemDetails == null)
				{
					throw new Exception($"Unknown bad request body when reading incremental replication log. Body: {body}");
				}

				if (problemDetails.Type == ProblemTypes.UseSnapshot)
				{
					ProblemDetailsWithSnapshots? problemDetailsWithSnapshots = JsonSerializer.Deserialize<ProblemDetailsWithSnapshots>(body, DefaultSerializerSettings);

					if (problemDetailsWithSnapshots == null)
					{
						throw new Exception($"Unable to cast the problem details to a snapshot version. Body: {body}");
					}

					BlobId snapshotBlob = problemDetailsWithSnapshots.SnapshotId;
					NamespaceId? blobNamespace = problemDetailsWithSnapshots.BlobNamespace;
					throw new UseSnapshotException(snapshotBlob, blobNamespace!.Value);
				}

				if (problemDetails.Type == ProblemTypes.NoDataFound)
				{
					throw new ToOldTimeBucketException(refBucket);
				}
				throw new Exception($"Unknown bad request response. Body: {body}");
			}
			else if (response.StatusCode == HttpStatusCode.NotFound)
			{
				throw new NamespaceNotFoundException(ns);
			}

			response.EnsureSuccessStatusCode();
			BlobReplicationLogEvents? replicationLogEvents = JsonSerializer.Deserialize<BlobReplicationLogEvents>(body, DefaultSerializerSettings);
			if (replicationLogEvents == null)
			{
				throw new Exception($"Unknown error when deserializing replication log events {ns} {refBucket}");
			}

			if (replicationLogEvents.Events == null)
			{
				throw new Exception($"Unknown error when deserializing replication log events {ns} {refBucket} as events were empty. Body was: {body}");
			}

			foreach (BlobReplicationLogEvent logEvent in replicationLogEvents.Events)
			{
				if (cancellationToken.IsCancellationRequested)
				{
					yield break;
				}

				yield return logEvent;
			}
		}

		private static IEnumerable<string> GetRefBuckets(string bucket, CancellationToken cancellationToken)
		{
			DateTime bucketTimestamp = bucket.FromReplicationBucketIdentifier();

			while (bucketTimestamp < DateTime.UtcNow)
			{
				if (cancellationToken.IsCancellationRequested)
				{
					yield break;
				}

				bucketTimestamp = bucketTimestamp.AddMinutes(5.0);
				if (bucketTimestamp > DateTime.UtcNow)
				{
					yield break;
				}

				yield return bucketTimestamp.ToReplicationBucketIdentifier();
			}
		}

		private async Task<bool> ReplicateBlobAsync(NamespaceId ns, BlobId blob, BucketId? bucketHint, CancellationToken cancellationToken)
		{
			// the point of the replicator is to transfer the blobs locally, thus we ignore if they exist in remote locations
			bool exists = await _blobService.ExistsAsync(ns, blob, storageLayers: null, ignoreRemoteBlobs: true, cancellationToken: cancellationToken);
			if (exists)
			{
				_logger.LogDebug("Not replicating blob {Blob} in {Namespace} as it already existed.", blob, ns);
				return false;
			}

			_logger.LogDebug("Attempting to replicate blob {Blob} in {Namespace}.", blob, ns);

			if (_replicatorSettings.EnableMultipartReplications)
			{
				return await ReplicateMultipartBlobAsync(ns, blob, bucketHint, cancellationToken);
			}
			
			return await ReplicateBlobHttpAsync(ns, blob, bucketHint, cancellationToken);
		}

		private async Task<bool> ReplicateMultipartBlobAsync(NamespaceId ns, BlobId blob, BucketId? bucketHint, CancellationToken cancellationToken)
		{
			const int RetryAttempts = 5;
			Exception? lastException = null;
			for (int retryAttempts = 0; retryAttempts < RetryAttempts; retryAttempts++)
			{
				lastException = null;
				
				try
				{
					const long InitialRangeFrom = 0;
					// we just assume 32 MB chunks right now, the chunk size doesn't matter much but its ideal if we stick to what the root backing store wants, which is 32 MB for S3, this should likely be configurable
					ulong chunkSize = 32 * 1024 * 1024 ;
					long initialRangeTo = (long)chunkSize - 1;
					
					using HttpRequestMessage blobRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/blobs/{ns}/{blob}", UriKind.Relative));
					blobRequest.Headers.Range = new RangeHeaderValue(InitialRangeFrom, initialRangeTo);
					
					using HttpResponseMessage blobResponse = await _httpClient.SendAsync(blobRequest, HttpCompletionOption.ResponseHeadersRead, cancellationToken);

					if (blobResponse.StatusCode == HttpStatusCode.NotFound)
					{
						_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to it not existing.", blob, ns);
						return false;
					}

					// Requested blob is smaller than the initial range. Fallback to request the entire blob
					if (blobResponse.StatusCode == HttpStatusCode.RequestedRangeNotSatisfiable)
					{
						return await ReplicateBlobHttpAsync(ns, blob, bucketHint, cancellationToken);
					}

					// Server didn't respect the range request for some reason, but successfully retrieved the content
					if (blobResponse.StatusCode == HttpStatusCode.OK)
					{
						await using Stream s = await blobResponse.Content.ReadAsStreamAsync(cancellationToken);
						long? contentLength = blobResponse.Content.Headers.ContentLength;

						if (contentLength == null)
						{
							throw new Exception("Expected content-length on blob response");
						}

						using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromStreamAsync(s, contentLength.Value, "blob-replicator", cancellationToken);
						await _blobService.PutObjectAsync(ns, payload, blob, bucketHint: bucketHint, cancellationToken: cancellationToken);

						return true;
					}

					if (blobResponse.StatusCode != HttpStatusCode.PartialContent)
					{
						_logger.LogError("Bad http response when replicating {Blob} in {Namespace} . Status code: {StatusCode}", blob, ns, blobResponse.StatusCode);
						return false;
					}
					
					// Received the first chunk. Use response headers to determine how to parallelize the rest
					ContentRangeHeaderValue responseRange = blobResponse.Content.Headers.ContentRange!;

					if (responseRange.From != InitialRangeFrom || responseRange.To > initialRangeTo)
					{
						throw new Exception("Response range does not match requested range");
					}

					if (!responseRange.HasLength)
					{
						throw new Exception("Expected length in Content-Range header");
					}
					
					ulong blobSize = (ulong)responseRange.Length!;
					int partsCount = (int)(blobSize / chunkSize) + (blobSize % chunkSize > 0 ? 1 : 0);

					IBufferedPayload[] parts = new IBufferedPayload[partsCount];
					long[] lengths = new long[partsCount];

					try
					{
						await using Stream blobStream = await blobResponse.Content.ReadAsStreamAsync(cancellationToken);
						long? contentLength = blobResponse.Content.Headers.ContentLength;

						if (contentLength == null)
						{
							throw new Exception("Expected content-length on blob response");
						}

						{
							parts[0] = await _bufferedPayloadFactory.CreateFromStreamAsync(blobStream, contentLength.Value, "blob-replicator", cancellationToken);
							lengths[0] = contentLength.Value;
						}

						// multipart object, download it in parts, we already have the first part
						await Parallel.ForEachAsync(Enumerable.Range(2, partsCount-1), cancellationToken, async (part, token) =>
						{
							ulong firstByte = chunkSize * ((ulong)part - 1);
							ulong lastByte = Math.Min(chunkSize * (ulong)part - 1, (ulong)blobSize -1);

							// part starts with part 1 which we skipped
							MultipartByteRange range = new MultipartByteRange()
							{
								FirstByte = firstByte, LastByte = lastByte, PartId = part.ToString()
							};
							IBufferedPayload? partPayload = await ReplicateMultipartBlobRangeHttpAsync(ns, blob, range, token);

							parts[part-1] = partPayload ?? throw new NotImplementedException($"Got null part for blob {blob} and part {part}");
							lengths[part-1] = partPayload.Length;
						});

						long totalSize = lengths.Sum();
						using FilesystemBufferedPayloadWriter writer = _bufferedPayloadFactory.CreateFilesystemBufferedPayloadWriter("blob-replicator-combine");

						{
							await using Stream writableStream = writer.GetWritableStream();
							foreach (IBufferedPayload part in parts)
							{
								await using Stream partStream = part.GetStream();
								await partStream.CopyToAsync(writableStream, cancellationToken);
							}
						}
						using IBufferedPayload combinedPayload = writer.Done();
						if (totalSize != combinedPayload.Length)
						{
							throw new Exception($"Multipart replicated blob did not match expected size of {totalSize} was instead {combinedPayload.Length}");
						}
						await _blobService.PutObjectAsync(ns, combinedPayload, blob, bucketHint: bucketHint, cancellationToken: cancellationToken);
					}
					finally
					{
						for (int i = 0; i < partsCount; i++)
						{
							// ReSharper disable once ConditionIsAlwaysTrueOrFalseAccordingToNullableAPIContract
							// This should never be null but during exceptions we have seen it be null so to avoid throwing more exceptions that hide the original exceptions we just null check here
							if (parts[i] != null)
							{
								parts[i].Dispose();
							}
						}
					}

					// break retry loop, we were successful
					break;
				}
				catch (AggregateException ae)
				{
					ae.Handle(e =>
					{
						if (e is not IOException or HttpIOException)
						{
							_logger.LogError(e, "Failed to replicate {Blob} in {Namespace} due to unknown exception", blob, ns);
							return false;
						}
						
						_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to retriable exception {Exception}", blob, ns, e);
						lastException = e;
						return true;
					});
				}
				catch (HttpRequestException e)
				{
					// rethrow unknown exceptions
					if (e.InnerException is not IOException && e.InnerException is not HttpIOException)
					{
						_logger.LogError(e, "Failed to replicate {Blob} in {Namespace} due to unknown exception", blob, ns);
						throw;
					}

					// is an io exception that we should retry
					lastException = e;

					if (e.StatusCode == HttpStatusCode.NotFound)
					{
						// delay the retry with a second if blob is missing, as its really not expected for it to actually miss, likely we just haven't reached consistency on the fact that it does exist
						await Task.Delay(1000, cancellationToken);
					}

					_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to retriable exception {Exception}.", blob, ns, e);
				}
			}

			if (lastException != null)
			{
				throw lastException;
			}

			return true;
		}

		private async Task<IBufferedPayload?> ReplicateMultipartBlobRangeHttpAsync(NamespaceId ns, BlobId blob, MultipartByteRange range, CancellationToken cancellationToken)
		{
			_logger.LogDebug("Retrieving blob {Blob} chunk range {Start}-{End} in {Namespace}.", blob, range.FirstByte, range.LastByte, ns);

			const int RetryAttempts = 5;
			Exception? lastException = null;
			for (int i = 0; i < RetryAttempts; i++)
			{
				lastException = null;

				try
				{
					using HttpRequestMessage blobRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/blobs/{ns}/{blob}", UriKind.Relative));
					blobRequest.Headers.Range = new RangeHeaderValue((long)range.FirstByte, (long)range.LastByte);

					using HttpResponseMessage blobResponse = await _httpClient.SendAsync(blobRequest, HttpCompletionOption.ResponseHeadersRead, cancellationToken);

					if (blobResponse.StatusCode != HttpStatusCode.PartialContent)
					{
						_logger.LogError("Bad http response when retrieving {Blob} chunk range {Start}-{End} in {Namespace} . Status code: {StatusCode}", blob, range.FirstByte, range.LastByte, ns, blobResponse.StatusCode);
					}
					
					ContentRangeHeaderValue responseRange = blobResponse.Content.Headers.ContentRange!;

					if (responseRange.From != (long)range.FirstByte || responseRange.To != (long)range.LastByte)
					{
						throw new Exception("Response range does not match requested range");
					}

					await using Stream s = await blobResponse.Content.ReadAsStreamAsync(cancellationToken);
					long? contentLength = blobResponse.Content.Headers.ContentLength;

					if (contentLength == null)
					{
						throw new Exception("Expected content-length on blob response");
					}

					IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromStreamAsync(s, contentLength.Value, "blob-replicator-part", cancellationToken);
					return payload;
				}
				catch (AggregateException ae)
				{
					ae.Handle(e =>
					{
						if (e is not IOException or HttpIOException)
						{
							_logger.LogError(e, "Failed to replicate {Blob} in {Namespace} due to unknown exception", blob, ns);
							return false;
						}
						
						_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to retriable exception {Exception}", blob, ns, e);
						lastException = e;
						return true;
					});
				}
				catch (HttpRequestException e)
				{
					// rethrow unknown exceptions
					if (e.InnerException is not IOException && e.InnerException is not HttpIOException)
					{
						_logger.LogError(e, "Failed to replicate {Blob} in {Namespace} due to unknown exception", blob, ns);
						throw;
					}

					// is an io exception that we should retry
					lastException = e;

					if (e.StatusCode == HttpStatusCode.NotFound)
					{
						// delay the retry with a second if blob is missing, as its really not expected for it to actually miss, likely we just haven't reached consistency on the fact that it does exist
						await Task.Delay(1000, cancellationToken);
					}

					_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to retriable exception {Exception}.", blob, ns, e);
				}
			}

			if (lastException != null)
			{
				throw lastException;
			}

			return null;
		}

		private async Task<bool> ReplicateBlobHttpAsync(NamespaceId ns, BlobId blob, BucketId? bucketHint, CancellationToken cancellationToken)
		{
			const int RetryAttempts = 5;
			Exception? lastException = null;
			for (int i = 0; i < RetryAttempts; i++)
			{
				lastException = null;
				
				try
				{
					
					using HttpRequestMessage blobRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/blobs/{ns}/{blob}", UriKind.Relative));
					using HttpResponseMessage blobResponse = await _httpClient.SendAsync(blobRequest, HttpCompletionOption.ResponseHeadersRead, cancellationToken);

					if (blobResponse == null)
					{
						throw new Exception("Blob response never set", lastException);
					}

					if (blobResponse.StatusCode == HttpStatusCode.NotFound)
					{
						_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to it not existing.", blob, ns);
						return false;
					}

					if (blobResponse.StatusCode != HttpStatusCode.OK)
					{
						_logger.LogError("Bad http response when replicating {Blob} in {Namespace} . Status code: {StatusCode}", blob, ns, blobResponse.StatusCode);
						return false;
					}

					await using Stream s = await blobResponse.Content.ReadAsStreamAsync(cancellationToken);
					long? contentLength = blobResponse.Content.Headers.ContentLength;

					if (contentLength == null)
					{
						throw new Exception("Expected content-length on blob response");
					}

					using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromStreamAsync(s, contentLength.Value, "blob-replicator", cancellationToken);
					
					await _blobService.PutObjectAsync(ns, payload, blob, bucketHint: bucketHint, cancellationToken: cancellationToken);
					break;
				}
				catch (AggregateException ae)
				{
					ae.Handle(e =>
					{
						if (e is not IOException or HttpIOException)
						{
							_logger.LogError(e, "Failed to replicate {Blob} in {Namespace} due to unknown exception", blob, ns);
							return false;
						}
						
						_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to retriable exception {Exception}", blob, ns, e);
						lastException = e;
						return true;
					});
				}
				catch (HttpRequestException e)
				{
					// rethrow unknown exceptions
					if (e.InnerException is not IOException && e.InnerException is not HttpIOException)
					{
						_logger.LogError(e, "Failed to replicate {Blob} in {Namespace} due to unknown exception", blob, ns);
						throw;
					}

					// is an io exception that we should retry
					lastException = e;

					if (e.StatusCode == HttpStatusCode.NotFound)
					{
						// delay the retry with a second if blob is missing, as its really not expected for it to actually miss, likely we just haven't reached consistency on the fact that it does exist
						await Task.Delay(1000, cancellationToken);
					}

					_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to retriable exception {Exception}.", blob, ns, e);
				}
			}

			if (lastException != null)
			{
				throw lastException;
			}

			return true;
		}

		private async Task AddToBlobReplicationLogAsync(NamespaceId ns, BlobId blob, BucketId? bucketHint)
		{
			await _replicationLog.InsertAddBlobEventAsync(ns, blob, bucketHint: bucketHint);
		}

		public void SetReplicationOffset(long? state)
		{
			if (!state.HasValue)
			{
				return;
			}

			DateTime bucketTime = DateTime.FromFileTimeUtc(state.Value);

			SetRefState(bucketTime.ToReplicationBucket().ToReplicationBucketIdentifier());
			_logger.LogWarning("Replication bucket set to {ReplicationBucket}", state.Value);
		}

		public async Task StopReplicatingAsync()
		{
			if (_disposed)
			{
				return;
			}

			await _replicationTokenSource.CancelAsync();
			await _replicationFinishedEvent.WaitOneAsync();
		}

		public ReplicatorState State => _replicationState;

		public ReplicatorInfo Info { get; private set; }

		public Task DeleteStateAsync()
		{
			_replicationState = new BlobsState();
			return SaveStateAsync(_replicationState);
		}

		public void SetRefState(string lastBucket)
		{
			_replicationState = new BlobsState() { LastBucket = lastBucket };
			_replicationLog.UpdateReplicatorStateAsync(_namespace, _name, _replicationState).Wait();
		}
	}

	public class ToOldTimeBucketException : Exception
	{
		public string TimeBucket { get; }

		public ToOldTimeBucketException(string timeBucket)
		{
			TimeBucket = timeBucket;
		}
	}

	public class BlobsState : ReplicatorState
	{
		public BlobsState()
		{
			ReplicatingGeneration = null;
			ReplicatorOffset = 0;

			LastEvent = null;
			LastBucket = null;
		}
	}
}
