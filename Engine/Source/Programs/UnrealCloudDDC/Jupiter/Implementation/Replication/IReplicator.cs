// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation
{
	public class ReplicatorState
	{
		public long? ReplicatorOffset { get; set; }
		public Guid? ReplicatingGeneration { get; set; }
		public string? LastBucket { get; set; }
		public Guid? LastEvent { get; set; }
	}
	public class ReplicatorInfo
	{
		public ReplicatorInfo(string replicatorName, NamespaceId namespaceToReplicate, ReplicatorState state)
		{
			ReplicatorName = replicatorName;
			NamespaceToReplicate = namespaceToReplicate;
			State = state;

			LastRun = DateTime.MinValue;
			CountOfRunningReplications = 0;
		}

		public string ReplicatorName { get; }
		public NamespaceId NamespaceToReplicate { get; }
		public ReplicatorState State { get; }

		public DateTime LastRun { get; set; }
		public int CountOfRunningReplications { get; set; }
	}

	public interface IReplicator : IDisposable
	{
		/// <summary>
		/// Attempt to run a new replication, if a replication is already in flight for this replicator this will early exist.
		/// </summary>
		/// <returns>True if the replication actually attempted to run</returns>
		Task<bool> TriggerNewReplicationsAsync();

		/// <summary>
		/// Forcefully set the replication offset to a new value, should only be used to recover a replicator stuck in a bad state
		/// </summary>
		void SetReplicationOffset(long? state);

		Task StopReplicatingAsync();

		public ReplicatorState State
		{
			get;
		}

		public ReplicatorInfo Info
		{
			get;
		}

		Task DeleteStateAsync();
	}
}
