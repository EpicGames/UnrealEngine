// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation.Replication
{
	public class MemoryCachedReplicationLog : IReplicationLog
	{
		private readonly IReplicationLog _inner;
		private readonly IOptionsMonitor<MemoryCacheReplicationLogSettings> _options;
		private readonly Tracer _tracer;

		private readonly ConcurrentDictionary<NamespaceId, MemoryCache> _replicationLogCache = new ConcurrentDictionary<NamespaceId, MemoryCache>();
		public MemoryCachedReplicationLog(IReplicationLog inner, IOptionsMonitor<MemoryCacheReplicationLogSettings> options, Tracer tracer)
		{
			_inner = inner;
			_options = options;
			_tracer = tracer;
		}

		private void AddCacheEntry(NamespaceId ns, string timeBucket, List<BlobReplicationLogEvent> logEvents)
		{
			MemoryCache cache = GetCacheForNamespace(ns);

			using ICacheEntry entry = cache.CreateEntry(timeBucket);
			entry.Value = logEvents;
			entry.Size = 60 * logEvents.Count;

			if (_options.CurrentValue.EnableSlidingExpiry)
			{
				entry.SlidingExpiration = TimeSpan.FromMinutes(_options.CurrentValue.SlidingExpirationMinutes);
			}
		}

		private MemoryCache GetCacheForNamespace(NamespaceId ns)
		{
			return _replicationLogCache.GetOrAdd(ns, _ => new MemoryCache(_options.CurrentValue));
		}

		public IAsyncEnumerable<NamespaceId> GetNamespacesAsync()
		{
			return _inner.GetNamespacesAsync();
		}

		public Task<(string, Guid)> InsertAddEventAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId objectBlob, DateTime? timeBucket = null)
		{
			return _inner.InsertAddEventAsync(ns, bucket, key, objectBlob, timeBucket);
		}

		public Task<(string, Guid)> InsertDeleteEventAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime? timeBucket = null)
		{
			return _inner.InsertDeleteEventAsync(ns, bucket, key, timeBucket);
		}

		public IAsyncEnumerable<ReplicationLogEvent> GetAsync(NamespaceId ns, string? lastBucket, Guid? lastEvent)
		{
			// TODO: We could potentially add a cache for this ref replication log if we wanted to, the last event reading does make it a bit tricky as we need to be certain the buckets hasn't mutated when we read it
			return _inner.GetAsync(ns, lastBucket, lastEvent);
		}

		public Task AddSnapshotAsync(SnapshotInfo snapshotHeader)
		{
			return _inner.AddSnapshotAsync(snapshotHeader);
		}

		public Task<SnapshotInfo?> GetLatestSnapshotAsync(NamespaceId ns)
		{
			return _inner.GetLatestSnapshotAsync(ns);
		}

		public IAsyncEnumerable<SnapshotInfo> GetSnapshotsAsync(NamespaceId ns)
		{
			return _inner.GetSnapshotsAsync(ns);
		}

		public Task UpdateReplicatorStateAsync(NamespaceId ns, string replicatorName, ReplicatorState newState)
		{
			return _inner.UpdateReplicatorStateAsync(ns, replicatorName, newState);
		}

		public Task<ReplicatorState?> GetReplicatorStateAsync(NamespaceId ns, string replicatorName)
		{
			return _inner.GetReplicatorStateAsync(ns, replicatorName);
		}

		public Task<(string, Guid)> InsertAddBlobEventAsync(NamespaceId ns, BlobId objectBlob, DateTime? timeBucket = null, BucketId? bucketHint = null)
		{
			return _inner.InsertAddBlobEventAsync(ns, objectBlob, timeBucket, bucketHint);
		}

		public async IAsyncEnumerable<BlobReplicationLogEvent> GetBlobEventsAsync(NamespaceId ns, string replicationBucket)
		{
			using TelemetrySpan scope = _tracer.StartActiveSpan("Log.read")
				.SetAttribute("operation.name", "Log.read")
				.SetAttribute("resource.name", $"{ns}.{replicationBucket}");

			MemoryCache cache = GetCacheForNamespace(ns);

			if (cache.TryGetValue(replicationBucket, out List<BlobReplicationLogEvent>? cachedResult))
			{
				scope.SetAttribute("Found", true);

				foreach (BlobReplicationLogEvent blobReplicationLogEvent in cachedResult!)
				{
					yield return blobReplicationLogEvent;
				}
			}

			scope.SetAttribute("Found", false);
			List<BlobReplicationLogEvent> blobEvents = await _inner.GetBlobEventsAsync(ns, replicationBucket).ToListAsync();

			DateTime bucketTimestamps = replicationBucket.FromReplicationBucketIdentifier();

			// the object can safely be cached if its older then 10 minutes as we bucket it into 5 minute buckets and we need to leave some room for time differences as well as time needed to reach consistency
			bool canCache = bucketTimestamps < DateTime.UtcNow.AddMinutes(-10);
			if (canCache)
			{
				AddCacheEntry(ns, replicationBucket, blobEvents);
			}

			foreach (BlobReplicationLogEvent blobReplicationLogEvent in blobEvents)
			{
				yield return blobReplicationLogEvent;
			}
		}

		public IReplicationLog GetUnderlyingContentIdStore()
		{
			return _inner;
		}
	}

	public class MemoryCacheReplicationLogSettings : MemoryCacheOptions
	{
		public bool Enabled { get; set; } = true;

		public bool EnableSlidingExpiry { get; set; } = true;
		public int SlidingExpirationMinutes { get; set; } = 2880;
	}
}
