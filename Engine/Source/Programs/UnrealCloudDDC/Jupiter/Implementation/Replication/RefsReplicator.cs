// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.Metrics;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Runtime.CompilerServices;
using System.Text;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Jupiter.Common;
using Jupiter.Common.Implementation;
using Jupiter.Controllers;
using Jupiter.Implementation.TransactionLog;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public class RefsReplicator : IReplicator
	{
		private readonly string _name;
		private readonly ILogger _logger;
		private readonly ManualResetEvent _replicationFinishedEvent = new ManualResetEvent(true);
		private readonly ReplicatorSettings _replicatorSettings;
		private readonly IBlobService _blobService;
		private readonly IReplicationLog _replicationLog;
		private readonly IServiceCredentials _serviceCredentials;
		private readonly Tracer _tracer;
		private readonly BufferedPayloadFactory _bufferedPayloadFactory;
		private readonly ReplicationLogFactory _replicationLogFactory;
		private readonly HttpClient _httpClient;
		private readonly NamespaceId _namespace;
		private RefsState _refsState;
		private bool _replicationRunning;
		private CancellationTokenSource _replicationTokenSource = new CancellationTokenSource();
		private bool _disposed = false;

		private static Histogram<long>? s_replicatedCounter;
		private static readonly JsonSerializerOptions DefaultSerializerSettings = ConfigureJsonOptions();
		public RefsReplicator(ReplicatorSettings replicatorSettings, IBlobService blobService, IHttpClientFactory httpClientFactory, IReplicationLog replicationLog, IServiceCredentials serviceCredentials, Tracer tracer, BufferedPayloadFactory bufferedPayloadFactory, ReplicationLogFactory replicationLogFactory, ILogger<RefsReplicator> logger, Meter meter)
		{
			_name = replicatorSettings.ReplicatorName;
			_namespace = new NamespaceId(replicatorSettings.NamespaceToReplicate);
			_replicatorSettings = replicatorSettings;
			_blobService = blobService;
			_replicationLog = replicationLog;
			_serviceCredentials = serviceCredentials;
			_tracer = tracer;
			_bufferedPayloadFactory = bufferedPayloadFactory;
			_replicationLogFactory = replicationLogFactory;
			_logger = logger;

			_httpClient = httpClientFactory.CreateClient();
			_httpClient.BaseAddress = new Uri(replicatorSettings.ConnectionString);

			ReplicatorState? replicatorState = _replicationLog.GetReplicatorStateAsync(_namespace, _name).Result;
			if (replicatorState == null)
			{
				_refsState = new RefsState();
			}
			else
			{
				_refsState = new RefsState()
				{
					LastBucket = replicatorState.LastBucket,
					LastEvent = replicatorState.LastEvent,
				};
			}

			Info = new ReplicatorInfo(replicatorSettings.ReplicatorName, _namespace, _refsState);

			if (s_replicatedCounter == null)
			{
				s_replicatedCounter = meter.CreateHistogram<long>("replication.active");
			}
		}

		private static JsonSerializerOptions ConfigureJsonOptions()
		{
			JsonSerializerOptions options = new JsonSerializerOptions();
			BaseStartup.ConfigureJsonOptions(options);
			return options;
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (_disposed)
			{
				return;
			}

			if (disposing)
			{
				_replicationFinishedEvent.WaitOne();
				_replicationFinishedEvent.Dispose();
				_replicationTokenSource.Dispose();

				_httpClient.Dispose();
			}

			_disposed = true;
		}

		private async Task SaveStateAsync(RefsState newState)
		{
			await _replicationLog.UpdateReplicatorStateAsync(Info.NamespaceToReplicate, _name, new ReplicatorState { LastBucket = newState.LastBucket, LastEvent = newState.LastEvent });
		}

		public async Task<bool> TriggerNewReplicationsAsync()
		{
			if (_replicationRunning)
			{
				_logger.LogDebug("Skipping replication of replicator: {Name} as it was already running.", _name);
				return false;
			}

			// read the state again to allow it to be modified by the admin controller / other instances of jupiter connected to the same filesystem
			ReplicatorState? replicatorState = await _replicationLog.GetReplicatorStateAsync(_namespace, _name);
			if (replicatorState == null)
			{
				_refsState = new RefsState();
			}
			else
			{
				_refsState = new RefsState()
				{
					LastBucket = replicatorState.LastBucket,
					LastEvent = replicatorState.LastEvent,
				};
			}

			_logger.LogDebug("Read Replication state for replicator: {Name}. {LastBucket} {LastEvent}.", _name, _refsState.LastBucket, _refsState.LastEvent);

			LogReplicationHeartbeat(0);

			bool hasRun;
			int countOfReplicationsDone = 0;

			try
			{
				//using IScope scope = Tracer.Instance.StartActive("replicator.run");
				//scope.Span.ResourceName =_name;

				_logger.LogDebug("Replicator: {Name} is starting a run.", _name);

				if (!_replicationTokenSource.TryReset())
				{
					_replicationTokenSource.Dispose();
					_replicationTokenSource = new CancellationTokenSource();
				}
				
				_replicationRunning = true;
				_replicationFinishedEvent.Reset();
				CancellationToken replicationToken = _replicationTokenSource.Token;

				NamespaceId ns = _namespace;

				Info.CountOfRunningReplications = 0;

				string? lastBucket = null;
				Guid? lastEvent = null;

				bool haveRunBefore = _refsState.LastBucket != null && _refsState.LastEvent != null;
				if (!haveRunBefore && !_replicatorSettings.SkipSnapshot)
				{
					// have not run before, replicate a snapshot
					_logger.LogInformation("{Name} Have not run replication before, attempting to use snapshot. State: {@State}", _name, _refsState);
					try
					{
						(string eventBucket, Guid eventId, int countOfEventsReplicated) = await ReplicateFromSnapshotAsync(ns, replicationToken);
						countOfReplicationsDone += countOfEventsReplicated;

						// finished replicating the snapshot, persist the state
						_refsState.LastBucket = eventBucket;
						_refsState.LastEvent = eventId;
						await SaveStateAsync(_refsState);
						hasRun = true;
					}
					catch (NoSnapshotAvailableException)
					{
						// no snapshot available so we attempt a incremental replication with no bucket set
						_refsState.LastBucket = null;
						_refsState.LastEvent = null;
					}
				}

				lastBucket = _refsState.LastBucket;
				lastEvent = _refsState.LastEvent;

				bool retry;
				bool haveAttemptedSnapshot = false;
				do
				{
					retry = false;
					UseSnapshotException? useSnapshotException = null;
					try
					{
						countOfReplicationsDone += await ReplicateIncrementallyAsync(ns, lastBucket, lastEvent, replicationToken);
					}
					catch (AggregateException ae)
					{
						ae.Handle(e =>
						{
							if (e is UseSnapshotException snapshotException)
							{
								useSnapshotException = snapshotException;
								return true;
							}

							return false;
						});
					}
					catch (UseSnapshotException e)
					{
						useSnapshotException = e;
					}

					if (useSnapshotException != null)
					{
						// if we have already attempted to recover using a snapshot and we still fail we just give up as there is no content to fetch
						if (haveAttemptedSnapshot)
						{
							break;
						}

						// if we fail to replicate incrementally we revert to using a snapshot
						haveAttemptedSnapshot = true;

						// if we are told to use a snapshot but are configured to not do so we will simply reset the state tracking and thus start from the oldest incremental event
						// this will result in a partial replication
						if (_replicatorSettings.SkipSnapshot)
						{
							lastBucket = null;
							lastEvent = null;
							retry = true;
							continue;
						}

						(string eventBucket, Guid eventId, int countOfEventsReplicated) = await ReplicateFromSnapshotAsync(ns, replicationToken, useSnapshotException.SnapshotBlob, useSnapshotException.BlobNamespace);
						countOfReplicationsDone += countOfEventsReplicated;

						// resume from these new events instead
						lastBucket = eventBucket;
						lastEvent = eventId;

						_refsState.LastBucket = eventBucket;
						_refsState.LastEvent = eventId;
						await SaveStateAsync(_refsState);
						retry = true;
					}
				} while (retry);
				hasRun = countOfReplicationsDone != 0;
			}
			finally
			{
				_replicationRunning = false;
				_replicationFinishedEvent.Set();
			}

			_logger.LogDebug("Replicator: {Name} finished its replication run. Replications completed: {ReplicationsDone} .", _name, countOfReplicationsDone);

			return hasRun;
		}

		private async Task<(string, Guid, int)> ReplicateFromSnapshotAsync(NamespaceId ns, CancellationToken cancellationToken, BlobId? snapshotBlob = null, NamespaceId? blobNamespace = null)
		{
			// determine latest snapshot if no specific blob was specified
			if (snapshotBlob == null)
			{
				using HttpRequestMessage snapshotRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/replication-log/snapshots/{ns}", UriKind.Relative));
				HttpResponseMessage snapshotResponse = await _httpClient.SendAsync(snapshotRequest, cancellationToken);
				snapshotResponse.EnsureSuccessStatusCode();

				string s = await snapshotResponse.Content.ReadAsStringAsync(cancellationToken);
				ReplicationLogSnapshots? snapshots = JsonSerializer.Deserialize<ReplicationLogSnapshots>(s);
				if (snapshots == null)
				{
					throw new NotImplementedException();
				}

				SnapshotInfo? snapshotInfo = snapshots.Snapshots?.FirstOrDefault();
				if (snapshotInfo == null)
				{
					throw new NoSnapshotAvailableException("No snapshots found");
				}

				snapshotBlob = snapshotInfo.SnapshotBlob;
				blobNamespace = snapshotInfo.BlobNamespace;
			}

			// process snapshot
			// fetch the snapshot from the remote blob store
			ReplicationLogSnapshot snapshot;
			{
				using HttpRequestMessage request = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/blobs/{blobNamespace}/{snapshotBlob}", UriKind.Relative));
				HttpResponseMessage response = await _httpClient.SendAsync(request, HttpCompletionOption.ResponseHeadersRead, cancellationToken);
				response.EnsureSuccessStatusCode();
				await using Stream blobStream = await response.Content.ReadAsStreamAsync(cancellationToken);
				snapshot = _replicationLogFactory.DeserializeSnapshotFromStream(blobStream);
			}

			if (!snapshot.LastEvent.HasValue)
			{
				throw new Exception("No last event found");
			}

			Guid snapshotEvent = snapshot.LastEvent.Value;
			string snapshotBucket = snapshot.LastBucket!;
			int countOfObjectsReplicated = 0;
			int countOfObjectsCurrentlyReplicating = 0;

			// if MaxParallelReplications is set to not limit we use the default behavior of ParallelForEachAsync which is to limit based on CPUs 
			int maxParallelism = _replicatorSettings.MaxParallelReplications != -1 ? _replicatorSettings.MaxParallelReplications : 0;
			await Parallel.ForEachAsync(snapshot.GetLiveObjects(),
				new ParallelOptions { MaxDegreeOfParallelism = maxParallelism, CancellationToken = cancellationToken },
				async (snapshotLiveObject, cancellationToken) =>
				{
					try
					{
						using TelemetrySpan? scope = _tracer.StartActiveSpan("replicator.replicate_op_snapshot")
							.SetAttribute("operation.name", "replicator.replicate_op_snapshot")
							.SetAttribute("resource.name", $"{ns}.{snapshotLiveObject.Blob}");
						Interlocked.Increment(ref countOfObjectsCurrentlyReplicating);
						LogReplicationHeartbeat(countOfObjectsCurrentlyReplicating);

						if (countOfObjectsReplicated % 100 == 0)
						{
							_logger.LogInformation("{Name} Snapshot replication still running, replicated {CountOfObjects} of {TotalCountOfObjects}. Replicating {Namespace}", _name, countOfObjectsReplicated, snapshot.LiveObjectsCount, ns);
						}

						Info.CountOfRunningReplications = countOfObjectsCurrentlyReplicating;
						Info.LastRun = DateTime.Now;

						bool blobWasReplicated = await ReplicateOpAsync(ns, snapshotLiveObject.Bucket, snapshotLiveObject.Key, snapshotLiveObject.Blob, cancellationToken);
						if (blobWasReplicated)
						{
							await AddToReplicationLogAsync(ns, snapshotLiveObject.Bucket, snapshotLiveObject.Key, snapshotLiveObject.Blob);
						}
					}
					finally
					{
						Interlocked.Increment(ref countOfObjectsReplicated);
						Interlocked.Decrement(ref countOfObjectsCurrentlyReplicating);
					}
				});

			snapshot.Dispose();

			// snapshot processed, we proceed to reading the incremental state
			return (snapshotBucket, snapshotEvent, countOfObjectsReplicated);
		}

		private async Task<HttpRequestMessage> BuildHttpRequestAsync(HttpMethod httpMethod, Uri uri)
		{
			string? token = await _serviceCredentials.GetTokenAsync();
			HttpRequestMessage request = new HttpRequestMessage(httpMethod, uri);
			if (!string.IsNullOrEmpty(token))
			{
				request.Headers.Add("Authorization", $"{_serviceCredentials.GetAuthenticationScheme()} {token}");
			}

			return request;
		}

		private async Task<int> ReplicateIncrementallyAsync(NamespaceId ns, string? lastBucket, Guid? lastEvent, CancellationToken replicationToken)
		{
			int countOfReplicationsDone = 0;
			_logger.LogInformation("{Name} Looking for new transaction. Previous state: {@State}", _name, _refsState);

			SortedSet<long> replicationTasks = new();

			// if MaxParallelReplications is set to not limit we use the default behavior of ParallelForEachAsync which is to limit based on CPUs 
			int maxParallelism = _replicatorSettings.MaxParallelReplications != -1 ? _replicatorSettings.MaxParallelReplications : 0;

			_logger.LogInformation("{Name} Starting incremental replication maxParallelism: {MaxParallelism} Last event: {LastEvent} Last Bucket {LastBucket}", _name, maxParallelism, lastEvent, lastBucket);

			using CancellationTokenSource cancellationTokenSource = new CancellationTokenSource();
			using CancellationTokenSource linkedTokenSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationTokenSource.Token, replicationToken);

			if (replicationToken.IsCancellationRequested)
			{
				return countOfReplicationsDone;
			}

			await Parallel.ForEachAsync(GetRefEventsAsync(ns, lastBucket, lastEvent, replicationToken),
				new ParallelOptions { MaxDegreeOfParallelism = maxParallelism, CancellationToken = linkedTokenSource.Token },
				async (ReplicationLogEvent @event, CancellationToken ctx) =>
				{
					using TelemetrySpan? scope = _tracer.StartActiveSpan("replicator.replicate_op_incremental")
						.SetAttribute("operation.name", "replicator.replicate_op_incremental")
						.SetAttribute("resource.name", $"{ns}.{@event.Bucket}.{@event.Key}")
						.SetAttribute("time-bucket", @event.Timestamp.ToString(CultureInfo.InvariantCulture));

					_logger.LogDebug("{Name} New transaction to replicate found. Ref: {Namespace} {Bucket} {Key} in {TimeBucket} ({TimeDate}) with id {EventId}. Count of running replications: {CurrentReplications}", _name, @event.Namespace, @event.Bucket, @event.Key, @event.TimeBucket, @event.Timestamp, @event.EventId, replicationTasks.Count);

					Info.CountOfRunningReplications = replicationTasks.Count;
					LogReplicationHeartbeat(replicationTasks.Count);
					long currentOffset = Interlocked.Increment(ref countOfReplicationsDone);

					try
					{
						string eventBucket = @event.TimeBucket;
						Guid eventId = @event.EventId;

						lock (replicationTasks)
						{
							replicationTasks.Add(currentOffset);
						}

						bool blobWasReplicated = false;
						// we do not need to replicate delete events
						if (@event.Op != ReplicationLogEvent.OpType.Deleted)
						{
							if (@event.Blob == null)
							{
								throw new Exception($"Event: {@event.Bucket} {@event.Key} in namespace {@event.Namespace} was missing a blob, unable to replicate it");
							}

							blobWasReplicated = await ReplicateOpAsync(@event.Namespace, @event.Bucket, @event.Key, @event.Blob, replicationToken);
						}

						if (blobWasReplicated)
						{
							// add events should all have blobs, and blobs are only replicated for adds events so this should always be true
							if (@event.Blob != null)
							{
								await AddToReplicationLogAsync(@event.Namespace, @event.Bucket, @event.Key, @event.Blob);
							}
						}

						bool wasOldest;
						lock (replicationTasks)
						{
							wasOldest = currentOffset <= replicationTasks.First();
						}

						// only update the state when we have replicated everything up that point
						if (wasOldest)
						{
							// we have replicated everything up to a point and can persist this in the state
							_refsState.LastBucket = eventBucket;
							_refsState.LastEvent = eventId;
							await SaveStateAsync(_refsState);

							_logger.LogInformation("{Name} replicated all events up to {Time} . Bucket: {EventBucket} Id: {EventId}", _name, @event.Timestamp, eventBucket, eventId);
						}

						Info.LastRun = DateTime.Now;
						LogReplicationHeartbeat(replicationTasks.Count);
					}
					catch (BlobNotFoundException)
					{
						_logger.LogWarning("{Name} Failed to replicate {@Op} in {Namespace} because blob was not present in remote store. Skipping.", _name, @event, Info.NamespaceToReplicate);
					}
					finally
					{
						lock (replicationTasks)
						{
							replicationTasks.Remove(currentOffset);
						}
					}
				});

			return countOfReplicationsDone;
		}

		private async Task<bool> ReplicateOpAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId objectToReplicate, CancellationToken cancellationToken)
		{
			using TelemetrySpan scope = _tracer.StartActiveSpan("replicator.replicate_op")
				.SetAttribute("operation.name", "replicator.replicate_op")
				.SetAttribute("resource.name", $"{ns}.{objectToReplicate}");

			_logger.LogInformation("Attempting to replicate ref {Key} in {Bucket} under {Namespace}.", key, bucket, ns);

			// We could potentially do this, but that could be dangerous if missing child references
			// check if this blob exists locally before replicating, if it does we assume we have all of its references already
			//if (await _blobService.Exists(ns, blob))
			//    return currentOffset;

			HttpResponseMessage? referencesResponse = null;
			Exception? lastException = null;
			const int RetryAttempts = 3;
			for (int i = 0; i < RetryAttempts; i++)
			{
				using HttpRequestMessage referencesRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/refs/{ns}/{bucket}/{key}/references", UriKind.Relative));

				try
				{
					referencesResponse = await _httpClient.SendAsync(referencesRequest, cancellationToken);
					break;
				}
				catch (HttpRequestException e)
				{
					referencesResponse = null;
					// rethrow unknown exceptions
					if (e.InnerException is not IOException)
					{
						throw;
					}

					lastException = e;
				}
			}

			if (referencesResponse == null)
			{
				throw new Exception("Reference response never set", lastException);
			}

			string body = await referencesResponse.Content.ReadAsStringAsync(cancellationToken);

			if (referencesResponse.StatusCode == HttpStatusCode.BadRequest)
			{
				_logger.LogWarning("Failed to resolve references for object {Blob} in {Namespace}. Skipping replication", objectToReplicate, ns);
				return false;
			}
			if (referencesResponse.StatusCode == HttpStatusCode.NotFound)
			{
				// objects that do not exist can not be replicated so we skip them
				_logger.LogWarning("Failed to resolve references for object {Blob} in {Namespace}. Got not found with message \"{Message}\". Skipping replication.", objectToReplicate, ns, referencesResponse.ReasonPhrase);
				return false;
			}
			referencesResponse.EnsureSuccessStatusCode();

			ResolvedReferencesResult? refs = JsonSerializer.Deserialize<ResolvedReferencesResult>(body, DefaultSerializerSettings);
			if (refs == null)
			{
				throw new Exception($"Unable to resolve references for object {objectToReplicate} in namespace {ns}");
			}

			BlobId[] potentialBlobs = new BlobId[refs.References.Length];
			Array.Copy(refs.References, potentialBlobs, refs.References.Length);

			BlobId[] missingBlobs = await _blobService.FilterOutKnownBlobsAsync(ns, potentialBlobs, cancellationToken);
			Task[] blobReplicationTasks = new Task[missingBlobs.Length];
			for (int i = 0; i < missingBlobs.Length; i++)
			{
				BlobId blobToReplicate = missingBlobs[i];
				blobReplicationTasks[i] = Task.Run(async () =>
				{
					_logger.LogInformation("Attempting to replicate blob {Blob} in {Namespace}.", blobToReplicate, ns);

					HttpResponseMessage? blobResponse = null;
					for (int i = 0; i < RetryAttempts; i++)
					{
						using HttpRequestMessage blobRequest = await BuildHttpRequestAsync(HttpMethod.Get, new Uri($"api/v1/blobs/{ns}/{blobToReplicate}", UriKind.Relative));
						try
						{
							blobResponse = await _httpClient.SendAsync(blobRequest, HttpCompletionOption.ResponseHeadersRead, cancellationToken);
							break;
						}
						catch (HttpRequestException e)
						{
							blobResponse = null;
							// rethrow unknown exceptions
							if (e.InnerException is not IOException)
							{
								throw;
							}

							lastException = e;
						}
					}

					if (blobResponse == null)
					{
						throw new Exception("Blob response never set", lastException);
					}

					if (blobResponse.StatusCode == HttpStatusCode.NotFound)
					{
						_logger.LogWarning("Failed to replicate {Blob} in {Namespace} due to it not existing.", blobToReplicate, ns);
						return;
					}

					if (blobResponse.StatusCode != HttpStatusCode.OK)
					{
						_logger.LogError("Bad http response when replicating {Blob} in {Namespace} . Status code: {StatusCode}", blobToReplicate, ns, blobResponse.StatusCode);
						return;
					}

					await using Stream s = await blobResponse.Content.ReadAsStreamAsync(cancellationToken);
					long? contentLength = blobResponse.Content.Headers.ContentLength;

					if (contentLength == null)
					{
						throw new Exception("Expected content-length on blob response");
					}

					using IBufferedPayload payload = await _bufferedPayloadFactory.CreateFromStreamAsync(s, contentLength.Value, "ref-replicator", cancellationToken);

					await _blobService.PutObjectAsync(ns, payload, blobToReplicate, bucketHint: bucket, cancellationToken: cancellationToken);
				}, cancellationToken);
			}

			await Task.WhenAll(blobReplicationTasks);
			return missingBlobs.Length != 0;
		}

		private async IAsyncEnumerable<ReplicationLogEvent> GetRefEventsAsync(NamespaceId ns, string? lastBucket, Guid? lastEvent, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			bool hasRunOnce = false;
			ReplicationLogEvents logEvents;
			do
			{
				if (cancellationToken.IsCancellationRequested)
				{
					yield break;
				}

				if (hasRunOnce && (lastBucket == null || lastEvent == null))
				{
					throw new Exception($"Failed to find state to resume from after first page of ref events, lastBucket: {lastBucket} lastEvent: {lastEvent}");
				}
				StringBuilder url = new StringBuilder($"/api/v1/replication-log/incremental/{ns}");

				// number of records in a single page (response)
				int pageSize = _replicatorSettings.PageSize;
				url.Append($"?count={pageSize}");

				// its okay for last bucket and last event to be null incase we have never run before, but after the first iteration we need them to keep track of where we were
				if (lastBucket != null)
				{
					url.Append($"&lastBucket={lastBucket}");
				}

				if (lastEvent != null)
				{
					url.Append($"&lastEvent={lastEvent}");
				}

				hasRunOnce = true;

				HttpResponseMessage? response = null;
				Exception? lastException = null;
				const int RetryAttempts = 3;
				for (int i = 0; i < RetryAttempts; i++)
				{
					using HttpRequestMessage request = await BuildHttpRequestAsync(HttpMethod.Get, new Uri(url.ToString(), UriKind.Relative));

					try
					{
						response = await _httpClient.SendAsync(request, cancellationToken);
						break;
					}
					catch (HttpRequestException e)
					{
						response = null;
						// rethrow unknown exceptions
						if (e.InnerException is not IOException)
						{
							throw;
						}

						lastException = e;
					}
				}

				if (response == null)
				{
					throw new Exception("Ref response never set", lastException);
				}

				string body = await response.Content.ReadAsStringAsync(cancellationToken);
				if (response.StatusCode == HttpStatusCode.BadRequest)
				{
					ProblemDetails? problemDetails = JsonSerializer.Deserialize<ProblemDetails>(body, DefaultSerializerSettings);
					if (problemDetails == null)
					{
						throw new Exception($"Unknown bad request body when reading incremental replication log. Body: {body}");
					}

					if (problemDetails.Type == ProblemTypes.UseSnapshot)
					{
						ProblemDetailsWithSnapshots? problemDetailsWithSnapshots = JsonSerializer.Deserialize<ProblemDetailsWithSnapshots>(body, DefaultSerializerSettings);

						if (problemDetailsWithSnapshots == null)
						{
							throw new Exception($"Unable to cast the problem details to a snapshot version. Body: {body}");
						}

						BlobId snapshotBlob = problemDetailsWithSnapshots.SnapshotId;
						NamespaceId? blobNamespace = problemDetailsWithSnapshots.BlobNamespace;
						throw new UseSnapshotException(snapshotBlob, blobNamespace!.Value);
					}

					if (problemDetails.Type == ProblemTypes.NoDataFound)
					{
						yield break;
					}
					throw new Exception($"Unknown bad request response. Body: {body}");
				}

				response.EnsureSuccessStatusCode();
				ReplicationLogEvents? replicationLogEvents = JsonSerializer.Deserialize<ReplicationLogEvents>(body, DefaultSerializerSettings);
				if (replicationLogEvents == null)
				{
					throw new Exception($"Unknown error when deserializing replication log events {ns} {lastBucket} {lastEvent}");
				}

				if (replicationLogEvents.Events == null)
				{
					throw new Exception($"Unknown error when deserializing replication log events {ns} {lastBucket} {lastEvent} as events were empty. Body was: {body}");
				}

				logEvents = replicationLogEvents;
				foreach (ReplicationLogEvent logEvent in logEvents.Events)
				{
					yield return logEvent;

					lastBucket = logEvent.TimeBucket;
					lastEvent = logEvent.EventId;
				}
			} while (logEvents.Events.Any());
		}

		private void LogReplicationHeartbeat(int countOfCurrentReplications)
		{
			s_replicatedCounter?.Record(countOfCurrentReplications, new KeyValuePair<string, object?>("replicator", _name), new KeyValuePair<string, object?>("namespace", _namespace));

			// log message used to verify replicators are actually running
			_logger.LogDebug("{Name} starting replication. Last transaction was {TransactionId} {Generation}. Count Of running replications: {CurrentReplications}", _name, State.ReplicatorOffset.GetValueOrDefault(0L), State.ReplicatingGeneration.GetValueOrDefault(Guid.Empty), countOfCurrentReplications);
		}

		private async Task AddToReplicationLogAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId blob)
		{
			await _replicationLog.InsertAddEventAsync(ns, bucket, key, blob);
		}

		public void SetReplicationOffset(long? state)
		{
			throw new NotImplementedException();
		}

		public async Task StopReplicatingAsync()
		{
			if (_disposed)
			{
				return;
			}

			await _replicationTokenSource.CancelAsync();
			await _replicationFinishedEvent.WaitOneAsync();
		}

		public ReplicatorState State => _refsState;

		public ReplicatorInfo Info { get; private set; }

		public Task DeleteStateAsync()
		{
			_refsState = new RefsState();
			return SaveStateAsync(_refsState);
		}

		public void SetRefState(string? lastBucket, Guid? lastEvent)
		{
			_refsState.LastBucket = lastBucket;
			_refsState.LastEvent = lastEvent;
			SaveStateAsync(_refsState).Wait();
		}
	}

	public class UseSnapshotException : Exception
	{
		public BlobId SnapshotBlob { get; }
		public NamespaceId BlobNamespace { get; }

		public UseSnapshotException(BlobId snapshotBlob, NamespaceId blobNamespace)
		{
			SnapshotBlob = snapshotBlob;
			BlobNamespace = blobNamespace;
		}
	}

	public class NoSnapshotAvailableException : Exception
	{
		public NoSnapshotAvailableException(string message) : base(message)
		{
		}
	}

	public class RefsState : ReplicatorState
	{
		public RefsState()
		{
			ReplicatingGeneration = null;
			ReplicatorOffset = 0;

			LastEvent = null;
			LastBucket = null;
		}
	}

	public class ProblemDetailsWithSnapshots : ProblemDetails
	{
		public BlobId SnapshotId { get; set; } = null!;
		public NamespaceId? BlobNamespace { get; set; } = null;
	}
}
