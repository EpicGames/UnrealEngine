// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class ReplicationState
	{
		public List<IReplicator> Replicators { get; } = new List<IReplicator>();
	}

	// ReSharper disable once ClassNeverInstantiated.Global
	public class ReplicationService : PollingService<ReplicationState>
	{
		private readonly IOptionsMonitor<ReplicationSettings> _settings;
		private readonly ILeaderElection _leaderElection;
		private readonly ILogger _logger;
		private readonly Dictionary<string, PendingReplication> _currentReplications = new Dictionary<string, PendingReplication>();

		protected override bool ShouldStartPolling()
		{
			return _settings.CurrentValue.Enabled;
		}

		public ReplicationService(IOptionsMonitor<ReplicationSettings> settings, IOptionsMonitor<ClusterSettings> clusterSettings, IServiceProvider provider, ILeaderElection leaderElection, ILogger<ReplicationService> logger) : base(serviceName: nameof(ReplicationService), TimeSpan.FromSeconds(settings.CurrentValue.ReplicationPollFrequencySeconds), new ReplicationState(), logger, startAtRandomTime: true)
		{
			_settings = settings;
			_leaderElection = leaderElection;
			_logger = logger;

			_leaderElection.OnLeaderChanged += OnLeaderChanged;

			foreach (ReplicatorSettings replicator in settings.CurrentValue.Replicators)
			{
				try
				{
					State.Replicators.Add(CreateReplicator(replicator, clusterSettings.CurrentValue, provider));
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Failed to create replicator {Name}", replicator.ReplicatorName);
				}
			}
		}

		private void OnLeaderChanged(object? sender, OnLeaderChangedEventArgs e)
		{
			if (e.IsLeader)
			{
				return;
			}

			// if we are no longer the leader cancel any pending replications
			// TODO: Reset replication token if we are the new leader
			/*Task[] stopReplicatingTasks = State.Replicators.Select(replicator => replicator.StopReplicating()).ToArray();
			Task.WaitAll(stopReplicatingTasks);*/

		}

		private static IReplicator CreateReplicator(ReplicatorSettings replicatorSettings, ClusterSettings clusterSettings, IServiceProvider provider)
		{
			switch (replicatorSettings.Version)
			{
				case ReplicatorVersion.Refs:
					return ActivatorUtilities.CreateInstance<RefsReplicator>(provider, replicatorSettings, clusterSettings);
				case ReplicatorVersion.Blobs:
					return ActivatorUtilities.CreateInstance<BlobsReplicator>(provider, replicatorSettings, clusterSettings);
				default:
					throw new NotImplementedException($"Unknown replicator version: {replicatorSettings.Version}");
			}
		}

		public override async Task<bool> OnPollAsync(ReplicationState state, CancellationToken cancellationToken)
		{
			if (!_settings.CurrentValue.Enabled)
			{
				_logger.LogInformation("Skipped running replication as it is disabled");
				return false;
			}

			if (!_leaderElection.IsThisInstanceLeader())
			{
				_logger.LogInformation("Skipped running replicators because this instance was not the leader");
				return false;
			}

			_logger.LogInformation("Polling for new replication to start");

			foreach (IReplicator replicator in state.Replicators)
			{
				if (_currentReplications.TryGetValue(replicator.Info.ReplicatorName, out PendingReplication? pendingReplication))
				{
					Task replicationTask = pendingReplication.Task;
					if (replicationTask.IsCompleted)
					{
						_currentReplications.Remove(replicator.Info.ReplicatorName);

						if (replicationTask.IsFaulted)
						{
							// we log the error but avoid raising it to make sure all replicators actually get to run even if there is a faulting one
							_logger.LogError(replicationTask.Exception, "Unhandled exception in replicator {Name}", replicator.Info.ReplicatorName);
							continue;
						}

						if (replicationTask.IsCanceled)
						{
							// Similarly, we check for cancellations but make sure all replicators get to run
							_logger.LogWarning("Replicator {Name} was canceled", replicator.Info.ReplicatorName);
							continue;
						}

						DateTime time = DateTime.Now;
						_logger.LogInformation("Joining replication task for replicator {Name}", replicator.Info.ReplicatorName);
						await replicationTask;
						_logger.LogInformation("Waited for replication task {Name} for {Duration} seconds", replicator.Info.ReplicatorName, (DateTime.Now - time).TotalSeconds);
					}
					else
					{
						// if the replication is still running let it continue to run
						_logger.LogDebug("Replication of replicator: {Name} is still running", replicator.Info.ReplicatorName);

						// if the replication has been running for more then 15 minutes join the task for a little while to make sure it gets some cpu time
						if (DateTime.Now - pendingReplication.LastForceRun > TimeSpan.FromMinutes(15))
						{
							_logger.LogDebug("Replication of replicator: {Name} has been running for a long time. Joining for a while to make sure it has time.", replicator.Info.ReplicatorName);

							await Task.WhenAny(replicationTask, Task.Delay(TimeSpan.FromSeconds(30), cancellationToken));
							pendingReplication.LastForceRun = DateTime.Now;
						}
						continue;
					}
				}

				// start a new run of the replication
				_logger.LogDebug("Triggering new replication of replicator: {Name}", replicator.Info.ReplicatorName);

				_currentReplications[replicator.Info.ReplicatorName] = new PendingReplication { Task =  replicator.TriggerNewReplicationsAsync(), LastForceRun = DateTime.Now};
			}

			if (state.Replicators.Count == 0)
			{
				_logger.LogInformation("Finished replication poll, but no replicators configured to run.");
			}

			return state.Replicators.Count != 0;

		}

		private class PendingReplication
		{
			public Task<bool> Task { get; init; } = null!;
			public DateTime LastForceRun { get; set; }
		}

		protected override async Task OnStopping(ReplicationState state)
		{
			await Task.WhenAll(state.Replicators.Select(replicator => replicator.StopReplicatingAsync()).ToArray());

			// we should have been stopped first so this should not be needed, but to make sure state is stored we dispose of it again
			foreach (IReplicator replicator in State.Replicators)
			{
				replicator.Dispose();
			}
		}

		public IEnumerable<IReplicator> GetReplicators(NamespaceId ns)
		{
			return State.Replicators
				.Where(pair => ns == pair.Info.NamespaceToReplicate)
				.Select(pair => pair);
		}
	}
}
