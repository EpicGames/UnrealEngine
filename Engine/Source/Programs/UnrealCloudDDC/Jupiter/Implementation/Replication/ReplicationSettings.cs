// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;

namespace Jupiter
{
	// ReSharper disable once ClassNeverInstantiated.Global
	public class ReplicationSettings : IValidatableObject
	{
		/// <summary>
		/// Enable to start a replicating another Jupiter instance into this one
		/// </summary>
		public bool Enabled { get; set; } = false;

		/// <summary>
		/// The frequency at which to poll for new replication events
		/// </summary>
		[Required]
		[Range(15, int.MaxValue)]
		public int ReplicationPollFrequencySeconds { get; set; } = 60;

		[Required]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		// ReSharper disable once CollectionNeverUpdated.Global
		public List<ReplicatorSettings> Replicators { get; set; } = new List<ReplicatorSettings>();

		public IEnumerable<ValidationResult> Validate(ValidationContext validationContext)
		{
			List<ValidationResult> results = new List<ValidationResult>();
			if (!Enabled)
			{
				return results;
			}

			Validator.TryValidateProperty(ReplicationPollFrequencySeconds, new ValidationContext(this, null, null) { MemberName = nameof(ReplicationPollFrequencySeconds) }, results);
			Validator.TryValidateProperty(Replicators, new ValidationContext(this, null, null) { MemberName = nameof(Replicators) }, results);
			return results;
		}
	}

	public class ReplicatorSettings
	{
		/// <summary>
		/// The namespace which this replicator is replicating, will be used both on the source and destination side 
		/// </summary>
		[Required]
		public string NamespaceToReplicate { get; set; } = "";

		/// <summary>
		/// Name of this replicator instance, should be unique within the cluster
		/// </summary>
		[Required]
		[Key]
		public string ReplicatorName { get; set; } = "";

		/// <summary>
		/// The site name of the peer which will be replicated
		/// </summary>
		public string? SourceSite { get; set; } = null;

		/// <summary>
		/// The connection string to the remote jupiter instance which will be replicated
		/// </summary>
		[Url]
		public string ConnectionString { get; set; } = "";
		
		public ReplicatorVersion Version { get; set; } = ReplicatorVersion.Refs;

		/// <summary>
		/// Max number of replications that can run in parallel, set to -1 to for it to be one per CPU (can easily be oversubscribed)
		/// </summary>
		public int MaxParallelReplications { get; set; } = 64;

		/// <summary>
		/// Whether to enable multipart transfers, when using blob stores that support it.
		/// </summary>
		public bool EnableMultipartReplications { get; set; } = false;

		/// <summary>
		/// Do not read in old history using a snapshot, instead just start replicating the new incremental state. Means we will only have partial state but we will also get going quicker on relatively useful blobs.
		/// </summary>
		public bool SkipSnapshot { get; set; } = false;

		/// <summary>
		/// The number of records returned in a single response (page) if available, this needs to be bumped as you increase `MaxParallelReplications`
		/// </summary>
		public int PageSize { get; set; } = 1000;
	}

	public enum ReplicatorVersion
	{
		Refs,
		Blobs
	}

	public class ServiceCredentialSettings
	{
		/// <summary>
		/// The client id to use for communication with other services
		/// </summary>
		public string OAuthClientId { get; set; } = "";

		/// <summary>
		/// The client secret to use for communication with other services
		/// </summary>
		public string OAuthClientSecret { get; set; } = "";

		/// <summary>
		/// The url to login for a token to use to connect to the other services. Set to empty to disable login, assuming a unprotected service.
		/// </summary>
		public Uri? OAuthLoginUrl { get; set; } = null;

		/// <summary>
		/// The scope to request
		/// </summary>
		public string OAuthScope { get; set; } = "cache_access";

		/// <summary>
		/// The authentication scheme to use for this token
		/// </summary>
		public string SchemeName { get; set; } = "Bearer";

		/// <summary>
		/// The access token to use if not using OIDC. Can be resolved using a secret string.
		/// </summary>
		public string? AccessToken { get; set; } = null;
	}
}
