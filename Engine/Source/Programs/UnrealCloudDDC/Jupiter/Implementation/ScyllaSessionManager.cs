// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using Cassandra;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	public interface IScyllaSessionManager
	{
		ISession GetSessionForReplicatedKeyspace();
		ISession GetSessionForLocalKeyspace();

		/// <summary>
		/// True if scylla specific features can be used
		/// </summary>
		bool IsScylla { get; }

		/// <summary>
		/// True when using a base cassandra protocol
		/// </summary>
		bool IsCassandra { get; }

		/// <summary>
		/// True when AWS Keyspace is being used
		/// </summary>
		bool IsAWSKeyspace { get; }
	}

	public class ScyllaSessionManager : IScyllaSessionManager
	{
		private readonly ISession _replicatedSession;
		private readonly ISession _localSession;

		public ScyllaSessionManager(ISession replicatedSession, ISession localSession, bool isScylla, bool isCassandra, bool isKeyspace)
		{
			_replicatedSession = replicatedSession;
			_localSession = localSession;
			IsAWSKeyspace = isKeyspace;
			IsScylla = isScylla;
			IsCassandra = isCassandra;
		}

		public ISession GetSessionForReplicatedKeyspace()
		{
			return _replicatedSession;

		}

		public ISession GetSessionForLocalKeyspace()
		{
			return _localSession;
		}

		public bool IsScylla { get; }
		public bool IsCassandra { get; }
		public bool IsAWSKeyspace { get; }
	}

	public static class ScyllaUtils
	{
		public static IEnumerable<(long, long)> GetTableRanges(uint nodesInCluster, uint coresPerNode, uint smudgeFactor)
		{
			ulong parallelQueries = nodesInCluster * coresPerNode * smudgeFactor;
			ulong countOfRanges = parallelQueries * 100;
			ulong maxSize = ulong.MaxValue;
			ulong rangeSize = maxSize / countOfRanges;

			long start = long.MinValue;
			long end;
			for (ulong i = 0; i < countOfRanges; i++)
			{
				end = start + (long)rangeSize;
				if (start > 0 && end < 0)
				{
					end = long.MaxValue;
				}
				yield return (start, end);

				start = end + 1;
			}
		}
	}

	public static class ScyllaTraceExtensions
	{
		private static readonly Tracer ScyllaTracer = TracerProvider.Default.GetTracer(ScyllaServiceName);

		private const string ScyllaServiceName = "ScyllaDB";

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE0060:Remove unused parameter", Justification = "Useful to create from the tracer in case we want to use that context")]
		public static TelemetrySpan BuildScyllaSpan(this Tracer tracer, string spanName)
		{
			return ScyllaTracer.StartActiveSpan(spanName, parentContext: Tracer.CurrentSpan.Context).SetAttribute("type", "db").SetAttribute("operation.name", spanName).SetAttribute("service.name", ScyllaServiceName);
		}
	}
}
