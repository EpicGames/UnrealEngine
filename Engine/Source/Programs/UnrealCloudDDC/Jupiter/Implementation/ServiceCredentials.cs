// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common.Implementation;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public interface IServiceCredentials
	{
		Task<string?> GetTokenAsync(CancellationToken cancellationToken = default);

		string GetAuthenticationScheme();
	}

	public class ServiceCredentials : IServiceCredentials
	{
		private readonly ClientCredentialOAuthAuthenticator? _authenticator;
		private readonly IOptionsMonitor<ServiceCredentialSettings> _settings;
		private readonly ISecretResolver _secretResolver;

		public ServiceCredentials(IServiceProvider provider, IOptionsMonitor<ServiceCredentialSettings> settings, ISecretResolver secretResolver)
		{
			_settings = settings;
			_secretResolver = secretResolver;
			if (settings.CurrentValue.OAuthLoginUrl != null)
			{
				string? clientId = secretResolver.Resolve(settings.CurrentValue.OAuthClientId);
				if (string.IsNullOrEmpty(clientId))
				{
					throw new ArgumentException("ClientId must be set when using a service credential");
				}

				string? clientSecret = secretResolver.Resolve(settings.CurrentValue.OAuthClientSecret);
				if (string.IsNullOrEmpty(clientSecret))
				{
					throw new ArgumentException("ClientSecret must be set when using a service credential");
				}

				_authenticator = ActivatorUtilities.CreateInstance<ClientCredentialOAuthAuthenticator>(provider, settings.CurrentValue.OAuthLoginUrl, clientId, clientSecret, settings.CurrentValue.OAuthScope);
			}
		}

		public string? GetToken()
		{
			if (_authenticator == null && !string.IsNullOrEmpty(_settings.CurrentValue.AccessToken))
			{
				return _secretResolver.Resolve(_settings.CurrentValue.AccessToken);
			}
			return _authenticator?.AuthenticateAsync().Result;
		}

		public async Task<string?> GetTokenAsync(CancellationToken cancellationToken)
		{
			if (_authenticator == null)
			{
				if (!string.IsNullOrEmpty(_settings.CurrentValue.AccessToken))
				{
					return _secretResolver.Resolve(_settings.CurrentValue.AccessToken);
				}

				return null;
			}
			return await _authenticator.AuthenticateAsync(cancellationToken);
		}

		public string GetAuthenticationScheme()
		{
			return _settings.CurrentValue.SchemeName;
		}
	}
}
