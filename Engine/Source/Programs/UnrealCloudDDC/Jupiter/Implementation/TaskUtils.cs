// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Threading.Tasks;

namespace Jupiter
{
	internal static class TaskUtils
	{
		public static byte[] CombineIntoSingleBuffer(Task<byte[]>[] tasks)
		{
			int totalLength = tasks.Sum(task => task.Result.Length);
			byte[] buffer = new byte[totalLength];
			int index = 0;
			foreach (byte[] partialObject in tasks.Select(t => t.Result))
			{
				Array.Copy(partialObject, 0, buffer, index, partialObject.Length);
				index += partialObject.Length;
			}

			return buffer;
		}
	}
}
