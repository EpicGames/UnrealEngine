// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using Jupiter.Common;

namespace Jupiter.Implementation
{
	public interface IReplicationLog
	{
		IAsyncEnumerable<NamespaceId> GetNamespacesAsync();

		Task<(string, Guid)> InsertAddEventAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId objectBlob, DateTime? timeBucket = null);
		Task<(string, Guid)> InsertDeleteEventAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime? timeBucket = null);
		IAsyncEnumerable<ReplicationLogEvent> GetAsync(NamespaceId ns, string? lastBucket, Guid? lastEvent);

		Task AddSnapshotAsync(SnapshotInfo snapshotHeader);
		Task<SnapshotInfo?> GetLatestSnapshotAsync(NamespaceId ns);
		IAsyncEnumerable<SnapshotInfo> GetSnapshotsAsync(NamespaceId ns);

		Task UpdateReplicatorStateAsync(NamespaceId ns, string replicatorName, ReplicatorState newState);
		Task<ReplicatorState?> GetReplicatorStateAsync(NamespaceId ns, string replicatorName);

		Task<(string, Guid)> InsertAddBlobEventAsync(NamespaceId ns, BlobId objectBlob, DateTime? timeBucket = null, BucketId? bucketHint = null);
		IAsyncEnumerable<BlobReplicationLogEvent> GetBlobEventsAsync(NamespaceId ns, string replicationBucket);
	}

	public class SnapshotInfo
	{
		[JsonConstructor]
		public SnapshotInfo(NamespaceId snapshottedNamespace, NamespaceId blobNamespace, BlobId snapshotBlob, DateTime timestamp)
		{
			SnapshottedNamespace = snapshottedNamespace;
			BlobNamespace = blobNamespace;
			SnapshotBlob = snapshotBlob;
			Timestamp = timestamp;
		}

		public NamespaceId SnapshottedNamespace { get; set; }
		public NamespaceId BlobNamespace { get; set; }
		public BlobId SnapshotBlob { get; set; }
		public DateTime Timestamp { get; set; }
	}

	public class ReplicationLogEvent
	{
		[JsonConstructor]
		public ReplicationLogEvent(NamespaceId @namespace, BucketId bucket, RefId key, BlobId? blob, Guid eventId, string timeBucket, DateTime timestamp, OpType op)
		{
			Namespace = @namespace;
			Bucket = bucket;
			Key = key;
			Blob = blob;
			EventId = eventId;
			TimeBucket = timeBucket;
			Timestamp = timestamp;
			Op = op;
		}

		// these are serialized as ints so make sure to keep the value intact
		public enum OpType
		{
			Added = 0,
			Deleted = 1
		};

		public NamespaceId Namespace { get; }

		public BucketId Bucket { get; }

		public RefId Key { get; }

		public OpType Op { get; }

		public DateTime Timestamp { get; }

		public string TimeBucket { get; }

		public Guid EventId { get; }

		public BlobId? Blob { get; }
	}

	public class BlobReplicationLogEvent
	{
		[JsonConstructor]
		public BlobReplicationLogEvent(NamespaceId @namespace, BlobId blob, Guid eventId, string timeBucket, DateTime timestamp, OpType op, BucketId? bucketHint)
		{
			Namespace = @namespace;
			Blob = blob;
			EventId = eventId;
			TimeBucket = timeBucket;
			Timestamp = timestamp;
			Op = op;
			BucketHint = bucketHint;
		}

		public enum OpType
		{
			Added = 0,
			Deleted = 1
		};

		public NamespaceId Namespace { get; }
		public BlobId Blob { get; }
		public OpType Op { get; }
		public DateTime Timestamp { get; }
		public string TimeBucket { get; }
		public Guid EventId { get; }
		public BucketId? BucketHint { get; }
	}

	public class IncrementalLogNotAvailableException : Exception
	{
	}
}
