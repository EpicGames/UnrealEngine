// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using Cassandra;
using EpicGames.Core;
using Jupiter.Common;

namespace Jupiter.Implementation
{
	internal class MemoryReplicationLog : IReplicationLog
	{
		private readonly ConcurrentDictionary<NamespaceId, SortedList<string, List<(TimeUuid, ReplicationLogEvent)>>> _replicationEvents = new();
		private readonly ConcurrentDictionary<NamespaceId, SortedList<string, List<(TimeUuid, BlobReplicationLogEvent)>>> _blobReplicationEvents = new();
		private readonly ConcurrentDictionary<NamespaceId, List<SnapshotInfo>> _snapshots = new();

		private readonly ConcurrentDictionary<NamespaceId, ConcurrentDictionary<string, ReplicatorState>> _replicatorState = new();

		private readonly ConcurrentDictionary<NamespaceId, bool> _namespaces = new();

		public IAsyncEnumerable<NamespaceId> GetNamespacesAsync()
		{
			return _namespaces.Keys.ToAsyncEnumerable();
		}

		public Task<(string, Guid)> InsertAddEventAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId objectBlob, DateTime? timestamp)
		{
			return DoInsertAsync(ns, bucket, key, objectBlob, ReplicationLogEvent.OpType.Added, timestamp);
		}

		private async Task<(string, Guid)> DoInsertAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId? hash, ReplicationLogEvent.OpType op, DateTime? lastTimestamp)
		{
			_namespaces.TryAdd(ns, true);

			DateTime timestamp = lastTimestamp.GetValueOrDefault(DateTime.Now);

			return await Task.Run(() =>
			{
				DateTime hourlyBucket = timestamp.ToHourlyBucket();
				string bucketId = hourlyBucket.ToReplicationBucketIdentifier();
				TimeUuid eventId = TimeUuid.NewId(timestamp);
				ReplicationLogEvent logEvent = new ReplicationLogEvent(ns, bucket, key, hash, eventId, bucketId, hourlyBucket, op);

				_replicationEvents.AddOrUpdate(ns, _ =>
				{
					SortedList<string, List<(TimeUuid, ReplicationLogEvent)>> l = new() { { bucketId, new() { (eventId, logEvent) } } };
					return l;
				}, (_, buckets) =>
				{
					lock (buckets)
					{
						if (buckets.TryGetValue(bucketId, out List<(TimeUuid, ReplicationLogEvent)>? events))
						{
							lock (events)
							{
								events.Add((eventId, logEvent));
							}
						}
						else
						{
							buckets.Add(bucketId, new() { (eventId, logEvent) });
						}
					}

					return buckets;
				});

				return (bucketId, eventId.ToGuid());
			});
		}

		public Task<(string, Guid)> InsertDeleteEventAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime? timestamp)
		{
			return DoInsertAsync(ns, bucket, key, null, ReplicationLogEvent.OpType.Deleted, timestamp);
		}

		public async IAsyncEnumerable<ReplicationLogEvent> GetAsync(NamespaceId ns, string? lastBucket, Guid? lastEvent)
		{
			await Task.CompletedTask;
			if (!_replicationEvents.TryGetValue(ns, out SortedList<string, List<(TimeUuid, ReplicationLogEvent)>>? buckets))
			{
				throw new NamespaceNotFoundException(ns);
			}

			// if no previous bucket was specified we started from the oldest one
			if (lastBucket == null)
			{
				// if we have no buckets we are done
				if (!buckets.Any())
				{
					yield break;
				}

				lastBucket = buckets.First().Key;
			}

			int currentBucketIndex = buckets.IndexOfKey(lastBucket!);
			if (currentBucketIndex == -1)
			{
				// the bucket to resume from does not exist anymore, we should start from a snapshot
				throw new IncrementalLogNotAvailableException();
			}
			// loop over all the buckets after this including the previous one
			// we will then skip all the events we have previously processed
			for (int i = currentBucketIndex; i < buckets.Keys.Count; i++)
			{
				string bucket = buckets.Keys[i];

				if (!buckets.TryGetValue(bucket, out List<(TimeUuid, ReplicationLogEvent)>? logEvents))
				{
					yield break;
				}
				logEvents.SortBy(tuple => tuple.Item1);
				bool shouldSkipEvents = lastEvent.HasValue;
				bool eventFound = false;
				foreach ((TimeUuid eventId, ReplicationLogEvent value) in logEvents)
				{
					if (shouldSkipEvents)
					{
						if (lastEvent!.Value.Equals(eventId))
						{
							// start reading after this event
							shouldSkipEvents = false;
							eventFound = true;
							continue;
						}
						// continue skipping
						continue;
					}

					eventFound = true;
					yield return value;
				}

				if (!lastEvent.HasValue && !eventFound)
				{
					throw new IncrementalLogNotAvailableException();
				}

				// once we have enumerated the bucket we were working on we should now return all events from the remaining ones
				lastEvent = null;
			}
		}

		public Task AddSnapshotAsync(SnapshotInfo snapshotHeader)
		{
			_snapshots.AddOrUpdate(snapshotHeader.SnapshottedNamespace, _ => new List<SnapshotInfo> { snapshotHeader }, (_, list) =>
			{
				// we want the newest snapshots first
				list.Insert(0, snapshotHeader);
				const int maxCountOfSnapshots = 10;
				if (list.Count > maxCountOfSnapshots)
				{
					// remove the last snapshots as they are the oldest
					list.RemoveRange(maxCountOfSnapshots, list.Count - 10);
				}
				return list;
			});

			return Task.CompletedTask;
		}

		public Task<SnapshotInfo?> GetLatestSnapshotAsync(NamespaceId ns)
		{
			if (!_snapshots.TryGetValue(ns, out List<SnapshotInfo>? snapshots))
			{
				return Task.FromResult<SnapshotInfo?>(null);
			}

			return Task.FromResult<SnapshotInfo?>(snapshots.Last());
		}

		public async IAsyncEnumerable<SnapshotInfo> GetSnapshotsAsync(NamespaceId ns)
		{
			await Task.CompletedTask;

			if (!_snapshots.TryGetValue(ns, out List<SnapshotInfo>? snapshots))
			{
				yield break;
			}

			foreach (SnapshotInfo snapshot in snapshots)
			{
				yield return snapshot;
			}
		}

		public Task UpdateReplicatorStateAsync(NamespaceId ns, string replicatorName,
			ReplicatorState newState)
		{
			_replicatorState.AddOrUpdate(ns,
				_ =>
				new ConcurrentDictionary<string, ReplicatorState>
				{
					[replicatorName] = newState
				},
				(_, states) =>
				{
					states[replicatorName] = newState;
					return states;
				});

			return Task.CompletedTask;
		}

		public Task<ReplicatorState?> GetReplicatorStateAsync(NamespaceId ns, string replicatorName)
		{
			if (_replicatorState.TryGetValue(ns, out ConcurrentDictionary<string, ReplicatorState>? replicationState))
			{
				replicationState.TryGetValue(replicatorName, out ReplicatorState? state);
				return Task.FromResult(state);
			}

			return Task.FromResult<ReplicatorState?>(null);
		}

		public Task<(string, Guid)> InsertAddBlobEventAsync(NamespaceId ns, BlobId objectBlob, DateTime? timestamp = null, BucketId? bucketHint = null)
		{
			_namespaces.TryAdd(ns, true);

			DateTime localTimestamp = timestamp.GetValueOrDefault(DateTime.UtcNow);

			DateTime replicationBucket = localTimestamp.ToReplicationBucket();
			string bucketId = replicationBucket.ToReplicationBucketIdentifier();
			TimeUuid eventId = TimeUuid.NewId(localTimestamp);
			BlobReplicationLogEvent logEvent = new BlobReplicationLogEvent(ns, objectBlob, eventId, bucketId, replicationBucket, BlobReplicationLogEvent.OpType.Added, bucketHint);

			_blobReplicationEvents.AddOrUpdate(ns, _ =>
			{
				SortedList<string, List<(TimeUuid, BlobReplicationLogEvent)>> l = new() { { bucketId, new() { (eventId, logEvent) } } };
				return l;
			}, (_, buckets) =>
			{
				lock (buckets)
				{
					if (buckets.TryGetValue(bucketId, out List<(TimeUuid, BlobReplicationLogEvent)>? events))
					{
						lock (events)
						{
							events.Add((eventId, logEvent));
						}
					}
					else
					{
						buckets.Add(bucketId, new() { (eventId, logEvent) });
					}
				}

				return buckets;
			});

			return Task.FromResult<(string, Guid)>((bucketId, eventId.ToGuid()));
		}

		public async IAsyncEnumerable<BlobReplicationLogEvent> GetBlobEventsAsync(NamespaceId ns, string replicationBucket)
		{
			await Task.CompletedTask;
			if (!_blobReplicationEvents.TryGetValue(ns, out SortedList<string, List<(TimeUuid, BlobReplicationLogEvent)>>? buckets))
			{
				throw new NamespaceNotFoundException(ns);
			}

			if (!buckets.TryGetValue(replicationBucket, out List<(TimeUuid, BlobReplicationLogEvent)>? logEvents))
			{
				yield break;
			}

			logEvents.SortBy(tuple => tuple.Item1);
			foreach ((TimeUuid _, BlobReplicationLogEvent value) in logEvents)
			{
				yield return value;
			}
		}
	}

	public static class DateTimeUtils
	{
		public static DateTime ToHourlyBucket(this DateTime timestamp)
		{
			return new DateTime(timestamp.Year, timestamp.Month, timestamp.Day, timestamp.Hour, 0, 0);
		}

		public static DateTime ToReplicationBucket(this DateTime timestamp)
		{
			// each bucket is 5 minutes big
			int minutesBucketed = (timestamp.Minute / 5) * 5;

			return new DateTime(timestamp.Year, timestamp.Month, timestamp.Day, timestamp.Hour, minutesBucketed, 0);
		}

		public static string ToReplicationBucketIdentifier(this DateTime timestamp)
		{
			return $"rep-{timestamp.ToFileTimeUtc()}";
		}

		public static DateTime FromReplicationBucketIdentifier(this string replicationBucketIdentifier)
		{
			if (!replicationBucketIdentifier.StartsWith("rep-", StringComparison.InvariantCultureIgnoreCase))
			{
				throw new Exception($"Provided string does not look like a replication bucket identifier: '{replicationBucketIdentifier}' .");
			}

			long fileTime = long.Parse(replicationBucketIdentifier.Substring(4));

			return DateTime.FromFileTimeUtc(fileTime);
		}
	}
}
