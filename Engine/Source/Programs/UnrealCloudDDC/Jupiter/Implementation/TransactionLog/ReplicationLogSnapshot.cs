// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.IO.Compression;
using System.Text;
using System.Text.Json;
using System.Threading;
using Jupiter.Common;
using JetBrains.Annotations;
using Jupiter.Common.Implementation;

namespace Jupiter.Implementation.TransactionLog
{
	[UsedImplicitly]
	public class ReplicationLogSnapshotState
	{
		public ReplicationLogSnapshotState(SnapshotHeader header, List<SnapshotLiveObject> liveObjects)
		{
			Header = header;
			LiveObjects = liveObjects;
		}

		public SnapshotHeader Header { get; set; }
		public List<SnapshotLiveObject> LiveObjects { get; }
	}

	public class JsonReplicationLogSnapshot : ReplicationLogSnapshot
	{
		private readonly List<SnapshotLiveObject> _liveObjects;
		JsonReplicationLogSnapshot(ReplicationLogSnapshotState snapshotState) : base(snapshotState.Header.Namespace, snapshotState.Header.LastBucket!, snapshotState.Header.LastEvent, (ulong)snapshotState.LiveObjects.Count)
		{
			_liveObjects = snapshotState.LiveObjects;
		}

		public static ReplicationLogSnapshot FromStream(Stream stream)
		{
			// legacy json snapshot found
			using GZipStream gzipStream = new GZipStream(stream, CompressionMode.Decompress);
			ReplicationLogSnapshotState? snapshotState = JsonSerializer.Deserialize<ReplicationLogSnapshotState>(gzipStream);
			if (snapshotState == null)
			{
				throw new NotImplementedException();
			}
			return new JsonReplicationLogSnapshot(snapshotState);
		}

		public override IEnumerable<SnapshotLiveObject> GetLiveObjects()
		{
			return _liveObjects;
		}
	}

	public class BinaryReplicationLogSnapshot : ReplicationLogSnapshot
	{
		private IBufferedPayload? _bufferedPayload;

		private BinaryReplicationLogSnapshot(NamespaceId ns, string lastBucket, Guid lastEvent, ulong countOfObjects, IBufferedPayload payload) : base(ns, lastBucket, lastEvent, countOfObjects)
		{
			_bufferedPayload = payload;
		}

		private BinaryReplicationLogSnapshot(NamespaceId ns) : base(ns)
		{
			_bufferedPayload = null;
		}

		protected override void Dispose(bool disposing)
		{
			if (disposing)
			{
				if (_bufferedPayload != null)
				{
					_bufferedPayload.Dispose();
					_bufferedPayload = null;
				}
			}

			base.Dispose(disposing);
		}

		public static ReplicationLogSnapshot FromStream(BufferedPayloadFactory payloadFactory, Stream stream)
		{
			IBufferedPayload payload = payloadFactory.CreateFilesystemBufferedPayloadAsync(stream, "snapshot-serialize", CancellationToken.None).Result;

			using Stream payloadStream = payload.GetStream();
			(NamespaceId ns, string lastBucket, Guid lastEvent, ulong countOfObjects) = ReadHeader(payloadStream);

			return new BinaryReplicationLogSnapshot(ns, lastBucket, lastEvent, countOfObjects, payload);
		}

		private static (NamespaceId, string, Guid, ulong) ReadHeader(Stream stream)
		{
			byte[] bytes = new byte[4];
			int bytesRead = stream.Read(bytes, 0, 4);
			if (bytesRead != 4)
			{
				throw new Exception("Partial stream read when deserializing snapshot");
			}

			bool hasMagic = bytes[0] == 'S' && bytes[1] == 'N' && bytes[2] == 'A' && bytes[3] == 'P';
			if (!hasMagic)
			{
				throw new Exception("Did not find magic prefix when deserializing snapshot, incorrect format?");
			}

			using BinaryReader reader = new BinaryReader(stream, Encoding.ASCII, leaveOpen: true);
			string ns = reader.ReadString();
			string lastBucket = reader.ReadString();
			Guid lastEvent = new Guid(reader.ReadBytes(16));
			ulong countOfObjects = reader.ReadUInt64();

			return (new NamespaceId(ns), lastBucket, lastEvent, countOfObjects);
		}

		public override IEnumerable<SnapshotLiveObject> GetLiveObjects()
		{
			if (_bufferedPayload == null)
			{
				yield break;
			}

			using Stream payloadStream = _bufferedPayload.GetStream();

			(NamespaceId ns, string lastBucket, Guid lastEvent, ulong countOfObjects) = ReadHeader(payloadStream);

			using GZipStream gzipStream = new GZipStream(payloadStream, CompressionMode.Decompress);
			using BinaryReader readerGzip = new BinaryReader(gzipStream);
			for (ulong i = 0; i < countOfObjects; i++)
			{
				BucketId bucket = new BucketId(readerGzip.ReadString());
				RefId key = new RefId(StringUtils.FormatAsHexString(readerGzip.ReadBytes(20)));
				BlobId blobIdentifier = new BlobId(readerGzip.ReadBytes(20));

				yield return new SnapshotLiveObject(bucket, key, blobIdentifier);
			}
		}

		public static ReplicationLogSnapshot CreateEmptySnapshot(NamespaceId ns)
		{
			return new BinaryReplicationLogSnapshot(ns);
		}
	}

	public class ReplicationLogFactory
	{
		private readonly BufferedPayloadFactory _payloadFactory;

		public ReplicationLogFactory(BufferedPayloadFactory payloadFactory)
		{
			_payloadFactory = payloadFactory;
		}

		public ReplicationLogSnapshot DeserializeSnapshotFromStream(Stream stream)
		{
			if (!stream.CanSeek)
			{
				throw new Exception("Seekable stream required");
			}

			byte[] bytes = new byte[4];
			int bytesRead = stream.Read(bytes, 0, 4);
			if (bytesRead != 4)
			{
				throw new Exception("Partial stream read when deserializing snapshot");
			}

			bool hasMagic = bytes[0] == 'S' && bytes[1] == 'N' && bytes[2] == 'A' && bytes[3] == 'P';
			stream.Seek(0, SeekOrigin.Begin);

			if (hasMagic)
			{
				return BinaryReplicationLogSnapshot.FromStream(_payloadFactory, stream);
			}

			return JsonReplicationLogSnapshot.FromStream(stream);
		}

		public static ReplicationLogSnapshot CreateEmptySnapshot(NamespaceId ns)
		{
			return BinaryReplicationLogSnapshot.CreateEmptySnapshot(ns);
		}
	}

	public class SnapshotHeader
	{
		public NamespaceId Namespace { get; set; }
		public string LastBucket { get; set; } = null!;
		public Guid LastEvent { get; set; }
	}

	public class SnapshotLiveObject
	{
		public SnapshotLiveObject(BucketId bucket, RefId key, BlobId blob)
		{
			Bucket = bucket;
			Key = key;
			Blob = blob;
		}

		public BucketId Bucket { get; set; }
		public RefId Key { get; set; }
		public BlobId Blob { get; set; }
	}

	public abstract class ReplicationLogSnapshot : IDisposable
	{
		private readonly List<SnapshotLiveObject> _addedObjects = new List<SnapshotLiveObject>();
		private readonly HashSet<(BucketId, RefId)> _removedObjects = new HashSet<(BucketId, RefId)>();

		protected ReplicationLogSnapshot(NamespaceId ns)
		{
			Namespace = ns;
		}

		protected ReplicationLogSnapshot(NamespaceId ns, string lastBucket, Guid lastEvent, ulong liveObjectsCount)
		{
			Namespace = ns;
			LastEvent = lastEvent;
			LastBucket = lastBucket;
			LiveObjectsCount = liveObjectsCount;
		}

		public NamespaceId Namespace { get; init; }
		public Guid? LastEvent { get; private set; }
		public string? LastBucket { get; private set; }
		public ulong LiveObjectsCount { get; set; }

		public abstract IEnumerable<SnapshotLiveObject> GetLiveObjects();

		public void Serialize(Stream stream)
		{
			if (LastBucket == null)
			{
				throw new Exception("No last bucket found when serializing state, did you really have events?");
			}

			if (LastEvent == null)
			{
				throw new Exception("No last event found when serializing state, did you really have events?");
			}

			void WriteHeader(Stream s, NamespaceId ns, string lastBucket, Guid lastEvent, ulong countOfObjects)
			{
				// write magic bytes
				s.Write(Encoding.ASCII.GetBytes("SNAP"));

				using BinaryWriter writer = new BinaryWriter(s, Encoding.ASCII, leaveOpen: true);
				writer.Write(ns.ToString());
				writer.Write(lastBucket);
				writer.Write(lastEvent.ToByteArray());
				writer.Write((ulong)countOfObjects);
			}

			void WriteLiveObject(BinaryWriter writer, SnapshotLiveObject liveObject)
			{
				writer.Write(liveObject.Bucket.ToString());
				// a io hash key is 20 bytes so we just write the binary format of it to keep the file small
				// IoHashKey should likely not use a string as its internal representation
				writer.Write(StringUtils.ToHashFromHexString(liveObject.Key.ToString()), 0, 20);
				writer.Write(liveObject.Blob.HashData, 0, 20);
			}

			// write header without the correct number of objects
			WriteHeader(stream, Namespace, LastBucket, LastEvent.Value, 0);

			ulong countOfLiveObjects = 0;

			{
				using GZipStream gzipStream = new GZipStream(stream, CompressionMode.Compress, leaveOpen: true);
				using BinaryWriter zipWriter = new BinaryWriter(gzipStream, Encoding.ASCII, leaveOpen: true);
				foreach (SnapshotLiveObject liveObject in GetLiveObjects())
				{
					bool isRemoved = _removedObjects.Contains((liveObject.Bucket, liveObject.Key));
					if (isRemoved)
					{
						_removedObjects.Remove((liveObject.Bucket, liveObject.Key));
						// this object was deleted so we skip it
						continue;
					}

					WriteLiveObject(zipWriter, liveObject);
					countOfLiveObjects++;
				}

				foreach (SnapshotLiveObject liveObject in _addedObjects)
				{
					WriteLiveObject(zipWriter, liveObject);
					countOfLiveObjects++;
				}
			}

			// seek back to the start and update the header
			stream.Seek(0, SeekOrigin.Begin);
			WriteHeader(stream, Namespace, LastBucket, LastEvent.Value, countOfLiveObjects);
		}

		public void ProcessEvent(ReplicationLogEvent entry)
		{
			LastBucket = entry.TimeBucket;
			LastEvent = entry.EventId;

			switch (entry.Op)
			{
				case ReplicationLogEvent.OpType.Added:
					ProcessAddEvent(entry.Bucket, entry.Key, entry.Blob!);
					break;
				case ReplicationLogEvent.OpType.Deleted:
					ProcessDeleteEvent(entry.Bucket, entry.Key);
					break;
				default:
					throw new NotImplementedException();
			}
		}

		private void ProcessAddEvent(BucketId bucket, RefId key, BlobId blob)
		{
			_addedObjects.Add(new SnapshotLiveObject(bucket, key, blob));
		}
		private void ProcessDeleteEvent(BucketId bucket, RefId key)
		{
			_removedObjects.Add((bucket, key));
		}

		protected virtual void Dispose(bool disposing)
		{
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}
	}
}
