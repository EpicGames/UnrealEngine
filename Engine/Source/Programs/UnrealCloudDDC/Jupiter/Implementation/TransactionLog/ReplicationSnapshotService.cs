// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using Jupiter.Common;
using Jupiter.Implementation.TransactionLog;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter.Implementation
{
	public class SnapshotState
	{
	}

	// ReSharper disable once ClassNeverInstantiated.Global
	public class ReplicationSnapshotService : PollingService<SnapshotState>
	{
		private readonly IServiceProvider _provider;
		private readonly IOptionsMonitor<SnapshotSettings> _settings;
		private readonly IReplicationLog _replicationLog;
		private readonly ILeaderElection _leaderElection;
		private readonly ILogger _logger;
		private readonly CancellationTokenSource _cancellationTokenSource = new CancellationTokenSource();
		private Task? _snapshotBuildTask = null;

		protected override bool ShouldStartPolling()
		{
			return _settings.CurrentValue.Enabled;
		}

		public ReplicationSnapshotService(IServiceProvider provider, IOptionsMonitor<SnapshotSettings> settings, IReplicationLog replicationLog, ILeaderElection leaderElection, ILogger<ReplicationSnapshotService> logger) : base(serviceName: nameof(ReplicationSnapshotService), TimeSpan.FromMinutes(15), new SnapshotState(), logger)
		{
			_provider = provider;
			_settings = settings;
			_replicationLog = replicationLog;
			_leaderElection = leaderElection;
			_logger = logger;
		}

		public override async Task<bool> OnPollAsync(SnapshotState state, CancellationToken cancellationToken)
		{
			if (!_settings.CurrentValue.Enabled)
			{
				_logger.LogInformation("Skipped running replication snapshot service as it is disabled");
				return false;
			}

			if (!_leaderElection.IsThisInstanceLeader())
			{
				_logger.LogInformation("Skipped running snapshot service because this instance was not the leader");
				return false;
			}

			bool ran = false;
			_snapshotBuildTask = Parallel.ForEachAsync(_replicationLog.GetNamespacesAsync(),
				new ParallelOptions { MaxDegreeOfParallelism = _settings.CurrentValue.MaxCountOfNamespacesToSnapshotInParallel, CancellationToken = _cancellationTokenSource.Token },
				async (ns, ctx) =>
				{
					SnapshotInfo? latestSnapshot = await _replicationLog.GetLatestSnapshotAsync(ns);
					if (latestSnapshot != null)
					{
						DateTime lastSnapshot = latestSnapshot.Timestamp;
						DateTime nextSnapshot = lastSnapshot.AddDays(1);
						if (DateTime.Now < nextSnapshot)
						{
							_logger.LogInformation("Skipped building snapshot for namespace {Namespace} as the previous snapshot ({PreviousSnapshot}) was not a day old.", ns, lastSnapshot);
							return;
						}
					}
					ReplicationLogSnapshotBuilder builder = ActivatorUtilities.CreateInstance<ReplicationLogSnapshotBuilder>(_provider);
					try
					{
						_logger.LogInformation("Building snapshot for {Namespace}", ns);
						BlobId snapshotBlob = await builder.BuildSnapshotAsync(ns, _settings.CurrentValue.SnapshotStorageNamespace, _cancellationTokenSource.Token);
						_logger.LogInformation("Snapshot built for {Namespace} with id {Id}", ns, snapshotBlob);

					}
					catch (IncrementalLogNotAvailableException)
					{
						_logger.LogWarning("Unable to generate a snapshot for {Namespace} as there was no incremental state available", ns);
					}

					ran = true;
				});
			await _snapshotBuildTask;
			_snapshotBuildTask = null;
			return ran;
		}

		protected override async Task OnStopping(SnapshotState state)
		{
			await _cancellationTokenSource.CancelAsync();

			if (_snapshotBuildTask != null)
			{
				await _snapshotBuildTask;
			}
		}
	}

	public class SnapshotSettings
	{
		/// <summary>
		/// Enable to start a replicating another Jupiter instance into this one
		/// </summary>
		public bool Enabled { get; set; } = false;

		public NamespaceId SnapshotStorageNamespace { get; set; } = INamespacePolicyResolver.JupiterInternalNamespace;
		public int MaxCountOfNamespacesToSnapshotInParallel { get; set; } = 1;
	}
}
