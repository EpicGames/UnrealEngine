// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using Cassandra;
using Cassandra.Mapping;
using Jupiter.Common;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace Jupiter.Implementation
{
	class ScyllaReplicationLog : IReplicationLog
	{
		private readonly IOptionsMonitor<ScyllaSettings> _settings;
		private readonly Tracer _tracer;
		private readonly ISession _session;
		private readonly Mapper _mapper;

		public ScyllaReplicationLog(IScyllaSessionManager scyllaSessionManager, IOptionsMonitor<ScyllaSettings> settings, Tracer tracer)
		{
			_settings = settings;
			_tracer = tracer;
			_session = scyllaSessionManager.GetSessionForLocalKeyspace();
			_mapper = new Mapper(_session);

			string suffix = scyllaSessionManager.IsAWSKeyspace ? "WITH CUSTOM_PROPERTIES={'ttl':{'status':'enabled'}}" : "";
			if (!settings.CurrentValue.AvoidSchemaChanges)
			{
				_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS replication_log (
					namespace varchar,
					replication_bucket bigint,
					replication_id timeuuid,
					bucket varchar, 
					key varchar, 
					type int,
					object_identifier blob_identifier,
					PRIMARY KEY ((namespace, replication_bucket), replication_id)
				) {suffix};"
				));

				_session.Execute(new SimpleStatement(@$"CREATE TABLE IF NOT EXISTS blob_replication_log (
					namespace varchar,
					replication_bucket bigint,
					replication_id timeuuid,
					type int,
					blob_id blob,
					bucket_hint varchar,
					PRIMARY KEY ((namespace, replication_bucket), replication_id)
				) {suffix};"
				));

				_session.Execute(new SimpleStatement(@"CREATE TABLE IF NOT EXISTS replication_snapshot (
					namespace varchar,
					id timeuuid,
					blob_snapshot blob_identifier,
					blob_namespace varchar,
					PRIMARY KEY ((namespace), id)
				) WITH CLUSTERING ORDER BY (id DESC);"
				));

				_session.Execute(new SimpleStatement(@"CREATE TABLE IF NOT EXISTS replication_namespace (
					namespace varchar,
					PRIMARY KEY ((namespace))
				);"
				));

				_session.Execute(new SimpleStatement(@"CREATE TABLE IF NOT EXISTS replication_state (
					namespace varchar,
					name varchar,
					last_bucket varchar,
					last_event uuid,
					PRIMARY KEY ((namespace), name)
				);"
				));
			}
		}

		public async IAsyncEnumerable<NamespaceId> GetNamespacesAsync()
		{
			IEnumerable<ScyllaNamespace> namespaces;
			
			try
			{
				namespaces = await _mapper.FetchAsync<ScyllaNamespace>("");
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}

			foreach (ScyllaNamespace scyllaNamespace in namespaces)
			{
				yield return new NamespaceId(scyllaNamespace.Namespace);
			}
		}

		public async Task<(string, Guid)> InsertAddEventAsync(NamespaceId ns, BucketId bucket, RefId key, BlobId objectBlob, DateTime? timestamp)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.insert_add_event");

			Task addNamespaceTask = PotentiallyAddNamespaceAsync(ns);
			DateTime timeBucket = timestamp.GetValueOrDefault(DateTime.UtcNow);

			try
			{
				ScyllaReplicationLogEvent log = new ScyllaReplicationLogEvent(ns.ToString(), bucket.ToString(), key.ToString(), timeBucket, ScyllaReplicationLogEvent.OpType.Added, objectBlob);
				await _mapper.InsertAsync<ScyllaReplicationLogEvent>(log, insertNulls: false, ttl: (int)_settings.CurrentValue.ReplicationLogTimeToLive.TotalSeconds);

				await addNamespaceTask;
				return (log.GetReplicationBucketIdentifier(), log.ReplicationId);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task<(string, Guid)> InsertDeleteEventAsync(NamespaceId ns, BucketId bucket, RefId key, DateTime? timestamp)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.insert_delete_event");

			Task addNamespaceTask = PotentiallyAddNamespaceAsync(ns);
			DateTime timeBucket = timestamp.GetValueOrDefault(DateTime.UtcNow);

			try
			{
				ScyllaReplicationLogEvent log = new ScyllaReplicationLogEvent(ns.ToString(), bucket.ToString(), key.ToString(), timeBucket, ScyllaReplicationLogEvent.OpType.Deleted, null);
				await _mapper.InsertAsync<ScyllaReplicationLogEvent>(log, insertNulls: false, ttl: (int)_settings.CurrentValue.ReplicationLogTimeToLive.TotalSeconds);

				await addNamespaceTask;
				return (log.GetReplicationBucketIdentifier(), log.ReplicationId);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		private async Task PotentiallyAddNamespaceAsync(NamespaceId ns)
		{
			try
			{
				await _mapper.InsertAsync(new ScyllaNamespace(ns.ToString()));
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async IAsyncEnumerable<ReplicationLogEvent> GetAsync(NamespaceId ns, string? lastBucket, Guid? lastEvent)
		{
			using TelemetrySpan getReplicationLogScope = _tracer.BuildScyllaSpan("scylla.get_replication_log");
			if (lastBucket == "now")
			{
				// for debug purposes we allow you to list the latest bucket
				lastBucket = DateTime.UtcNow.ToHourlyBucket().ToReplicationBucketIdentifier();
			}

			IAsyncEnumerable<long> buckets = FindReplicationBucketsAsync(ns, lastBucket);

			// loop thru the buckets starting with the oldest to try and find were lastBucket refers to
			bool bucketFound = false;
			await foreach (long bucketField in buckets)
			{
				DateTime t = DateTime.FromFileTimeUtc(bucketField);
				string bucket = t.ToReplicationBucketIdentifier();
				using TelemetrySpan readReplicationScope = _tracer.BuildScyllaSpan("scylla.read_replication_bucket").SetAttribute("resource.name", bucket);

				if (lastBucket != null && bucket != lastBucket)
				{
					continue;
				}

				// at least one bucket was found
				bucketFound = true;
				// we have found the bucket to resume from, we should no do any more filtering
				lastBucket = null;

				IEnumerable<ScyllaReplicationLogEvent> events;
				try
				{
					events = await _mapper.FetchAsync<ScyllaReplicationLogEvent>("WHERE namespace = ? AND replication_bucket = ?", ns.ToString(), bucketField);
				}
				catch (QueryExecutionException e)
				{
					throw new RemoteServiceException("Error communicating with Scylla", e);
				}

				bool skipEvents = lastEvent.HasValue;
				foreach (ScyllaReplicationLogEvent scyllaReplicationLog in events)
				{
					// check if we are resuming from a previous event, if we are omit every event up to it
					if (lastEvent.HasValue && scyllaReplicationLog.ReplicationId == lastEvent)
					{
						skipEvents = false;
						continue; // start reading as of the next event
					}

					if (skipEvents)
					{
						continue;
					}

					yield return new ReplicationLogEvent(
						new NamespaceId(scyllaReplicationLog.Namespace),
						new BucketId(scyllaReplicationLog.Bucket),
						new RefId(scyllaReplicationLog.Key),
						scyllaReplicationLog.ObjectIdentifier?.AsBlobIdentifier(),
						scyllaReplicationLog.ReplicationId,
						scyllaReplicationLog.GetReplicationBucketIdentifier(),
						scyllaReplicationLog.GetReplicationBucketTimestamp(),
						(ReplicationLogEvent.OpType)scyllaReplicationLog.Type);
				}

				// when continuing to the next bucket we start from the beginning of it
				lastEvent = null;
			}

			if (!bucketFound)
			{
				try
				{
					ScyllaNamespace? namespaces = await _mapper.SingleOrDefaultAsync<ScyllaNamespace>("WHERE namespace = ?", ns.ToString());
					if (namespaces == null)
					{
						throw new NamespaceNotFoundException(ns);
					}

					throw new IncrementalLogNotAvailableException();
				}
				catch (QueryExecutionException e)
				{
					throw new RemoteServiceException("Error communicating with Scylla", e);
				}
			}
		}

		private async IAsyncEnumerable<long> FindReplicationBucketsAsync(NamespaceId ns, string? lastBucket)
		{
			using TelemetrySpan findReplicationBucketScope = _tracer.BuildScyllaSpan("scylla.find_replication_buckets");
			// ignore any bucket that is older then a cutoff, as that can cause us to end up scanning thru a lot of hours that will never exist (incremental logs are deleted after 7 days)
			DateTime oldCutoff = DateTime.UtcNow.AddDays(-14);

			DateTime startBucketTime;

			if (lastBucket != null)
			{
				long bucket = FromReplicationBucketIdentifier(lastBucket);
				startBucketTime = DateTime.FromFileTimeUtc(bucket);
				if (startBucketTime < oldCutoff)
				{
					// attempting to use a old bucket, this will not exist anymore so we reset back to the oldest timestamp we have
					DateTime oldestTimestamp = DateTime.UtcNow.AddSeconds(-1 * _settings.CurrentValue.ReplicationLogTimeToLive.TotalSeconds);
					startBucketTime = oldestTimestamp;
				}
				else
				{
					yield return bucket;
				}
			}
			else
			{
				using TelemetrySpan firstReplicationBucketScope = _tracer.BuildScyllaSpan("scylla.determine_first_replication_bucket");
				// we should have no data older then the ttl to lets just assume that the bucket to start searching from is now - time to live
				DateTime oldestTimestamp = DateTime.UtcNow.AddSeconds(-1 * _settings.CurrentValue.ReplicationLogTimeToLive.TotalSeconds);
				startBucketTime = oldestTimestamp;
			}

			// we returned the start bucket earlier so now we start with the next one
			DateTime bucketTime = startBucketTime.AddHours(1.0).ToHourlyBucket();
			while (bucketTime < DateTime.UtcNow && bucketTime > oldCutoff)
			{
				using TelemetrySpan determineBucketExistsScope = _tracer.BuildScyllaSpan("scylla.determine_replication_bucket_exists");
				// fetch all the buckets that exists and sort them based on time
				ScyllaReplicationLogEvent? e = null;
				try
				{
					e = await _mapper.FirstOrDefaultAsync<ScyllaReplicationLogEvent>("WHERE namespace = ? AND replication_bucket = ?", ns.ToString(), bucketTime.ToFileTimeUtc());
				}
				catch (ReadTimeoutException)
				{
					// if the request times out we scanned the db and didn't find anything, we can just move on to the next bucket
				}

				if (e != null)
				{
					yield return e.ReplicationBucket;
				}
				bucketTime = bucketTime.AddHours(1.0);
			}
		}

		private static long FromReplicationBucketIdentifier(string bucket)
		{
			if (!bucket.StartsWith("rep-", StringComparison.OrdinalIgnoreCase))
			{
				throw new ArgumentException($"Invalid bucket identifier: \"{bucket}\"", nameof(bucket));
			}

			string timestamp = bucket.Substring(bucket.IndexOf("-", StringComparison.OrdinalIgnoreCase) + 1);
			long filetime = long.Parse(timestamp);
			return filetime;
		}

		public async Task AddSnapshotAsync(SnapshotInfo snapshotHeader)
		{
			try
			{
				await _mapper.InsertAsync<ScyllaSnapshot>(new ScyllaSnapshot(snapshotHeader.SnapshottedNamespace.ToString(), snapshotHeader.BlobNamespace.ToString(), TimeUuid.NewId(), snapshotHeader.SnapshotBlob));
				await CleanupSnapshotsAsync(snapshotHeader.SnapshottedNamespace);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task CleanupSnapshotsAsync(NamespaceId ns)
		{
			try
			{
				// determine if we have to many snapshots and remove the oldest ones if we do
				RowSet rowSet = await _session.ExecuteAsync(new SimpleStatement("SELECT Count(*) FROM replication_snapshot WHERE namespace = ?", ns.ToString()));
				Row row = rowSet.First();
				int countOfSnapshots = (int)(long)row[0];
				if (countOfSnapshots > _settings.CurrentValue.MaxSnapshotsPerNamespace)
				{
					int deleteCount = countOfSnapshots - _settings.CurrentValue.MaxSnapshotsPerNamespace;

					List<ScyllaSnapshot> snapshots = (await _mapper.FetchAsync<ScyllaSnapshot>("WHERE namespace = ?", ns.ToString())).ToList();
					for (int i = 0; i < deleteCount; i++)
					{
						// since snapshots are sorted newest first we delete from the end
						ScyllaSnapshot snapshotToDelete = snapshots[^(i + 1)];
						await _mapper.DeleteAsync<ScyllaSnapshot>(snapshotToDelete);
					}
				}
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task<SnapshotInfo?> GetLatestSnapshotAsync(NamespaceId ns)
		{
			SnapshotInfo? s = await GetSnapshotsAsync(ns).FirstOrDefaultAsync();

			return s;
		}

		public async IAsyncEnumerable<SnapshotInfo> GetSnapshotsAsync(NamespaceId ns)
		{
			IEnumerable<ScyllaSnapshot> snapshots;
			try
			{
				snapshots = await _mapper.FetchAsync<ScyllaSnapshot>("WHERE namespace = ?", ns.ToString());
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}

			foreach (ScyllaSnapshot snapshot in snapshots)
			{
				yield return new SnapshotInfo(ns, new NamespaceId(snapshot.BlobNamespace), snapshot.BlobSnapshot.AsBlobIdentifier(), snapshot.Id.GetDate().DateTime);
			}
		}

		public async Task UpdateReplicatorStateAsync(NamespaceId ns, string replicatorName, ReplicatorState newState)
		{
			try
			{
				await _mapper.UpdateAsync<ScyllaReplicationState>(new ScyllaReplicationState(ns, replicatorName, newState.LastBucket, newState.LastEvent));
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task<ReplicatorState?> GetReplicatorStateAsync(NamespaceId ns, string name)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.get_replicator_state");

			try
			{
				ScyllaReplicationState? replicationState = await _mapper.FirstOrDefaultAsync<ScyllaReplicationState>("WHERE namespace = ? AND name = ?", ns.ToString(), name);

				return replicationState?.ToReplicatorState();
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async Task<(string, Guid)> InsertAddBlobEventAsync(NamespaceId ns, BlobId objectBlob, DateTime? timestamp = null, BucketId? bucketHint = null)
		{
			using TelemetrySpan scope = _tracer.BuildScyllaSpan("scylla.insert_blob_add_event");

			Task addNamespaceTask = PotentiallyAddNamespaceAsync(ns);
			DateTime timeBucket = timestamp.GetValueOrDefault(DateTime.UtcNow);

			try
			{
				ScyllaBlobReplicationLogEvent log = new ScyllaBlobReplicationLogEvent(ns.ToString(), objectBlob, timeBucket, ScyllaBlobReplicationLogEvent.OpType.Added, bucketHint);
				await _mapper.InsertAsync<ScyllaBlobReplicationLogEvent>(log, insertNulls: false, ttl: (int)_settings.CurrentValue.ReplicationLogTimeToLive.TotalSeconds);

				await addNamespaceTask;
				return (log.GetReplicationBucketIdentifier(), log.ReplicationId);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
		}

		public async IAsyncEnumerable<BlobReplicationLogEvent> GetBlobEventsAsync(NamespaceId ns, string replicationBucket)
		{
			using TelemetrySpan span = _tracer.BuildScyllaSpan("scylla.get_blob_replication_log").SetAttribute("resource.name", replicationBucket);

			DateTime timestamp = replicationBucket.FromReplicationBucketIdentifier();
			DateTime oldestBucketTimestamp = DateTime.UtcNow.AddSeconds(-1 * _settings.CurrentValue.ReplicationLogTimeToLive.TotalSeconds);
			
			// if the requested bucket is older then anything we would have kept around we flag it as to old 
			if (timestamp < oldestBucketTimestamp)
			{
				throw new IncrementalLogNotAvailableException();
			}

			IEnumerable<ScyllaBlobReplicationLogEvent> events;
			try
			{
				long bucketId = replicationBucket.FromReplicationBucketIdentifier().ToFileTimeUtc();
				events = await _mapper.FetchAsync<ScyllaBlobReplicationLogEvent>("WHERE namespace = ? AND replication_bucket = ?", ns.ToString(), bucketId);
			}
			catch (QueryExecutionException e)
			{
				throw new RemoteServiceException("Error communicating with Scylla", e);
			}
			
			bool eventFound = false;
			foreach (ScyllaBlobReplicationLogEvent scyllaReplicationLog in events)
			{
				eventFound = true;
				yield return new BlobReplicationLogEvent(
					new NamespaceId(scyllaReplicationLog.Namespace),
					new BlobId(scyllaReplicationLog.BlobId),
					scyllaReplicationLog.ReplicationId,
					scyllaReplicationLog.GetReplicationBucketIdentifier(),
					scyllaReplicationLog.GetReplicationBucketTimestamp(),
					(BlobReplicationLogEvent.OpType)scyllaReplicationLog.Type,
					scyllaReplicationLog.BucketHint == null ? null : new BucketId(scyllaReplicationLog.BucketHint));
			}

			if (!eventFound)
			{
				try
				{
					ScyllaNamespace? namespaces = await _mapper.SingleOrDefaultAsync<ScyllaNamespace>("WHERE namespace = ?", ns.ToString());
					if (namespaces == null)
					{
						throw new NamespaceNotFoundException(ns);
					}
				}
				catch (QueryExecutionException e)
				{
					throw new RemoteServiceException("Error communicating with Scylla", e);
				}
			}
		}
	}

	[Cassandra.Mapping.Attributes.Table("replication_log")]
	class ScyllaReplicationLogEvent
	{
		public enum OpType
		{
			Added,
			Deleted
		};

		public ScyllaReplicationLogEvent()
		{
			Namespace = null!;
			Bucket = null!;
			Key = null!;
		}

		public ScyllaReplicationLogEvent(string @namespace, string bucket, string key, DateTime lastTimestamp, OpType opType, BlobId? objectIdentifier)
		{
			Namespace = @namespace;
			Bucket = bucket;
			Key = key;
			ReplicationBucket = lastTimestamp.ToUniversalTime().ToHourlyBucket().ToFileTimeUtc();
			ReplicationId = TimeUuid.NewId(lastTimestamp);
			Type = (int)opType;
			ObjectIdentifier = objectIdentifier != null ? new ScyllaBlobIdentifier(objectIdentifier) : null;
		}

		[Cassandra.Mapping.Attributes.PartitionKey]
		public string Namespace { get; set; }

		// we store bucket as a long (datetime converted to filetime bucketed per hour) to make sure it sorts oldest first
		[Cassandra.Mapping.Attributes.PartitionKey]
		[Cassandra.Mapping.Attributes.Column("replication_bucket")]
		public long ReplicationBucket { get; set; }

		[Cassandra.Mapping.Attributes.Column("replication_id")]
		public TimeUuid ReplicationId { get; set; }

		[Cassandra.Mapping.Attributes.Column("bucket")]
		public string Bucket { get; set; }

		[Cassandra.Mapping.Attributes.Column("key")]
		public string Key { get; set; }

		[Cassandra.Mapping.Attributes.Column("type")]
		public int Type { get; set; }

		[Cassandra.Mapping.Attributes.Column("object_identifier")]
		public ScyllaBlobIdentifier? ObjectIdentifier { get; set; }

		public string GetReplicationBucketIdentifier()
		{
			// the replication bucket identifier is a a string to avoid people assuming its a timestamp, we do not store that in the db though as the string does not sort correctly then
			return DateTime.FromFileTimeUtc(ReplicationBucket).ToReplicationBucketIdentifier();
		}

		public DateTime GetReplicationBucketTimestamp()
		{
			return DateTime.FromFileTimeUtc(ReplicationBucket);
		}
	}

	[Cassandra.Mapping.Attributes.Table("blob_replication_log")]
	class ScyllaBlobReplicationLogEvent
	{
		public enum OpType
		{
			Added,
			Deleted
		};

		public ScyllaBlobReplicationLogEvent()
		{
			Namespace = null!;
			BlobId = null!;
		}

		public ScyllaBlobReplicationLogEvent(string @namespace, BlobId blobId, DateTime lastTimestamp, OpType opType, BucketId? bucketHint)
		{
			Namespace = @namespace;
			BlobId = blobId.HashData;
			BucketHint = bucketHint.ToString();
			ReplicationBucket = lastTimestamp.ToReplicationBucket().ToFileTimeUtc();
			ReplicationId = TimeUuid.NewId(lastTimestamp);
			Type = (int)opType;
		}

		[Cassandra.Mapping.Attributes.PartitionKey]
		public string Namespace { get; set; }

		// we store bucket as a long (datetime converted to filetime bucketed into 5 minute buckets) to make sure it sorts oldest first
		[Cassandra.Mapping.Attributes.PartitionKey]
		[Cassandra.Mapping.Attributes.Column("replication_bucket")]
		public long ReplicationBucket { get; set; }

		[Cassandra.Mapping.Attributes.Column("replication_id")]
		public TimeUuid ReplicationId { get; set; }

		[Cassandra.Mapping.Attributes.Column("blob_id")]
		public byte[] BlobId { get; set; }

		[Cassandra.Mapping.Attributes.Column("type")]
		public int Type { get; set; }

		[Cassandra.Mapping.Attributes.Column("bucket_hint")]
		public string? BucketHint { get; set; }

		public string GetReplicationBucketIdentifier()
		{
			// the replication bucket identifier is a a string to avoid people assuming its a timestamp, we do not store that in the db though as the string does not sort correctly then
			return DateTime.FromFileTimeUtc(ReplicationBucket).ToReplicationBucketIdentifier();
		}

		public DateTime GetReplicationBucketTimestamp()
		{
			return DateTime.FromFileTimeUtc(ReplicationBucket);
		}
	}

	[Cassandra.Mapping.Attributes.Table("replication_snapshot")]
	class ScyllaSnapshot
	{
		public ScyllaSnapshot()
		{
			Namespace = null!;
			BlobSnapshot = null!;
			BlobNamespace = null!;
		}

		public ScyllaSnapshot(string @namespace, string blobNamespace, TimeUuid id, BlobId objectIdentifier)
		{
			Namespace = @namespace;
			BlobNamespace = blobNamespace;
			Id = id;
			BlobSnapshot = new ScyllaBlobIdentifier(objectIdentifier);
		}

		[Cassandra.Mapping.Attributes.PartitionKey]
		public string Namespace { get; set; }

		[Cassandra.Mapping.Attributes.Column("id")]
		[Cassandra.Mapping.Attributes.ClusteringKey]
		public TimeUuid Id { get; set; }

		[Cassandra.Mapping.Attributes.Column("blob_snapshot")]
		public ScyllaBlobIdentifier BlobSnapshot { get; set; }

		[Cassandra.Mapping.Attributes.Column("blob_namespace")]
		public string BlobNamespace { get; set; }
	}

	[Cassandra.Mapping.Attributes.Table("replication_namespace")]
	class ScyllaNamespace
	{
		public ScyllaNamespace()
		{
			Namespace = null!;
		}

		public ScyllaNamespace(string @namespace)
		{
			Namespace = @namespace;
		}

		[Cassandra.Mapping.Attributes.PartitionKey]
		public string Namespace { get; set; }
	}

	[Cassandra.Mapping.Attributes.Table("replication_state")]
	class ScyllaReplicationState
	{
		public ScyllaReplicationState()
		{
			Namespace = null!;
			Name = null!;
			LastBucket = null!;
		}

		public ScyllaReplicationState(NamespaceId ns, string name, string? lastBucket, Guid? lastEvent)
		{
			Namespace = ns.ToString();
			Name = name;
			LastBucket = lastBucket;
			LastEvent = lastEvent;
		}

		[Cassandra.Mapping.Attributes.PartitionKey]
		public string Namespace { get; set; }

		[Cassandra.Mapping.Attributes.ClusteringKey]
		public string Name { get; set; }

		[Cassandra.Mapping.Attributes.Column("last_bucket")]
		public string? LastBucket { get; set; }

		[Cassandra.Mapping.Attributes.Column("last_event")]
		public Guid? LastEvent { get; set; }

		public ReplicatorState ToReplicatorState()
		{
			return new ReplicatorState
			{
				LastBucket = LastBucket,
				LastEvent = LastEvent
			};
		}
	}
}
