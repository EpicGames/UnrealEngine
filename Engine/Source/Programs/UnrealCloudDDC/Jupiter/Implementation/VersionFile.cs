// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using YamlDotNet.Serialization;
using YamlDotNet.Serialization.NamingConventions;

namespace Jupiter.Implementation
{
	public interface IVersionFile
	{
		public string? VersionString { get; }
	}

	public class VersionFile : IVersionFile
	{
		private readonly VersionFileContents _versionFileContents;

		private class VersionFileContents
		{
			public string? Version { get; set; }
		}

		public VersionFile()
		{
			FileInfo assemblyFile = new FileInfo(typeof(VersionFile).Assembly.Location);
			FileInfo fi = new FileInfo(Path.Combine(assemblyFile!.Directory!.FullName, "version.yaml"));
			IDeserializer deserializer = new DeserializerBuilder()
				.WithNamingConvention(CamelCaseNamingConvention.Instance)
				.Build();

			using StreamReader tr = fi.OpenText();
			_versionFileContents = deserializer.Deserialize<VersionFileContents>(tr);
		}

		public string? VersionString => _versionFileContents.Version;
	}
}
