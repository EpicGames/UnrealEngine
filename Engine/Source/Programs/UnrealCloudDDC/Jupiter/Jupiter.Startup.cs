// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Security;
using System.Runtime.InteropServices;
using System.Security.Authentication;
using System.Threading;
using Amazon;
using Amazon.Extensions.NETCore.Setup;
using Amazon.Runtime;
using Amazon.S3;
using Amazon.SecretsManager;
using Cassandra;
using Jupiter.Common.Implementation;
using Jupiter.Controllers;
using Jupiter.DataStore;
using Jupiter.Implementation;
using Jupiter.Implementation.Blob;
using Jupiter.Implementation.LeaderElection;
using Jupiter.Implementation.Objects;
using Jupiter.Implementation.Replication;
using Jupiter.Implementation.TransactionLog;
using Microsoft.AspNetCore.Authorization;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace Jupiter
{
	// ReSharper disable once ClassNeverInstantiated.Global
	public class JupiterStartup : BaseStartup
	{
		public JupiterStartup(IConfiguration configuration) : base(configuration, CreateLogger<JupiterStartup>())
		{

		}

		protected override void OnAddAuthorization(AuthorizationOptions authorizationOptions, List<string> defaultSchemes)
		{

		}

		protected override void OnAddService(IServiceCollection services)
		{
			services.AddHttpClient(Options.DefaultName, (provider, client) =>
			{
				static string GetPlatformName()
				{
					if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
					{
						return "Linux";
					}
					else if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
					{
						return "Windows";
					}
					else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
					{
						return "OSX";
					}
					else
					{
						throw new NotSupportedException("Unknown OS when formatting platform name");
					}
				}
				IVersionFile? versionFile = provider.GetService<IVersionFile>();
				string engineVersion = "5.1"; // TODO: we should read this from the version file
				string? version = versionFile?.VersionString ?? "unknown";
				string platformName = GetPlatformName();
				client.DefaultRequestHeaders.TryAddWithoutValidation("UserAgent", $"UnrealEngine/{engineVersion} ({platformName}) Jupiter/{version}");
			});

			services.AddOptions<UnrealCloudDDCSettings>().Bind(Configuration.GetSection("UnrealCloudDDC")).ValidateDataAnnotations();
			services.AddOptions<MongoSettings>().Bind(Configuration.GetSection("Mongo")).ValidateDataAnnotations();

			services.AddOptions<S3Settings>().Bind(Configuration.GetSection("S3")).ValidateDataAnnotations();
			services.AddOptions<AzureSettings>().Bind(Configuration.GetSection("Azure")).ValidateDataAnnotations();
			services.AddOptions<FilesystemSettings>().Bind(Configuration.GetSection("Filesystem")).ValidateDataAnnotations();
			services.AddOptions<DebugSettings>().Bind(Configuration.GetSection("Debug")).ValidateDataAnnotations();

			services.AddOptions<NginxSettings>().Bind(Configuration.GetSection("Nginx")).ValidateDataAnnotations();

			services.AddOptions<SymbolsSettings>().Bind(Configuration.GetSection("Symbols")).ValidateDataAnnotations();
			services.AddOptions<BuildsSettings>().Bind(Configuration.GetSection("Builds")).ValidateDataAnnotations();

			services.AddOptions<ConsistencyCheckSettings>().Bind(Configuration.GetSection("ConsistencyCheck")).ValidateDataAnnotations();
			services.AddOptions<ReferenceResolverSettings>().Bind(Configuration.GetSection("ReferenceResolver")).ValidateDataAnnotations();

			services.AddOptions<BufferedPayloadOptions>().Bind(Configuration.GetSection("PayloadBuffering")).ValidateDataAnnotations();

			services.AddOptions<UpstreamRelaySettings>().Configure(o => Configuration.GetSection("Upstream").Bind(o)).ValidateDataAnnotations();
			services.AddOptions<ClusterSettings>().Configure(o => Configuration.GetSection("Cluster").Bind(o)).ValidateDataAnnotations();

			services.AddOptions<GCSettings>().Configure(o => Configuration.GetSection("GC").Bind(o)).ValidateDataAnnotations();
			services.AddOptions<ReplicationSettings>().Configure(o => Configuration.GetSection("Replication").Bind(o)).ValidateDataAnnotations();
			services.AddOptions<ServiceCredentialSettings>().Configure(o => Configuration.GetSection("ServiceCredentials").Bind(o)).ValidateDataAnnotations();
			services.AddOptions<SnapshotSettings>().Configure(o => Configuration.GetSection("Snapshot").Bind(o)).ValidateDataAnnotations();
			services.AddOptions<MetricsServiceSettings>().Configure(o => Configuration.GetSection("Metrics").Bind(o)).ValidateDataAnnotations();

			services.AddOptions<ScyllaSettings>().Configure(o => Configuration.GetSection("Scylla").Bind(o)).ValidateDataAnnotations();

			services.AddOptions<KubernetesLeaderElectionSettings>().Configure(o => Configuration.GetSection("Kubernetes").Bind(o)).ValidateDataAnnotations();
			services.AddOptions<StaticPeerServiceDiscoverySettings>().Configure(o => Configuration.GetSection("ServiceDiscovery").Bind(o)).ValidateDataAnnotations();

			services.AddOptions<MemoryCacheReferencesSettings>().Configure(o => Configuration.GetSection("CacheRef").Bind(o)).ValidateDataAnnotations();
			services.AddOptions<MemoryCacheContentIdSettings>().Configure(o => Configuration.GetSection("CacheContentId").Bind(o)).ValidateDataAnnotations();

			services.AddOptions<DataStoreSettings>().Configure(o => Configuration.GetSection("DataStore").Bind(o)).ValidateDataAnnotations();
			services.AddSingleton(typeof(CompressedBufferUtils), CreateCompressedBufferUtils);

			services.AddSingleton<AWSCredentials>(provider =>
			{
				AWSCredentialsSettings awsSettings = provider.GetService<IOptionsMonitor<AWSCredentialsSettings>>()!.CurrentValue;

				return AWSCredentialsHelper.GetCredentials(awsSettings, "Jupiter");
			});
			services.AddSingleton<BufferedPayloadFactory>();
			services.AddSingleton<ReplicationLogFactory>();

			services.AddSingleton<BlobCleanupService>();
			services.AddHostedService<BlobCleanupService>(p => p.GetService<BlobCleanupService>()!);

			// TODO: Expose all compatible object stores as a IDataStoreObjectStore
			services.AddSingleton<IDataStoreObjectStore>(provider => provider.GetRequiredService<AmazonS3Store>());

			services.AddSingleton<IDataStoreTableService, DataStoreTableService>();
			services.AddSingleton<IDataStoreTableLogService, DataStoreTableLogService>();
			services.AddHostedService<DataStoreTableLogService>(p => (p.GetRequiredService<IDataStoreTableLogService>() as DataStoreTableLogService)!);

			services.AddSingleton(serviceType: typeof(IRefService), typeof(ObjectService));
			services.AddSingleton(serviceType: typeof(IReferencesStore), ObjectStoreFactory);
			services.AddSingleton(serviceType: typeof(IReferenceResolver), typeof(ReferenceResolver));
			services.AddSingleton(serviceType: typeof(IContentIdStore), ContentIdStoreFactory);

			services.AddSingleton(serviceType: typeof(IBlobIndex), BlobIndexFactory);
			services.AddSingleton(serviceType: typeof(IAmazonS3), CreateS3);

			services.AddSingleton(serviceType: typeof(IBlockStore), BlockIndexFactory);
			services.AddSingleton(serviceType: typeof(IBuildStore), BuildStoreFactory);

			services.AddSingleton<AmazonS3Store>();
			services.AddSingleton<FileSystemStore>();
			services.AddSingleton<AzureBlobStore>();
			services.AddSingleton<PeerBlobStore>();
			services.AddSingleton<MemoryBlobStore>();
			services.AddSingleton<RelayBlobStore>();

			services.AddSingleton<OrphanBlobCleanupRefs>();

			services.AddSingleton(typeof(IBlobService), typeof(BlobService));
			services.AddSingleton(serviceType: typeof(IScyllaSessionManager), ScyllaFactory);

			services.AddSingleton(serviceType: typeof(IReplicationLog), ReplicationLogFactory);

			services.AddSingleton<LastAccessTrackerReference>();
			services.AddSingleton(serviceType: typeof(ILastAccessCache<LastAccessRecord>), p => p.GetService<LastAccessTrackerReference>()!);
			services.AddSingleton(serviceType: typeof(ILastAccessTracker<LastAccessRecord>), p => p.GetService<LastAccessTrackerReference>()!);

			services.AddSingleton(serviceType: typeof(ILeaderElection), CreateLeaderElection);

			services.AddTransient<IRequestHelper, RequestHelper>();
			services.AddSingleton<BufferedPayloadFactory>();
			services.AddSingleton(Configuration);

			services.AddSingleton<FormatResolver>();
			services.AddSingleton<NginxRedirectHelper>();

			services.AddSingleton(serviceType: typeof(ISecretResolver), typeof(SecretResolver));
			services.AddSingleton(typeof(IAmazonSecretsManager), CreateAWSSecretsManager);

			services.AddSingleton<IServiceCredentials, ServiceCredentials>(p => ActivatorUtilities.CreateInstance<ServiceCredentials>(p));

			services.AddSingleton<ReplicationService>();
			services.AddHostedService<ReplicationService>(p => p.GetService<ReplicationService>()!);

			services.AddSingleton<ReplicationSnapshotService>();
			services.AddHostedService<ReplicationSnapshotService>(p => p.GetService<ReplicationSnapshotService>()!);

			services.AddSingleton<BlobStoreConsistencyCheckService>();
			services.AddHostedService<BlobStoreConsistencyCheckService>(p => p.GetService<BlobStoreConsistencyCheckService>()!);

			services.AddSingleton<BlobIndexConsistencyCheckService>();
			services.AddHostedService<BlobIndexConsistencyCheckService>(p => p.GetService<BlobIndexConsistencyCheckService>()!);

			services.AddSingleton<RefStoreConsistencyCheckService>();
			services.AddHostedService<RefStoreConsistencyCheckService>(p => p.GetService<RefStoreConsistencyCheckService>()!);

			services.AddSingleton<BuildStoreConsistencyCheckService>();
			services.AddHostedService<BuildStoreConsistencyCheckService>(p => p.GetService<BuildStoreConsistencyCheckService>()!);
			
			services.AddSingleton<MetricsService>();
			services.AddHostedService<MetricsService>(p => p.GetService<MetricsService>()!);

			services.AddSingleton(typeof(IPeerStatusService), typeof(PeerStatusService));
			services.AddHostedService<PeerStatusService>(p => (PeerStatusService)p.GetService<IPeerStatusService>()!);

			services.AddTransient(typeof(IRefCleanup), typeof(RefLastAccessCleanup));

			services.AddTransient(typeof(VersionFile), typeof(VersionFile));

			services.AddSingleton<RefCleanupService>();
			services.AddHostedService<RefCleanupService>(p => p.GetService<RefCleanupService>()!);

			services.AddSingleton<TempCleanupService>();
			services.AddHostedService<TempCleanupService>(p => p.GetService<TempCleanupService>()!);

			// This will technically create a new settings object, but as we do not dynamically update it the settings will reflect what we actually want
			UnrealCloudDDCSettings settings = services.BuildServiceProvider().GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue!;

			if (settings.LeaderElectionImplementation == UnrealCloudDDCSettings.LeaderElectionImplementations.Kubernetes)
			{
				// add the kubernetes leader instance under its actual type as well to make it easier to find
				services.AddSingleton<KubernetesLeaderElection>(p => (KubernetesLeaderElection)p.GetService<ILeaderElection>()!);
				services.AddHostedService<KubernetesLeaderElection>(p => p.GetService<KubernetesLeaderElection>()!);
			}

			if (settings.UseLastAccessBackgroundService)
			{
				services.AddSingleton<LastAccessBackgroundService>();
				services.AddHostedService<LastAccessBackgroundService>(p => p.GetService<LastAccessBackgroundService>()!);
			}
			else
			{
				services.AddSingleton<LastAccessServiceReferences>();
				services.AddHostedService<LastAccessServiceReferences>(p => p.GetService<LastAccessServiceReferences>()!);
			}

			services.AddSingleton<BlobStoreConsistencyCheckService>();
			services.AddSingleton(typeof(IPeerServiceDiscovery), ServiceDiscoveryFactory);
		}

		private IBlockStore BlockIndexFactory(IServiceProvider provider)
		{
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue!;
			switch (settings.BuildStoreImplementation)
			{
				case UnrealCloudDDCSettings.BuildStoreImplementations.Memory:
					return ActivatorUtilities.CreateInstance<MemoryBlockStore>(provider);
				case UnrealCloudDDCSettings.BuildStoreImplementations.Scylla:
					return ActivatorUtilities.CreateInstance<ScyllaBlockStore>(provider);
				default:
					throw new NotImplementedException($"Unknown build index implementation: {settings.BuildStoreImplementation}");
			}
		}

		private IBuildStore BuildStoreFactory(IServiceProvider provider)
		{
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue!;
			switch (settings.BuildStoreImplementation)
			{
				case UnrealCloudDDCSettings.BuildStoreImplementations.Memory:
					return ActivatorUtilities.CreateInstance<MemoryBuildStore>(provider);
				case UnrealCloudDDCSettings.BuildStoreImplementations.Scylla:
					return ActivatorUtilities.CreateInstance<ScyllaBuildStore>(provider);
				default:
					throw new NotImplementedException($"Unknown build store implementation: {settings.BuildStoreImplementation}");
			}
		}

		private IBlobIndex BlobIndexFactory(IServiceProvider provider)
		{
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue!;
			switch (settings.BlobIndexImplementation)
			{
				case UnrealCloudDDCSettings.BlobIndexImplementations.Memory:
					return ActivatorUtilities.CreateInstance<MemoryBlobIndex>(provider);
				case UnrealCloudDDCSettings.BlobIndexImplementations.Scylla:
					return ActivatorUtilities.CreateInstance<ScyllaBlobIndex>(provider);
				case UnrealCloudDDCSettings.BlobIndexImplementations.Mongo:
					return ActivatorUtilities.CreateInstance<MongoBlobIndex>(provider);
				case UnrealCloudDDCSettings.BlobIndexImplementations.Cache:
					return ActivatorUtilities.CreateInstance<CachedBlobIndex>(provider);
				case UnrealCloudDDCSettings.BlobIndexImplementations.DataStore:
					return ActivatorUtilities.CreateInstance<DataStoreBlobIndex>(provider);
				default:
					throw new NotImplementedException($"Unknown blob index implementation: {settings.BlobIndexImplementation}");
			}
		}

		private IPeerServiceDiscovery ServiceDiscoveryFactory(IServiceProvider provider)
		{
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue!;
			switch (settings.ServiceDiscoveryImplementation)
			{
				case UnrealCloudDDCSettings.ServiceDiscoveryImplementations.Kubernetes:
					return ActivatorUtilities.CreateInstance<KubernetesPeerServiceDiscovery>(provider);
				case UnrealCloudDDCSettings.ServiceDiscoveryImplementations.Static:
					return ActivatorUtilities.CreateInstance<StaticPeerServiceDiscovery>(provider);
				default:
					throw new NotImplementedException($"Unknown service discovery implementation: {settings.ServiceDiscoveryImplementation}");
			}
		}

		private IAmazonSecretsManager CreateAWSSecretsManager(IServiceProvider provider)
		{
			AWSCredentials awsCredentials = provider.GetService<AWSCredentials>()!;
			AWSOptions awsOptions = Configuration.GetAWSOptions();
			awsOptions.Credentials = awsCredentials;

			IAmazonSecretsManager serviceClient = awsOptions.CreateServiceClient<IAmazonSecretsManager>();
			return serviceClient;
		}

		private CompressedBufferUtils CreateCompressedBufferUtils(IServiceProvider provider)
		{
			return ActivatorUtilities.CreateInstance<CompressedBufferUtils>(provider);
		}

		private object ContentIdStoreFactory(IServiceProvider provider)
		{
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue!;
			IContentIdStore store = settings.ContentIdStoreImplementation switch
			{
				UnrealCloudDDCSettings.ContentIdStoreImplementations.Memory => ActivatorUtilities
					.CreateInstance<MemoryContentIdStore>(provider),
				UnrealCloudDDCSettings.ContentIdStoreImplementations.Scylla => ActivatorUtilities
					.CreateInstance<ScyllaContentIdStore>(provider),
				UnrealCloudDDCSettings.ContentIdStoreImplementations.Mongo => ActivatorUtilities
					.CreateInstance<MongoContentIdStore>(provider),
				UnrealCloudDDCSettings.ContentIdStoreImplementations.Cache => ActivatorUtilities
					.CreateInstance<CacheContentIdStore>(provider),
				UnrealCloudDDCSettings.ContentIdStoreImplementations.DataStore => ActivatorUtilities
					.CreateInstance<DataStoreContentIdStore>(provider),
				_ => throw new NotImplementedException(
					$"Unknown content id store implementation: {settings.ContentIdStoreImplementation}")
			};

			MemoryCacheContentIdSettings memoryCacheSettings = provider.GetService<IOptionsMonitor<MemoryCacheContentIdSettings>>()!.CurrentValue;

			if (memoryCacheSettings.Enabled)
			{
				store = ActivatorUtilities.CreateInstance<MemoryCachedContentIdStore>(provider, store);
			}
			return store;
		}

		private IScyllaSessionManager ScyllaFactory(IServiceProvider provider)
		{
			ScyllaSettings settings = provider.GetService<IOptionsMonitor<ScyllaSettings>>()!.CurrentValue!;
			ISecretResolver secretResolver = provider.GetService<ISecretResolver>()!;

			using Serilog.Extensions.Logging.SerilogLoggerProvider serilogLoggerProvider = new();
			Diagnostics.AddLoggerProvider(serilogLoggerProvider);

			string? connectionString = secretResolver.Resolve(settings.ConnectionString);
			if (string.IsNullOrEmpty(connectionString))
			{
				throw new Exception("Connection string has to be specified");
			}

			CassandraConnectionStringBuilder cassandraConnectionStringBuilder = new CassandraConnectionStringBuilder(connectionString);
			if (string.IsNullOrEmpty(cassandraConnectionStringBuilder.DefaultKeyspace))
			{
				throw new Exception("Default Keyspace has to be specified");
			}

			// Configure the builder with your cluster's contact points
			Builder clusterBuilder = Cluster.Builder()
				.WithConnectionString(connectionString)
				.WithLoadBalancingPolicy(Policies.NewDefaultLoadBalancingPolicy(settings.LocalDatacenterName))
				.WithPoolingOptions(PoolingOptions.Create().SetMaxConnectionsPerHost(HostDistance.Local, settings.MaxConnectionForLocalHost).SetMaxRequestsPerConnection(settings.MaxRequestsPerConnection))
				.WithExecutionProfiles(options =>
					options.WithProfile("default", builder => builder.WithConsistencyLevel(settings.DefaultConsistencyLevel))
						.WithProfile("local_consistent", builder =>
						{
							ConsistencyLevel level;

							// Use the DefaultConsistencyLevel if it provides higher consistency than LocalQuorum
							switch (settings.DefaultConsistencyLevel)
							{
								case ConsistencyLevel.Quorum:
								case ConsistencyLevel.EachQuorum:
								case ConsistencyLevel.All:
									level = settings.DefaultConsistencyLevel;
									break;
								default:
									level = ConsistencyLevel.LocalQuorum;
									break;
							}
							
							builder.WithConsistencyLevel(level);
						}));

			if (settings.ReadTimeout != -1)
			{
				clusterBuilder.SocketOptions.SetReadTimeoutMillis(settings.ReadTimeout);
			}
			if (settings.UseAzureCosmosDB || settings.UseAWSKeyspace)
			{
				CassandraConnectionStringBuilder connectionStringBuilder = new CassandraConnectionStringBuilder(connectionString);
				string[] contactPoints = connectionStringBuilder.ContactPoints;

				// Connect to cassandra cluster using TLSv1.2.
				if (settings.UseSSL)
				{
					SSLOptions sslOptions = new SSLOptions(SslProtocols.None, false,
						(sender, certificate, chain, sslPolicyErrors) =>
						{
							if (sslPolicyErrors == SslPolicyErrors.None)
							{
								return true;
							}

							Logger.LogError("Certificate error: {Errors}", sslPolicyErrors);
							// Do not allow this client to communicate with unauthenticated servers.
							return false;
						});

					// Prepare a map to resolve the host name from the IP address.
					Dictionary<IPAddress, string> hostNameByIp = new Dictionary<IPAddress, string>();
					foreach (string contactPoint in contactPoints)
					{
						IPAddress[] resolvedIps = Dns.GetHostAddresses(contactPoint);
						foreach (IPAddress resolvedIp in resolvedIps)
						{
							hostNameByIp[resolvedIp] = contactPoint;
						}
					}

					sslOptions.SetHostNameResolver((ipAddress) =>
					{
						if (hostNameByIp.TryGetValue(ipAddress, out string? resolvedName))
						{
							return resolvedName;
						}
						IPHostEntry hostEntry = Dns.GetHostEntry(ipAddress.ToString());
						return hostEntry.HostName;
					});

					clusterBuilder = clusterBuilder.WithSSL(sslOptions);
				}
			}
			Cluster cluster = clusterBuilder.Build();

			Dictionary<string, string> replicationStrategy = ReplicationStrategies.CreateSimpleStrategyReplicationProperty(2);
			if (settings.KeyspaceReplicationStrategy != null)
			{
				replicationStrategy = settings.KeyspaceReplicationStrategy;
				Logger.LogInformation("Scylla Replication strategy for replicated keyspace is set to {@ReplicationStrategy}", replicationStrategy);
			}

			ISession replicatedSession;
			const int MaxRetryAttempts = 100;
			int countOfAttempts = 0;
			while (true)
			{
				try
				{
					countOfAttempts += 1;
					replicatedSession = cluster.ConnectAndCreateDefaultKeyspaceIfNotExists(replicationStrategy);
					break;
				}
				catch (NoHostAvailableException)
				{
					Logger.LogWarning("Failed to connect to scylla, waiting a while and will then retry. Attempt {AttemptNum} of {NumAttempts}", countOfAttempts, MaxRetryAttempts);
					// wait for a few seconds before attempt to connect again
					Thread.Sleep(5000);
					if (countOfAttempts >= MaxRetryAttempts)
					{
						Logger.LogError("Failed to connect to Scylla after {NumAttempts} attempts, giving up.", countOfAttempts);
						throw;
					}
				}
			}

			string keyspace = replicatedSession.Keyspace;
			if (!settings.AvoidSchemaChanges)
			{
				replicatedSession.Execute(new SimpleStatement("CREATE TYPE IF NOT EXISTS blob_identifier (hash blob)"));
				replicatedSession.Execute(new SimpleStatement("CREATE TYPE IF NOT EXISTS object_reference (bucket text, key text)"));
			}

			replicatedSession.UserDefinedTypes.Define(UdtMap.For<ScyllaBlobIdentifier>("blob_identifier", keyspace));
			replicatedSession.UserDefinedTypes.Define(UdtMap.For<ScyllaObjectReference>("object_reference", keyspace));

			string localKeyspaceName = $"{keyspace}_local_{settings.LocalKeyspaceSuffix}";

			Dictionary<string, string> replicationStrategyLocal = ReplicationStrategies.CreateSimpleStrategyReplicationProperty(2);
			if (settings.LocalKeyspaceReplicationStrategy != null)
			{
				replicationStrategyLocal = settings.LocalKeyspaceReplicationStrategy;
				Logger.LogInformation("Scylla Replication strategy for local keyspace is set to {@ReplicationStrategy}", replicationStrategyLocal);
			}

			// the local keyspace is never replicated so we do not support controlling how the replication strategy is setup
			replicatedSession.CreateKeyspaceIfNotExists(localKeyspaceName, replicationStrategyLocal);
			ISession localSession = cluster.Connect(localKeyspaceName);

			if (!settings.AvoidSchemaChanges)
			{
				localSession.Execute(new SimpleStatement("CREATE TYPE IF NOT EXISTS blob_identifier (hash blob)"));
			}
			localSession.UserDefinedTypes.Define(UdtMap.For<ScyllaBlobIdentifier>("blob_identifier", localKeyspaceName));

			bool isScylla = !settings.UseAzureCosmosDB && !settings.UseAWSKeyspace;
			return new ScyllaSessionManager(replicatedSession, localSession, isScylla, !isScylla, settings.UseAWSKeyspace);
		}

		private IReferencesStore ObjectStoreFactory(IServiceProvider provider)
		{
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue!;
			IReferencesStore store = settings.ReferencesDbImplementation switch
			{
				UnrealCloudDDCSettings.ReferencesDbImplementations.Memory => ActivatorUtilities.CreateInstance<MemoryReferencesStore>(provider),
				UnrealCloudDDCSettings.ReferencesDbImplementations.Scylla => ActivatorUtilities.CreateInstance<ScyllaReferencesStore>(provider),
				UnrealCloudDDCSettings.ReferencesDbImplementations.Mongo => ActivatorUtilities.CreateInstance<MongoReferencesStore>(provider),
				UnrealCloudDDCSettings.ReferencesDbImplementations.Cache => ActivatorUtilities.CreateInstance<CacheReferencesStore>(provider),
				UnrealCloudDDCSettings.ReferencesDbImplementations.Galileo => ActivatorUtilities.CreateInstance<GalileoReferencesStore>(provider),
				UnrealCloudDDCSettings.ReferencesDbImplementations.DataStore => ActivatorUtilities.CreateInstance<DataStoreReferencesStore>(provider),
				_ => throw new NotImplementedException(
					$"Unknown references db implementation: {settings.ReferencesDbImplementation}")
			};

			MemoryCacheReferencesSettings memoryCacheSettings = provider.GetService<IOptionsMonitor<MemoryCacheReferencesSettings>>()!.CurrentValue;

			// Memory cache can not be enabled when using data store as it in turn has an in memory cache
			bool useMemoryCache = settings.ReferencesDbImplementation != UnrealCloudDDCSettings.ReferencesDbImplementations.DataStore && settings.ReferencesDbImplementation != UnrealCloudDDCSettings.ReferencesDbImplementations.Galileo;
			if (useMemoryCache && memoryCacheSettings.Enabled)
			{
				store = ActivatorUtilities.CreateInstance<MemoryCachedReferencesStore>(provider, store);
			}

			return store;
		}

		private ILeaderElection CreateLeaderElection(IServiceProvider provider)
		{
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue!;
			if (settings.LeaderElectionImplementation == UnrealCloudDDCSettings.LeaderElectionImplementations.Kubernetes)
			{
				return ActivatorUtilities.CreateInstance<KubernetesLeaderElection>(provider);
			}
			else if (settings.LeaderElectionImplementation == UnrealCloudDDCSettings.LeaderElectionImplementations.Static)
			{
				// hard coded leader election that assumes it is always the leader
				return ActivatorUtilities.CreateInstance<LeaderElectionStub>(provider, true);
			}
			else if (settings.LeaderElectionImplementation == UnrealCloudDDCSettings.LeaderElectionImplementations.Disabled)
			{
				// disabled leader election means we are never the leader
				return ActivatorUtilities.CreateInstance<LeaderElectionStub>(provider, false);
			}
			else
			{
				throw new NotImplementedException($"Unknown leader election set {settings.LeaderElectionImplementation}");
			}
		}

		private IAmazonS3 CreateS3(IServiceProvider provider)
		{
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue!;

			if (settings.StorageImplementations == null)
			{
				throw new ArgumentException("No storage implementation set");
			}

			bool isS3InUse = settings.StorageImplementations.Any(x =>
				string.Equals(x, UnrealCloudDDCSettings.StorageBackendImplementations.S3.ToString(), StringComparison.OrdinalIgnoreCase));

			if (isS3InUse)
			{
				S3Settings s3Settings = provider.GetService<IOptionsMonitor<S3Settings>>()!.CurrentValue!;
				AWSCredentials awsCredentials = provider.GetService<AWSCredentials>()!;
				AWSOptions awsOptions = Configuration.GetAWSOptions();
				if (s3Settings.ConnectionString.ToUpper() != "AWS")
				{
					awsOptions.DefaultClientConfig.ServiceURL = s3Settings.ConnectionString;
				}

				awsOptions.Credentials = awsCredentials;

				IAmazonS3 serviceClient = awsOptions.CreateServiceClient<IAmazonS3>();

				if (serviceClient.Config is AmazonS3Config c)
				{
					if (s3Settings.ForceAWSPathStyle)
					{
						c.ForcePathStyle = true;
						// this works around a issue with presigned urls when using minio to emulate S3 due to AWS SDK not defaulting to signature v4 for presigned urls outside of us-east-1
						// see https://github.com/minio/minio/issues/19887
						AWSConfigsS3.UseSignatureVersion4 = true;
					}

					if (s3Settings.UseArnRegion.HasValue)
					{
						c.UseArnRegion = s3Settings.UseArnRegion.Value;
					}
				}
				else
				{
					throw new NotImplementedException();
				}

				return serviceClient;
			}

			return null!;
		}

		private IReplicationLog ReplicationLogFactory(IServiceProvider provider)
		{
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue;
			IReplicationLog replicationLog = settings.ReplicationLogWriterImplementation switch
			{
				UnrealCloudDDCSettings.ReplicationLogWriterImplementations.Scylla =>
					ActivatorUtilities.CreateInstance<ScyllaReplicationLog>(provider),
				UnrealCloudDDCSettings.ReplicationLogWriterImplementations.Memory =>
					ActivatorUtilities.CreateInstance<MemoryReplicationLog>(provider),
				_ => throw new NotImplementedException()
			};

			MemoryCacheReplicationLogSettings memoryCacheSettings = provider.GetService<IOptionsMonitor<MemoryCacheReplicationLogSettings>>()!.CurrentValue;

			if (memoryCacheSettings.Enabled)
			{
				replicationLog = ActivatorUtilities.CreateInstance<MemoryCachedReplicationLog>(provider, replicationLog);
			}

			return replicationLog;
		}

		protected override void OnAddHealthChecks(IServiceCollection services, IHealthChecksBuilder healthChecks)
		{
			ServiceProvider provider = services.BuildServiceProvider();
			UnrealCloudDDCSettings settings = provider.GetService<IOptionsMonitor<UnrealCloudDDCSettings>>()!.CurrentValue;

			foreach (UnrealCloudDDCSettings.StorageBackendImplementations impl in settings.GetStorageImplementations())
			{
				switch (impl)
				{
					case UnrealCloudDDCSettings.StorageBackendImplementations.S3:

						// The S3 health checks are disabled as they do not support dynamic credentials similar to the issue with dynamo
						/*AWSCredentials awsCredentials = provider.GetService<AWSCredentials>();
						S3Settings s3Settings = provider.GetService<IOptionsMonitor<S3Settings>>()!.CurrentValue;
						healthChecks.AddS3(options =>
						{
							options.BucketName = s3Settings.BucketName;
							options.S3Config = provider.GetService<IAmazonS3>().Config as AmazonS3Config;
							options.Credentials = awsCredentials;
						}, tags: new[] {"services"});*/
						break;
					case UnrealCloudDDCSettings.StorageBackendImplementations.Azure:
						// Health checks for Azure are disabled as the connection string will vary based on the namespace used
						/*AzureSettings azureSettings = provider.GetService<IOptionsMonitor<AzureSettings>>()!.CurrentValue;
						healthChecks.AddAzureBlobStorage(AzureBlobStore.GetConnectionString(azureSettings, provider), tags: new[] {"services"});*/

						break;

					case UnrealCloudDDCSettings.StorageBackendImplementations.FileSystem:
						healthChecks.AddDiskStorageHealthCheck(options =>
						{
							FilesystemSettings filesystemSettings = provider.GetService<IOptionsMonitor<FilesystemSettings>>()!.CurrentValue;
							string? driveRoot = Path.GetPathRoot(PathUtil.ResolvePath(filesystemSettings.RootDir));
							if (!string.IsNullOrEmpty(driveRoot))
							{
								options.AddDrive(driveRoot);
							}
						});
						break;
					case UnrealCloudDDCSettings.StorageBackendImplementations.Memory:
					case UnrealCloudDDCSettings.StorageBackendImplementations.Relay:
					case UnrealCloudDDCSettings.StorageBackendImplementations.Peer:
						break;
					default:
						throw new ArgumentOutOfRangeException("Unhandled storage impl " + impl);
				}
			}

			healthChecks.AddCheck<LastAccessServiceCheck>("LastAccessServiceCheck", tags: new[] { "services" });

			healthChecks.AddCheck<ReplicatorServiceCheck>("ReplicatorServiceCheck", tags: new[] { "services" });
			healthChecks.AddCheck<ReplicationSnapshotServiceCheck>("ReplicationSnapshotServiceCheck", tags: new[] { "services" });

			GCSettings gcSettings = provider.GetService<IOptionsMonitor<GCSettings>>()!.CurrentValue;

			if (gcSettings.CleanOldRefRecords)
			{
				healthChecks.AddCheck<RefCleanupServiceCheck>("RefCleanupCheck", tags: new[] { "services" });
			}

			if (gcSettings.BlobCleanupServiceEnabled)
			{
				healthChecks.AddCheck<BlobCleanupServiceCheck>("BlobStoreCheck", tags: new[] { "services" });
			}

			if (settings.LeaderElectionImplementation == UnrealCloudDDCSettings.LeaderElectionImplementations.Kubernetes)
			{
				healthChecks.AddCheck<KubernetesLeaderServiceCheck>("KubernetesLeaderService", tags: new[] { "services" });
			}
		}
	}
}
