<Project Sdk="Microsoft.NET.Sdk.Web">

  <!-- Embed the git hash as a specific assembly metadata for easier reference -->
  <Target Name="WriteGitHash" BeforeTargets="CoreCompile">
    <PropertyGroup>
      <GitAssemblyInfoFile>$(IntermediateOutputPath)GitAssemblyInfo.cs</GitAssemblyInfoFile>
    </PropertyGroup>

    <ItemGroup>
      <Compile Include="$(GitAssemblyInfoFile)" />
    </ItemGroup>

    <ItemGroup>
      <AssemblyAttributes Include="AssemblyMetadata">
        <_Parameter1>GitHash</_Parameter1>
        <!-- Make sure to enable the Github source link to make msbuild automatically populate this -->
        <_Parameter2>$(SourceRevisionId)</_Parameter2>
      </AssemblyAttributes>

      <AssemblyAttributes Include="AssemblyMetadata">
        <_Parameter1>PerforceChangelist</_Parameter1>
        <_Parameter2>$(UE_P4_CHANGELIST)</_Parameter2>
      </AssemblyAttributes>

    </ItemGroup>
    <WriteCodeFragment Language="C#" OutputFile="$(GitAssemblyInfoFile)" AssemblyAttributes="@(AssemblyAttributes)" />
  </Target>

  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <Nullable>enable</Nullable>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <WarningsAsErrors />
    <LangVersion>latest</LangVersion>
	<Configurations>Debug;Release;Analyze</Configurations>
    <DocumentationFile>Jupiter.xml</DocumentationFile>
    <NoWarn>$(NoWarn);1701;1702;1591</NoWarn>
    <WarningsNotAsErrors>NU1902;NU1903</WarningsNotAsErrors>

    <!--- SourceLink -->
    <PublishRepositoryUrl>false</PublishRepositoryUrl>
    <EmbedUntrackedSources>false</EmbedUntrackedSources>
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)|$(Platform)'=='Analyze|AnyCPU'">
    <RunAnalyzersDuringBuild>True</RunAnalyzersDuringBuild>
    <TreatWarningsAsErrors>False</TreatWarningsAsErrors>
    <EnforceCodeStyleInBuild>True</EnforceCodeStyleInBuild>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
  </PropertyGroup>

  <ItemGroup>
    <Compile Remove="Tests\**" />
    <Content Remove="Tests\**" />
    <EmbeddedResource Remove="Tests\**" />
    <None Remove="Tests\**" />
  </ItemGroup>

  <ItemGroup>
    <None Remove="appsettings.Cache.yaml" />
  </ItemGroup>

  <ItemGroup>
    <Content Include="appsettings.Cache.yaml">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </Content>
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="AspNetCore.HealthChecks.Aws.S3" Version="8.0.0" />
    <PackageReference Include="AspNetCore.HealthChecks.MongoDb" Version="8.0.0" />
    <PackageReference Include="AspNetCore.HealthChecks.System" Version="8.0.0" />
    <PackageReference Include="AWSSDK.Extensions.NETCore.Setup" Version="3.7.300" />
    <PackageReference Include="AWSSDK.S3" Version="3.7.305.8" />
    <PackageReference Include="Azure.Identity" Version="1.11.4" />
    <PackageReference Include="Azure.Security.KeyVault.Secrets" Version="4.5.0" />
    <PackageReference Include="Azure.Storage.Blobs" Version="12.15.1" />
    <PackageReference Include="KubernetesClient" Version="17.0.14" />
    <PackageReference Include="MongoDB.Driver" Version="2.23.1" />
    <PackageReference Include="Newtonsoft.Json" Version="13.0.4" />
    <PackageReference Include="Serilog.Enrichers.ClientInfo" Version="2.1.1" />
    <PackageReference Include="SerilogAnalyzer" Version="0.15.0" />
    <PackageReference Include="Crc32.NET" Version="1.2.0" />
    <PackageReference Include="K4os.Compression.LZ4" Version="1.3.6" />

    <PackageReference Include="CassandraCSharpDriver" Version="3.19.5" />

    <PackageReference Include="System.Linq.Async" Version="7.0.0" />

    <!-- Embeds git information into the build process 
        Enable this if you are on git to embedd the git hash into the build 
        Produces warnings while source is in perforce -->
    <!--<PackageReference Include="Microsoft.SourceLink.GitHub" Version="1.1.1">
      <PrivateAssets>all</PrivateAssets>
    </PackageReference>-->
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.AspNet\EpicGames.AspNet.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Oodle\EpicGames.Oodle.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Serialization\EpicGames.Serialization.csproj" />
    <ProjectReference Include="..\Jupiter.Common\Jupiter.Common.csproj" />
  </ItemGroup>

  <ItemGroup>
    <None Include="..\version.yaml">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </None>
  </ItemGroup>

</Project>
