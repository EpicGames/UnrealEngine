<wpf:ResourceDictionary xml:space="preserve" xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml" xmlns:s="clr-namespace:System;assembly=mscorlib" xmlns:ss="urn:shemas-jetbrains-com:settings-storage-xaml" xmlns:wpf="http://schemas.microsoft.com/winfx/2006/xaml/presentation">
	<s:Boolean x:Key="/Default/CodeInspection/NamespaceProvider/NamespaceFoldersToSkip/=implementation/@EntryIndexedValue">True</s:Boolean></wpf:ResourceDictionary>