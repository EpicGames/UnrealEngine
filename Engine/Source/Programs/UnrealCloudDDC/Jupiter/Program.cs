// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Hosting;

namespace Jupiter
{
	// ReSharper disable once ClassNeverInstantiated.Global
	public static class Program
	{
		public static int Main(string[] args)
		{
			return BaseProgram<JupiterStartup>.BaseMain(args);
		}

		public static IHostBuilder CreateHostBuilder(string[] Args)
		{
			return BaseProgram<JupiterStartup>.CreateHostBuilder(Args);
		}
	}
}
