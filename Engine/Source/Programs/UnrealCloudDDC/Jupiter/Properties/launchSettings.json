{
  "profiles": {
    "Project": {
      "commandName": "Project",
      "environmentVariables": {
        "ASPNETCORE_ENVIRONMENT": "Development",
        "ASPNETCORE_PORTS": "\"\""
      }
    }
  }
}