// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel.DataAnnotations;
using System.Linq;
using Jupiter.Common;
using Jupiter.Controllers;
using Microsoft.Extensions.Caching.Memory;

namespace Jupiter
{

	public class UnrealCloudDDCSettings
	{
		public enum ReplicationLogWriterImplementations
		{
			Memory,
			Scylla,
			Mongo
		}

		public enum StorageBackendImplementations
		{
			S3,
			Azure,
			FileSystem,
			Memory,
			Relay,
			Peer
		}

		public enum ReferencesDbImplementations
		{
			Memory,
			Scylla,
			Mongo,
			Cache,
			Galileo,
			DataStore
		}

		public enum ContentIdStoreImplementations
		{
			Memory,
			Scylla,
			Mongo,
			Cache,
			DataStore
		}

		public enum BlobIndexImplementations
		{
			Memory,
			Scylla,
			Mongo,
			Cache,
			DataStore
		}

		public enum BuildStoreImplementations
		{
			Memory,
			Scylla,
		}

		public enum LeaderElectionImplementations
		{
			Static,
			Kubernetes,
			Disabled
		}

		public enum ServiceDiscoveryImplementations
		{
			Static,
			Kubernetes
		}

		private sealed class ValidStorageBackend : ValidationAttribute
		{
			public override string FormatErrorMessage(string name)
			{
				return "Need to specify at least one storage backend. Valid ones are: " + string.Join(", ", Enum.GetNames(typeof(StorageBackendImplementations)));
			}

			public override bool IsValid(object? value)
			{
				if (value == null)
				{
					return true;
				}

				return value is IEnumerable<string> backends && backends.All(x => Enum.TryParse(typeof(StorageBackendImplementations), x, true, out _));
			}
		}

		[ValidStorageBackend]
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Naming", "CA1721:Property names should not match get methods", Justification = "This pattern is used to work around limitations in dotnet configurations support for enums in arrays")]
		public string[]? StorageImplementations { get; set; }

		public IEnumerable<UnrealCloudDDCSettings.StorageBackendImplementations> GetStorageImplementations()
		{
			foreach (string s in StorageImplementations ?? new[] { UnrealCloudDDCSettings.StorageBackendImplementations.Memory.ToString() })
			{
				UnrealCloudDDCSettings.StorageBackendImplementations impl = (UnrealCloudDDCSettings.StorageBackendImplementations)Enum.Parse(typeof(UnrealCloudDDCSettings.StorageBackendImplementations), s, ignoreCase: true);

				yield return impl;
			}
		}

		[Required] public ReplicationLogWriterImplementations ReplicationLogWriterImplementation { get; set; } = ReplicationLogWriterImplementations.Memory;

		[Required]
		public ReferencesDbImplementations ReferencesDbImplementation { get; set; } = ReferencesDbImplementations.Memory;

		public LeaderElectionImplementations LeaderElectionImplementation { get; set; } = LeaderElectionImplementations.Static;
		public ContentIdStoreImplementations ContentIdStoreImplementation { get; set; } = ContentIdStoreImplementations.Memory;
		public BlobIndexImplementations BlobIndexImplementation { get; set; } = BlobIndexImplementations.Memory;
		public BuildStoreImplementations BuildStoreImplementation { get; set; } = BuildStoreImplementations.Memory;
		public ServiceDiscoveryImplementations ServiceDiscoveryImplementation { get; set; } = ServiceDiscoveryImplementations.Static;

		public int? MaxSingleBlobSize { get; set; } = null; // disable blob partitioning

		public int LastAccessRollupFrequencySeconds { get; set; } = 900; // 15 minutes
		public bool EnableLastAccessTracking { get; set; } = true;
		public bool EnableOnDemandReplication { get; set; } = true;

		public bool EnableBucketStatsTracking { get; set; } = false;
		public bool EnableInlineSmallBlobs { get; set; } = true;

		/// <summary>
		/// Forces the inlined blobs to also be submitted into the blob store, is the old behavior and is not recommended.
		/// </summary>
		public bool EnableForceSubmitRefBlobToBlobStore { get; set; } = false;

		public long InlineBlobMaxSize { get; set; } = 32 * 1024; // default to 32 kb blobs max

		/// <summary>
		/// Used to force a more parallel resolve than normal (which has no limits). Can be used to go wider then the number of processors which improves how quickly the finalize for a ref can be done, especially useful for refs with a lot of dependencies
		/// </summary>
		public int RefFinalizeResolveMaxParallel { get; set; } = Environment.ProcessorCount;

		/// <summary>
		/// Set to enable using the new last access service which is registered as a background service
		/// </summary>
		public bool UseLastAccessBackgroundService { get; set; } = false;

		/// <summary>
		/// Remove the incoming blob references when the blob is deleted
		/// </summary>
		public bool RemoveReferencesOnBlobDeletion { get; set; } = true;
	}

	public class MongoSettings
	{
		[Required] public string ConnectionString { get; set; } = "";

		public bool RequireTls12 { get; set; } = true;
		public bool CreateDatabaseIfMissing { get; set; } = true;
	}

	public class MemoryCacheContentIdSettings : MemoryCacheOptions
	{
		public bool Enabled { get; set; } = true;

		public bool EnableSlidingExpiry { get; set; } = true;
		public int SlidingExpirationMinutes { get; set; } = 120;
	}

	public class MemoryCacheReferencesSettings : MemoryCacheOptions
	{
		public bool Enabled { get; set; } = true;

		public bool EnableSlidingExpiry { get; set; } = true;
		public int SlidingExpirationMinutes { get; set; } = 120;
	}

	public class AzureSettings
	{
		[Required] public string ConnectionString { get; set; } = string.Empty;

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Modified by settings")]
		// ReSharper disable once CollectionNeverUpdated.Global
		public Dictionary<string, string> StoragePoolConnectionStrings { get; set; } = new Dictionary<string, string>();

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Modified by settings")]
		// ReSharper disable once CollectionNeverUpdated.Global
		public Dictionary<string, string> StoragePoolContainerOverride { get; set; } = new Dictionary<string, string>();
	}

	public class FilesystemSettings
	{
		[Required] public string RootDir { get; set; } = "";
		public ulong MaxSizeBytes { get; set; } = 500 * 1024 * 1024;
		public double TriggerThresholdPercentage { get; set; } = 0.95;
		public double TargetThresholdPercentage { get; set; } = 0.85;

		/// <summary>
		/// Enable to switch garbage collection to per namespace mode where every namespace has their own dedicated amount of space they can use
		/// This allows you to specify the MaxFilesystemStorageBytes option per namespace, the MaxSizeBytes in the Filesystem settings is considered the default value for namespaces that to not override it
		/// </summary>
		public bool PerNamespaceGC { get; set; } = false;
	}

	public class S3Settings
	{
		[Required] public string ConnectionString { get; set; } = "";

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Modified by settings")]
		public Dictionary<string, Dictionary<string, string>> SiteBucketsPerStoragePool { get; set; } = new Dictionary<string, Dictionary<string, string>>();

		public string BucketName { get; set; } = "";

		public bool ForceAWSPathStyle { get; set; }
		public bool AssumeHttpForRedirectUri { get; set; } = false;
		public bool CreateBucketIfMissing { get; set; } = true;

		// Options to disable setting of bucket access policies, useful for local testing as minio does not support them.
		public bool SetBucketPolicies { get; set; } = true;

		public bool UseBlobIndexForExistsCheck { get; set; } = false;

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Modified by settings")]
		// ReSharper disable once CollectionNeverUpdated.Global
		public Dictionary<string, string> StoragePoolBucketOverride { get; set; } = new Dictionary<string, string>();

		public bool? UseArnRegion { get; set; } = null;
		public bool UseMultiPartUpload { get; set; } = true;
		public bool UseMultiPartDownload { get; set; } = true;

		/// <summary>
		/// Allows you to override S3 behavior with chunk encoding, this needs to be set to false for uploads against GCS
		/// </summary>
		public bool UseChunkEncoding { get; set; } = true;

		/// <summary>
		/// Keeps S3 list queries within one prefix (first 2 bytes in the hash) - can help reduce errors from S3 about to many operations
		/// Is also a speed up on larger datasets
		/// </summary>
		public bool PerPrefixListing { get; set; } = true;

		/// <summary>
		/// Max number of keys returned in a single request when listing S3
		/// </summary>
		public int PerPrefixMaxKeys { get; set; } = 10_000;

		/// <summary>
		/// Max size of bytes that is buffered into memory before switching to filesystem buffers
		/// Larger value means more RAM used but less load on the filesystem.
		/// </summary>
		public int MultiPartMaxMemoryBufferSize { get; set; } = int.MaxValue;

		/// <summary>
		/// Set this to disable payload signing and checksum generation	 during S3 uploads, saves cpu load when uploading large objects at some consistency risk and a requirement to use https connections see
		/// https://docs.aws.amazon.com/sdkfornet/v3/apidocs/items/S3/TTransferUtilityUploadRequest.html
		/// </summary>
		public bool? DisablePayloadChecksums { get; set; } = null;
	}

	public class GCSettings
	{
		public bool BlobCleanupServiceEnabled { get; set; } = true;

		public bool CleanOldRefRecords { get; set; } = false;
		public bool CleanOldBlobs { get; set; } = true;
		public bool CleanTempFiles { get; set; } = true;
		public bool RunFilesystemCleanup { get; set; } = false;

		public TimeSpan LastAccessCutoff { get; set; } = TimeSpan.FromDays(14);

		public TimeSpan BlobCleanupPollFrequency { get; set; } = TimeSpan.FromMinutes(60);
		public TimeSpan RefCleanupPollFrequency { get; set; } = TimeSpan.FromMinutes(60);
		public TimeSpan TempCleanupPollFrequency { get; set; } = TimeSpan.FromMinutes(60);
		public TimeSpan TempCleanupMaximumAge { get; set; } = TimeSpan.FromHours(6);
		public int OrphanGCMaxParallelOperations { get; set; } = 8;
		public int OrphanRefMaxParallelOperations { get; set; } = 8;
		public bool WriteDeleteToReplicationLog { get; set; } = false;
		public NamespacePolicy.StoragePoolGCMethod DefaultGCPolicy { get; set; } = NamespacePolicy.StoragePoolGCMethod.LastAccess;
		public bool AllowBlockMetadataCleanup { get; set; } = true;

		/// <summary>
		/// How long a blob needs to be around before its considered for potential GC
		/// </summary>
		public int OrphanBlobCutoffDurationMinutes { get; set; } = 60;
	}

	public class UpstreamRelaySettings
	{
		[Required] public string ConnectionString { get; set; } = null!;
	}

	public class ClusterSettings
	{
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<PeerSettings> Peers { get; set; } = new List<PeerSettings>();

		public DiscoverySettings? Discovery { get; set; } = null;
	}

	public class PeerSettings
	{
		[Required] public string Name { get; set; } = null!;

		[Required] public string FullName { get; set; } = null!;

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "Used by serialization")]
		public List<PeerEndpoints> Endpoints { get; set; } = new List<PeerEndpoints>();
	}

	public class PeerEndpoints
	{
		[Required] public Uri Url { get; set; } = null!;

		public bool IsInternal { get; set; } = false;
	}

	public class ConsistencyCheckSettings
	{
		public bool EnableBlobStoreChecks { get; set; } = false;
		public bool EnableBlobIndexChecks { get; set; } = false;
		public bool EnableRefStoreChecks { get; set; } = false;
		public double ConsistencyCheckPollFrequencySeconds { get; set; } = TimeSpan.FromHours(2).TotalSeconds;
		public int BlobIndexMaxParallelOperations { get; set; } = 4;
		public bool AllowDeletesInBlobIndex { get; set; } = false;
		public bool RunBlobStoreConsistencyCheckOnRootStore { get; set; } = false;
		public bool CheckRefStoreLastAccessTimeConsistency { get; set; } = false;
		public bool CheckRefStoreRegionalConsistency { get; set; } = false;
		public bool CheckRefStorePartialObjects { get; set; } = false;
		public string[] RegionalConsistencyCheckNamespaces { get; set; } = Array.Empty<string>();
		public bool EnableBlobReferenceChecks { get; set; } = false;
		public bool EnableBuildStoreConsistencyCheck { get; set; } = false;
		public bool AllowDeletesInBuildStore { get; set; } = false;
		public bool CheckBuildStorePartialObjects { get; set; } = false;
		public bool CheckBuildStoreMissingBlobs { get; set; } = false;
		public int BuildStoreConsistencyMaxParallelBuilds { get; set; } = 4;
		public int BuildStoreConsistencyMaxParallelBlobs { get; set; } = 4;
	}
}
