---
Serilog:
  MinimumLevel:
    Override:
      Microsoft: "Information"

UnrealCloudDDC:
  # switch implementations over to cache version
  ReferencesDbImplementation: Cache
  ContentIdStoreImplementation: Relay
  BlobIndexImplementation: Cache

  StorageImplementations:
  - Filesystem # cache data in the filesystem
  - Relay # if not available in the local filesystem relay to the upstream

# Disable GC as the cache layer maintains themselves, and we do not need to maintain the rest
GC:
 CleanOldBlobs: false
 CleanOldRefRecords: false

Replication:
  Enabled: false
  StateRoot: "$(ExecutableLocation)/replication-state" # the replication state is actually not in use but the settings is required

Mongo:
  ConnectionString: "mongodb://localhost:27017" # we assume you are running a mongo db on the localhost

# Required settings
#Filesystem:
#  RootDir:

#Upstream:
#  ConnectionString:

#Jupiter:
#  CurrentSite: "unique-identifier-of-this-cache-site"

# you will likely need some auth configuration as well
