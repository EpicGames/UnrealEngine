{
    "Serilog": {
        "Using": [ "Serilog.Sinks.Console" ],
        "MinimumLevel": {
            "Default": "Information",
            "Override": {
                "Microsoft": "Information",
                "System": "Information"
            }
        },
        "WriteTo": [
            {
                "Name": "Console"
            }
        ],
        "Enrich": [ "FromLogContext" ]
    },

    "Mongo": {
        // this assumes a mongo db were we have localhost exemption, e.g. we do not need to authenticate
        // if using mongo via a docker container you will need to update this
        "ConnectionString": "mongodb://localhost:27017",
        "RequireTls12": true
    },

    // scylla is normally not enabled in development but we still configure it so we can easily switch
    "Scylla": {
        "LocalDatacenterName": "datacenter1",
        "LocalKeyspaceSuffix": "local"
    },

    "UnrealCloudDDC": {
        // assume a single instance is running
        "LeaderElectionImplementation": "Static",

        // use mongo for any database needs
        "ReferencesDbImplementation": "Mongo",
        "ContentIdStoreImplementation": "Mongo",
        "BlobIndexImplementation": "Mongo",

        // store blobs in the filesystem
        "StorageImplementations": [
            "Filesystem"
        ]
    },

    "Filesystem": {
        "RootDir": "$(ExecutableLocation)/BlobStorage"
    },

    "Replication": {
        "StateRoot": "$(ExecutableLocation)/ganymede",
        "ReplicationPollFrequencySeconds": 15,
        "CurrentSite": "local-test",
        "Enabled": false
    },

    "GC": {
        "BlobCleanupServiceEnabled": false,
        "CleanOldBlobs": false,
        "CleanOldRefRecords": false
    },

    "Jupiter": {
        "CurrentSite": "local",
        "PublicApiPorts": [ 57001, 57002 ],
        "CorpApiPorts": [ 57003, 57004 ],
        "InternalApiPorts": [ 57005, 57006 ]
    },

    "Cluster": {
		"Peers": [
			{
				"Name": "local"
			}
		]
    },

    // disable auth and allow any namespace
    "Auth": {
        "Enabled": false
    },

    "Namespaces": {
        "Policies": {
            "*": {
                "Acls": [
                    {
                        "Claims": ["*"]
                    }
                ]
            },
            "test.private": {
                "Acls": [
                    {
                        "Claims": [ "*" ]
                    }
                ],
                "IsPublicNamespace": false
            }
        }
    },

    "AllowedHosts": "*",


    "Kestrel": {
        "Endpoints": {
            "Http": {
                "Url": "http://0.0.0.0:57001"
            },
            "Http2": {
                "Url": "http://0.0.0.0:57002",
                "Protocols": "Http2"
            },
            "CorpHttp": {
                "Url": "http://0.0.0.0:57003"
            },
            "CorpHttp2": {
                "Url": "http://0.0.0.0:57004",
                "Protocols": "Http2"
            },
            "InternalHttp": {
                "Url": "http://0.0.0.0:57005"
            },
            "InternalHttp2": {
                "Url": "http://0.0.0.0:57006",
                "Protocols": "Http2"
            }
        }
    }
}
