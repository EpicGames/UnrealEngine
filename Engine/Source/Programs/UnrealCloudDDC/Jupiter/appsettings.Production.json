{
    "Serilog": {
        "Using": [ "Serilog.Sinks.Console" ],
        "MinimumLevel": {
            "Default": "Information",
            "Override": {
                "Microsoft": "Warning",
                "System": "Warning"
            }
        },
        "WriteTo": [
            {
                "Name": "Console",
                "Args": {
                    "formatter": "Serilog.Formatting.Json.JsonFormatter"
                }
            }
        ],
        "Enrich": [ "FromLogContext" ]
    },

    "UnrealCloudDDC": {
        "ReplicationLogWriterImplementation": "Scylla",
        "ReferencesDbImplementation": "Scylla",
        "ContentIdStoreImplementation": "Scylla",
        "BlobIndexImplementation": "Scylla"
    },
    "Mongo": {
        "RequireTls12": true
    },

    "Jupiter": {
        "PublicApiPorts": [ 80, 8081 ],
        "CorpApiPorts": [ 8008, 8082 ],
        "InternalApiPorts": [ 8080, 8083 ]
    },

    "AllowedHosts": "*",
    "Kestrel": {
        "Endpoints": {
            "Http": {
                "Url": "http://0.0.0.0:80"
            },
            "Http2": {
                "Url": "http://0.0.0.0:8081",
                "Protocols": "Http2"
            },
            "CorpHttp": {
                "Url": "http://0.0.0.0:8008"
            },
            "CorpHttp2": {
                "Url": "http://0.0.0.0:8082",
                "Protocols": "Http2"
            },
            "InternalHttp": {
                "Url": "http://0.0.0.0:8080"
            },
            "InternalHttp2": {
                "Url": "http://0.0.0.0:8083",
                "Protocols": "Http2"
            }
        }
    }
}
