{
	"Serilog": {
		"Using": [ "Serilog.Sinks.Console" ],
		"MinimumLevel": {
			"Default": "Information",
			"Override": {
				"Microsoft": "Warning",
				"System": "Information"
			}
		},
		"WriteTo": [
			{ "Name": "Console" }
		],
		"Enrich": [ "FromLogContext", "WithMachineName", "WithThreadId" ],
		"Properties": {
			"Application": "UnrealCloudDDC"
		}
	},

	"Mongo": {
		"ConnectionString": "mongodb://root:abc123@localhost:27018"
	},

	"Scylla": {
		"LocalDatacenterName": "datacenter1",
		"LocalKeyspaceSuffix": "test"
	},

	"UnrealCloudDDC": {
		"StorageImplementations": [ "memory" ]
	},

	"GC": {
		"BlobCleanupServiceEnabled": false,
		"CleanOldBlobs": false,
		"CleanOldRefRecords": false,
		"CleanTempFiles": false
	},

	"S3": {
		// use a local S3 emulation when running the tests
		"ConnectionString": "http://127.0.0.1:9000",
		"ForceAWSPathStyle": true, // required by minio,
		"AssumeHttpForRedirectUri": true, // minio runs over http but AWS SDK generates https links for redirect uris, force them to http
		"SetBucketPolicies": false, // minio does not support bucket access policies
		"BucketPrefix": "tests"
	},
	"AWSCredentials": {
		// minio and health checks expect access keys to have been set
		// note that these credentials are not actually real credentials but rather something as we pass into minio when starting it
		"AwsAccessKey": "FSYPTVCPKC2IV5D1U8RE",
		"AwsSecretKey": "WaUf4Zq+J6YMlvbCVvTRkQJIPc7DvZwTm5UP0tPJ"
	},
	"Azure": {
		"ConnectionString": "DefaultEndpointsProtocol=https;BlobEndpoint=http://localhost:11002/blobaccount;AccountName=blobaccount;AccountKey=iU6uTvlF1ysppmft+NO5lAD0E3hwrAORr5Rb5xcBWUgEz/OicrSkFxwZYMNK5XL29/wXZKGOoOVSW040nAOfPg=="
	},
	"Auth": {
		"Enabled": false,
		"RequireAcls": true,
		"Policies": [
			{
				"Claims": [ "*" ],
				"Actions": [
					"ReadObject",
					"WriteObject",
					"EnumerateBucket",
					"DeleteObject",
					"ReadBuildPart",
					"WriteBuildPart",
					"DeleteBucket",
					"ReadTransactionLog",
					"WriteTransactionLog",
					"AdminAction"
				],
				"Scope": {
					"Namespaces": [
						{
							"values": [ "test-builds-scylla", "test-builds-memory", "test-builds-copy-scylla", "test-builds-copy-memory" ]
						}
					],
					"Buckets": [
						/* access to all buckets except the ones that end with secret-platform */
						{
							"values": [ "secret-platform" ],
							"match": ".*\\..*\\..*\\.(.*)",
							"not": true
						}
					],
					"Resources": [
						{
							"values": [ "*" ],
							"resourceType":  "parts"
						}
					]
				}
			},
			{
				"Claims": [ "*" ],
				"Actions": [
					"ReadObject",
					"WriteObject",
					"ReadBuildPart",
					"WriteBuildPart",
					"EnumerateBucket"
				],
				"Scope": {
					"Namespaces": [
						{
							"values": [ "test-builds-wildcard" ]
						}
					],
					"Buckets": [
						{
							"values": [ "*" ]
						}
					],
					"Resources": [
						{
							"values": [ "*" ],
							"resourceType": "parts"
						}
					]
				}
			},
			{
				"Claims": [ "*" ],
				"Actions": [
					"ReadObject",
					"WriteObject",
					"DeleteObject"
				],
				"Scope": {
					"Namespaces": [
						{
							"values": [
								"ue4.ddc", "TestNamespace", "testbucket", "fallback-namespace","first-namespace", "second-namespace", "test-blobindex-memory", "test-blobindex-list-memory", "test-blobindex-mongo", "test-blobindex-list-mongo", "test-blobindex-scylla", "test-blobindex-list-scylla", "test-blobindex-cassandra", "test-blobindex-list-cassandra" ]
						}
					],
					"Buckets": [
						{
							"values": [ "*" ]
						}
					]
				}
			},
			{
				"Claims": [ "*" ],
				"Actions": [
					"ReadObject",
					"WriteObject",
					"DeleteObject",
					"DeleteBucket",
					"ReadTransactionLog",
					"WriteTransactionLog",
					"AdminAction"
				],
				"Scope": {
					"Namespaces": [
						{
							"values": [ "test-namespace", "test-namespace-scylla", "test-namespace-mongo", "test-namespace-memory", "test-namespace-memory", "test-namespace-cassandra", "test-namespace-replication", "test-namespace-enumeration", "test-namespace-bundle", "test-namespace-bypass-cache", "test-namespace-redirect", "test-namespace-nginx-redirect", "test-symbols-memory", "test-symbols-scylla", "test-namespace-galileo", "test-namespace-datastore" ]
						}
					],
					"Buckets": [
						{
							"values": [ "*" ]
						}
					]
				}
			},
			{
				"Claims": [ "*" ],
				"Actions": [
					"ReadObject",
					"WriteObject",
					"DeleteObject",
					"DeleteBucket",
					"DeleteNamespace"
				],
				"Scope": {
					"Namespaces": [
						{
							"values": [ "test-delete-namespace", "test-namespace-delete-cassandra", "test-namespace-delete-scylla", "test-namespace-delete-mongo", "test-namespace-delete-memory", "test-namespace-no-overwrite-memory", "test-namespace-no-overwrite-mongo", "test-namespace-no-overwrite-scylla", "test-namespace-no-overwrite-cassandra", "test-namespace-delete-galileo", "test-namespace-delete-datastore", "test-namespace-no-overwrite-galileo", "test-namespace-no-overwrite-datastore" ]
						}
					],
					"Buckets": [
						{
							"values": [ "*" ]
						}
					]
				}
			},
			{
				// global operations run in all namespaces and need the admin action
				// this should really be protected with a specific claim but for tests we disable auth so we do not have claims to control on
				// do NOT copy this setup for production deployments
				"Claims": [ "*" ],
				"Actions": [
					"AdminAction"
				],
				"Scope": {
					"Namespaces": [
						{
							// global operations
							"values": [ "*" ]
						}
					],
					"Buckets": [
						{
							"values": [ "*" ]
						}
					]
				}
			}
		]
	},
	"Namespaces": {
		"Policies": {
			"test-namespace": {
				"StoragePool": "test"
			},
			"test-namespace-bundle": {
				"UseContentAddressedStorage": false,
				"AllowRedirectUris": true
			},
			"test-namespace-bypass-cache": {
				"BypassCacheOnWrite": true
			},
			"test-namespace-nginx-redirect": {
				"BypassCacheOnWrite": true
			},
			"test-namespace-redirect": {
				"AllowRedirectUris": true
			},
			"test-namespace-no-overwrite-memory": {
				"AllowOverwritesOfRefs": false
			},
			"test-namespace-no-overwrite-mongo": {
				"AllowOverwritesOfRefs": false
			},
			"test-namespace-no-overwrite-scylla": {
				"AllowOverwritesOfRefs": false
			},
			"test-namespace-no-overwrite-cassandra": {
				"AllowOverwritesOfRefs": false
			},
			"test-namespace-no-overwrite-datastore": {
				"AllowOverwritesOfRefs": false
			},
			"test-namespace-no-overwrite-galileo": {
				"AllowOverwritesOfRefs": false
			},
			"test-builds-scylla": {
				"UseBlobIndexForExists": true
			},
			"test-builds-copy-scylla": {
				"UseBlobIndexForExists": true
			},
			"first-namespace": {
				"FallbackNamespace": "fallback-namespace"
			},
			"second-namespace": {
				"FallbackNamespace": "fallback-namespace-no-auth"
			}
		}
	},

	"AllowedHosts": "*",

	"Jupiter": {
		"CurrentSite": "test",
		"AssumeLocalConnectionsHasFullAccess": "true",
		"InternalApiPorts": [ 0 ] // consider all traffic as internal traffic so we can each the replication endpoints
	},

	"Kestrel": {
		"Endpoints": {
			"Http": {
				"Url": "http://localhost:57001"
			},
			"Http2": {
				"Url": "http://localhost:57002",
				"Protocols": "Http2"
			}
		}
	}
}
