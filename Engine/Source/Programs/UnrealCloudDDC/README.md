# Introduction

UnrealCloudDDC - A Distributed storage service primarily used as a Cloud based DDC cache.

Unreal Cloud DDC stores Compact Binary objects in a Content Addressable store that can be replicated across the world. Compact binary objects are self describing key value objects (think json) that has good binary serialization and a extensive type system. One of the key types are the Attachment types which lets you have a object that references a large blob without containing it. These attached payloads are content addressed (meaning the hash of the payload is used as the identifier of it), content addressing is a common practise in distributed storage systems most notiably in git. Content addressing allows us to quickly generate a immutable identifier of a payload to quickly determine if its already available in a cache and to determine if we need to replicate this somewhere else.

Unreal Cloud DDC can signficantly help teams speed up their cook processes in the Unreal Engine for cases were they do not have access to a primied local (or fileshare) cache, specifically for cases when users are remote. Its a fairly complicated software to setup and operate so we do not think this is a great fit for every team that uses Unreal, but if you are geographically distributed and have engineers with cloud experience then this can be very helpful.

# Table of contents
- [Introduction](#introduction)
- [Table of contents](#table-of-contents)
- [License](#license)
- [Directories](#directories)
- [Dependencies](#dependencies)
- [Functional Test requirements](#functional-test-requirements)
- [Running locally](#running-locally)
- [Deployment](#deployment)
  - [Scylla](#scylla)
  - [AWS](#aws)
  - [On premise](#on-premise)
  - [Azure](#azure)
  - [GCE](#gce)
  - [Testing your deployment](#testing-your-deployment)
- [Monitoring](#monitoring)
  - [Health Checks](#health-checks)
- [Authentication](#authentication)
  - [IdentityProvider setup](#identityprovider-setup)
  - [ACL Policies](#acl-policies)
- [Common Configuration](#common-configuration)
  - [DDC](#ddc)
  - [Oplogs](#oplogs)
- [Networking setup](#networking-setup)
  - [Public Port](#public-port)
  - [Private port](#private-port)
  - [Internal Port](#internal-port)
- [Common operations](#common-operations)
  - [Running a local cook against a local instance](#running-a-local-cook-against-a-local-instance)
  - [Add new region](#add-new-region)
  - [Blob replication setup](#blob-replication-setup)

# License
The source of Unreal Cloud DDC is covered by the regular Unreal Engine source license.
We do provide container images at https://github.com/orgs/EpicGames/packages/container/package/unreal-cloud-ddc - these containers are provided under MIT.

# Directories 

* Jupiter - Most of the UnrealCloudDDC functionality.
* Jupiter.Common - Lib - Legacy library with functionality that could be used outside of UnrealCloudDDC.
* Helm - Helm charters for all components
* Benchmark - Useful templates for doing simple http benchmarking user SuperBenchmark (sb) as well as a docker container that can be used to run benchmarks using vegeta.
* Composes - a bunch of different docker compose files for easier local deployment and testing

# Dependencies

* DotNet Core 6 (and Visual Studio 2022 or VS Code)
* Docker
* Database (Scylla is recommended, MongoDB is also supported for single regions)
* Blob storage (S3, S3 emulations like Minio, Azure Blob Store or a local filesystem)
* Docker Compose

# Functional Test requirements

Before starting the tests start the prerequite services using this command line
`docker-compose -f Composes\docker-compose-tests.yml -p jupiter-test up`


# Running locally
If you want to try out UnrealCloudDDC you can easily start it up using docker-compose. Note that as UnrealCloudDDC supports multiple different backends we provide different compositions for your use-case.
To start run:
`docker-compose -f Composes\docker-compose.yml -f Composes\docker-compose-aws.yml up --build`
The AWS compose file can be replaced with `docker-compose-azure.yml` if you want to run with services closer to what is available on Azure.

Docker compose setups disable authentication for to make it quick to get started, generally we recommend that you hook UnrealCloudDDC up to a OIDC provider before deploying this.

# Deployment
UnrealCloudDDC is currently only run in production on AWS for Epic, but the requirements on storage and db are very generic and also abstracted. 
We have licensees running on Azure and include support for that but we have limited testing coverage for this.
GCE can be used by using GCS with its S3 api which we have licensees that do, as with azure we have very limited testing for this mode.

We provide helm values (under `/Helm`) that we use for epic internal deployments to kubernetes, but kubernetes is not a requirement.

Docker images are published to the Epic Games github organization `https://github.com/orgs/EpicGames/packages/container/package/unreal-cloud-ddc`

## Scylla
You will need to setup a scylla cluster for UnrealCloudDDC to talk to.
UnrealCloudDDC supports running with Scylla Open Source (which is free) or their paid offerings. The paid offerings can help reduce the amount of effort you need to put into managing the cluster as Scylla manager will help you with maintainance tasks.

Some useful links from Scylla on how to setup a multi region cluster:
https://docs.scylladb.com/stable/operating-scylla/procedures/cluster-management/create-cluster-multidc.html

Download link to open source version:
https://www.scylladb.com/download/#open-source

Scylla provides machine images for use in cloud environments.

## AWS
This is the most tested deployment form as this is how we operate it at Epic. The helm chart we install into each regions kubernetes cluster is available in the  `Helm` directory.

## On premise
UnrealCloudDDC can be deployed onprem without using any cloud resources. You can either setup a Mongo database for this (if you only intend to run this in a single region) or Scylla if you inted to run it multi region but still on premise. If you are starting with one region but might expand later we recommend using Scylla - as that allows you to just scale out while Mongo would require drop all your existing state.

## Azure
To deploy on AWS you will just need to set Azure as your cloud provider and specify the `Azure.ConnectionString` setting with a connection string to your Azure Blob Storage.

## GCE
To run using GCS you will need to use the S3 api they provide as well as set the `S3.UseChunkEncoding` setting to `false`

## Testing your deployment
Once you have a deployment up and running you can connect to the machine and run curl commands to verify its working as it should.

First you can attempt to use the health checks, these should just return the string `Health`
```
curl http://localhost/health/live
```

Next you can attempt to add and fetch content into a namespace
This will insert a test string (`test`) into the `test-namespace`, you may need to use a different namespace depending on your setup. This also assumes you have authentication disabled. This should return a 200 status code with a empty "needs" list.
```
curl http://localhost/api/v1/refs/test-namespace/default/00000000000000000000000000000000000000aa -X PUT --data 'test' -H 'content-type: application/octet-stream' -H 'X-Jupiter-IoHash: 4878CA0425C739FA427F7EDA20FE845F6B2E46BA' -i
```

After that you can attempt to retrive this object, this should print the 'test' string and a 200 staus code.

```
curl http://localhost/api/v1/refs/test-namespace/default/00000000000000000000000000000000000000aa.raw -i
```

# Monitoring
We use Datadog to monitor our services, as such UnrealCloudDDC is instrumented to work well with that service. All logs are delivered as structured logs to stdout, so any monitoring service that understands structured logs should be able to monitor it quite well. Traces are output using OpenTelemetry formats so any monitoring service that can ingest that should be compatible.

## Health Checks
All UnrealCloudDDC services use health checks to monitor themselves, any background services they may run and any dependent service they may have (DB / Blob store etc).
You can reach the health checks at `/health/live` and `/health/ready` for live and ready checks respectively. Ready checks is used to verify that the service is working, if this returns false the app will not get any traffic (load balancer ignores it). Live checks are used to see if the pod is working as it should, if this returns false the entire pod is killed. This only applies when run in a kubernetes cluster.

# Authentication
UnrealCloudDDC supports using any OIDC provider that does JWT verfication for authentication. We use Okta at Epic so this is what has been tested but other OIDCs should be compatible as well.

You configure authentication in two steps, setting up the IdentityProvider (IdP) and then setting up authorization for each namespace.

## IdentityProvider setup

You specify auth schemes to in the `auth` settings. 
```  
auth:
    defaultScheme: Bearer
    schemes:
      Bearer: 
        implementation: "JWTBearer"
        jwtAudience: "api://unreal"
        jwtAuthority: "<url-to-your-idp>
```
We recommend naming your scheme `Bearer` if its your first and only scheme. You can use multiple schemes to connect against multiple IdPs, this is mostly useful during a migration.

The implementation field is usually `JWTBearer` but we do offer a `Okta` if you are using Okta with custom auth servers, for Okta using the org auth server you will need to use `JWTBearer` as well.

## ACL Policies
Access to operations within UnrealCloudDDC is controlled using a set of actions:
```
    - ReadObject
    - WriteObject
    - DeleteObject
    - ReadBuildPart
    - WriteBuildPart
    - EnumerateBucket
    - DeleteBucket
    - DeleteNamespace
    - ReadTransactionLog
    - WriteTransactionLog
    - AdminAction
```

These can be assigned to an explicit set of namespaces and / or bucket (not all apis support per bucket access control).
Example configuration that sets transaction log access for users that have access to it, admin access for admins and then per namespace access.

```
    policies:
    - claims:
      - groups=app-ddc-storage-transactionlog-full
      actions:
      - ReadTransactionLog
      - WriteTransactionLog
      scope:
        namespaces:
        - values: "*"
        buckets:
        - values: "*"

    - claims:
      - groups=app-ddc-storage-ue-generic
      actions:
      - ReadObject
      - WriteObject
      - EnumerateObject
      - ReadBuildPart
      - WriteBuildPart
      scope:
        namespaces:
        - values: 
          - ue.ddc 
        buckets:
        - values: ["*"]

    - claims:
      - groups=app-ddc-storage-admin
      scopes:
        namespaces:
        - values: 
          - "*"
        buckets:
        - values:
          - "*"
      actions:
      - ReadObject
      - WriteObject
      - DeleteObject
      - ReadBuildPart
      - WriteBuildPart
      - DeleteBucket
      - DeleteNamespace
      - AdminAction

```
If you specify multiple claims in the claims array these are ANDed together thus requiring all the claims. A claim statement like `A=B` requires claim `A` to have value `B` (or contain value `B` in the case of a array).
You can also specify the `*` claim which grants access for any valid token no matter which claims it has, this is mostly for debug / testing scenarios and should not be used for production data.

If you specify multiple values under namespace and/or bucket they are ORed (it matches that list of namespaces and buckets).
Multiple statements under the namespaces and buckets directly are ANDed (all of them have to be satisfied).

You can also use the `match` and `not` values to create more advanced rules that exclude certain patterns and uses regular expressions to extract only a part of the bucket or namespace.

This is an example of an advanced setup for builds that exclude access to certain platforms (in this example linux is considered a secret platform) as well as excluding access to a build part called "sensitive" unless 
explicit access is granted via a seperate claim:

```
    - claims:
      - groups=app-ddc-storage-ue-generic
      scopes:
        namespaces:
        - values: 
          - ue.builds
        buckets:
        # access to all buckets except the ones that are secret platforms
        - values: 
          - "linux"
          match: .*\..*\.*\.(.*)
          not: true
        resources:
        # access to all parts except the sensitive part
        - resourceType: "parts"
          values: 
          - "sensitive"
          not: true
      actions:
      - ReadObject
      - WriteObject
      - EnumerateObject
      - ReadBuildPart
      - WriteBuildPart

    - claims:
      - groups=app-ddc-storage-ue-platform-linux
      actions:
      - ReadObject
      - WriteObject
      - EnumerateObject
      scopes:
        namespaces:
        - values: 
          - ue.builds
        buckets:
        - values: 
          - linux
          match: .*\..*\.*\.(.*)

    - claims:
      - groups=app-ddc-storage-ue-sensitive-access
      actions:
      - ReadBuildPart
      - WriteBuildPart
      scopes:
        namespaces:
        - values: 
          - ue.builds
        buckets:
        - values: 
          - *
        resources:
        - resourceType: "parts"
          values: 
          - "sensitive"
```

# Common Configuration
Unreal Cloud DDC can be used for multiple types of object storage and the configuration for these different kinds can vary a bit. This section will attempt to document some of the use cases and how to configure it.
Epic applies a naming convention to namespaces that can be useful to follow, `<project>.<type>` (all lowercase), so a namespace for UE projects would be named `ue.sddc` (sddc stands for structured ddc) and a oplog namespace would be called `ue.oplog`.

Namespace will share storage amongst all namespaces that share the same storage pool and they all default to the default storage pool (no suffix on bucket names). Its recommended to seperate out `oplog` and `ddc` into seperate storage pools as there storage requirments are somewhat different and this makes it easier to get an overview of how much data you are storing of each type. 


## DDC
For DDC there is no need to customize any options in the namespace, everything defaults to a setup that is useful for ddc.

## Oplogs
For oplogs we recommend enabling certain options on your namespace. The most important is the `GCMethod` which should be set to TTL (Time to live). This will make oplog entries automatically expire after 14 days (can be overridden by setting the `DefaultTTL`). Note that the uploaded content isn't removed after those 14 days, just the reference to it. Blob garbage collection will remove unused blobs once this happens. This makes sure old oplogs are automatically removed when not needed anymore (unlike the default GC method which keep anything that is used around which can quickly become a lot of storage for these large oplogs).
The `storagePool` is something we recommend overriding to a separate pool as well. This will expect you to have a S3 bucket / Azure blob storage that matched your configured name but with the storage pool suffix. So if your s3 bucket name is set to `ue-cloud-ddc-bucket` then the expected bucket for oplogs is `ue-cloud-ddc-bucket-oplog` e.g a `-oplog` suffix.

Next you can enable the `UseBlobIndexForExists` option, this will use the metadata db (ScyllaDB) for exist checks rather then querying blob storage. This is much faster but puts more effort on your DB so make sure to have it scaled appropriately, Scylla usually handles this just fine. For large oplogs the existence checks can be a bottle neck.
Lastly we have the `OnDemandReplication` option. When this is set that means any blob that is missing in a region is fetched on demand to other remote regions. This means that you do not necessarily need to have the speculative blob replication enabled but will be much slower for the first user that pulls down a oplog. We have that oplogs generally work okay with such a setup as long as you have enough users in a region. For smaller regions the speculative replication is better and it is always a net benefit to user experience (at a cost to transfer the data and store it).

This is an example of how the policies for a namespace could be configured for oplogs.
```
namespace:
  policies:
    ue.oplog:
      storagePool: "oplog"
      UseBlobIndexForExists: true
      GcMethod: TTL
      OnDemandReplication: true
```

# Networking setup
UnrealCloudDDC derives alot of its performance from using your normal internet connection and not relying on a VPN tunnel. As its we strongly recommend that you expose UnrealCloudDDC on a public internet endpoint. From that follows recommendations like using https and setting up your authentication (see `Authentication`) to prevent anyone from accessing this data.

UnrealCloudDDC also provides multiple ports that you can use to control access level of the api.
## Public Port
This is the port that you should expose to public internet and what most users should be using to access the service. This port does not expose some of the more sensitive apis (enumerating all contents for instance). This is exposed on port `80` and as `http` within kubernetes. If you are only operating a single region this is the only port you need to expose.

## Private port
This can also be called the Corp port - its intended to be used if you have a route exposed to your intranet. The purpose of this is to have certain very sensitive namespaces that are only exposed to users on your intranet. Use the `IsPublicNamespace` (set to false) in the namespace policy to enable this. We do *NOT* recommend using this for DDC as that prevents users WFH (Working From Home) to access the namespace without a VPN (which is typically to slow for DDC use cases).

This is typically exposed on port `8008` and as `corp-http` within kubernetes.

        "PublicApiPorts": [ 80, 8081 ],
        "CorpApiPorts": [ 8008, 8082 ],
        "InternalApiPorts": [ 8080, 8083 ]

## Internal Port
The internal port is only needed to be reachable by other UnrealCloudDDC instances. This exposes everything that the private port does but also certain apis that are deemed sensitive (enumerating content via the replication log primarily).
This is exposed on port `8080` and as `internal-http` within kubernetes.
Its recommended to keep this port only to other UnrealCloudDDC instances via a private VPC or using some kind of ip-range allow list or similar.

Note that this port is primarily used for the speculative blob replication (see `Blob replication setup`). 

# Common operations

## Running a local cook against a local instance
If running a local instance (see `Running locally`) you can run your local cook against it by passing the option `-UE-CloudDataCacheHost=http://localhost` This assumes that your project has been setup to use Cloud DDC already and that it uses `UE-CloudDataCacheHost=None` as its host override (this can vary a bit between projects).

If its working as intended you should see output like this in your cooker:
```
DerivedDataCache http://localhost: HTTP DDC: Healthy
```

## Add new region

The new region will need to contain:
* S3 storage
* Compute (Kubernetes cluster or VMs)
* Scylla deployments

UnrealCloudDDC itself does not require a lot of configuration for adding a new region - you should just update your cluster settings on all nodes to include the dns of the new region.
You should also make sure to set your `LocalKeyspaceReplicationStrategy` for the new region.

Scylla does require a bit of effort to scale out to the new region, see their documentation on scaling out:
https://docs.scylladb.com/stable/operating-scylla/procedures/cluster-management/add-dc-to-existing-dc.html
Specifically the important part is updating keyspaces

```
ALTER KEYSPACE jupiter WITH replication = { 'class' : 'NetworkTopologyStrategy', '<exiting_dc>' : 3, <new_dc> : 3};
ALTER KEYSPACE system_auth WITH replication = { 'class' : 'NetworkTopologyStrategy', '<exiting_dc>' : 3, <new_dc> : 3};
ALTER KEYSPACE system_distributed WITH replication = { 'class' : 'NetworkTopologyStrategy', '<exiting_dc>' : 3, <new_dc> : 3};
ALTER KEYSPACE system_traces WITH replication = { 'class' : 'NetworkTopologyStrategy', '<exiting_dc>' : 3, <new_dc> : 3};
```
We use a replication factor of 3 everywhere so just add the name of the new region (DC).

You will also need to alter the keyspaces of each of the `local` keyspaces to set the replication factor to 0 for the new region (see `LocalKeyspaceReplicationStrategy`)
```
ALTER KEYSPACE jupiter_local_regionA WITH replication = { 'class' : 'NetworkTopologyStrategy', 'regionA' : 2, 'regionB' : 0}
ALTER KEYSPACE jupiter_local_regionB WITH replication = { 'class' : 'NetworkTopologyStrategy', 'regionA' : 0, 'regionB' : 2}
```
This makes sure that the local keyspace is only written to the local region - while this isnt cruical this data will only ever be requested within that region and as such this saves you on a lot of bandwidth and storage within the Scylla cluster.

You will also likely want to update your replicators in your UnrealCloudDDC worker configuration to replicate from this new region. See `Blob replication` setup.

## Blob replication setup

UnrealCloudDDC has two methods of replication. On-demand replication and speculative replication.
On-demand replication will copy a blob from region A to region B as requests happen in region B that is missing the required blob. This type of replication is opt in per namespace by setting the `OnDemandReplication` to true in the namespace policy. We do *NOT* recommend setting this for DDC namespaces as it causes response times to be very variable. For DDC its better to accept the cache miss and rebuild the content in that case, but generally rely on the speculative replication to transfer blobs so that they are available everywhere without the added latency.

Speculative replication uses a journal kept in each region as refs are added to know which content to replicate. This will follow along as changes happen in a namespace and copy all blobs that are being referenced by these new refs. This will end up copying all content, which may never actually be used or needed in the local region but has the benefit of most often having a local blob available once a ref is being resolved. For DDC, where response times are quite important to keep low, we recommended relying on the speculative replication.
To set it up you will add a section to your worker configuration like this (see `example-values-ABC.yaml`)
```
worker:
  config:
    Replication:
      Enabled: true
      Replicators: 
      - ReplicatorName: DEF-to-ABC-test-namespace
        Namespace: test-namespace
        ConnectionString: http://url-to-region-DEF.com
```
The replicator name can be any string that uniquly identifies this replicator (used to store the state of where that replicator has gotten to as well as being used in the logging)
`Namespace` is the namespace to replicate.
`ConnectionString` is the url to use to connect to the other regions UnrealCloudDDC deployment.
This needs to be exposed using the internal ports (see `Networking setup`). You will also need to have credentials setup for UnrealCloudDDC to use (in the `ServiceCredentials` section) and those credentials will need to have `ReadTransactionLog` access)
