<wpf:ResourceDictionary xml:space="preserve" xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml" xmlns:s="clr-namespace:System;assembly=mscorlib" xmlns:ss="urn:shemas-jetbrains-com:settings-storage-xaml" xmlns:wpf="http://schemas.microsoft.com/winfx/2006/xaml/presentation">
	<s:Boolean x:Key="/Default/CodeStyle/CodeFormatting/CommonFormatter/AUTODETECT_INDENT_SETTINGS/@EntryValue">False</s:Boolean>
	<s:String x:Key="/Default/CodeStyle/CodeFormatting/CSharpCodeStyle/MODIFIERS_ORDER/@EntryValue">public private protected internal static new abstract virtual sealed readonly override extern unsafe volatile async</s:String>
	<s:Int64 x:Key="/Default/CodeStyle/CodeFormatting/CSharpFormat/WRAP_LIMIT/@EntryValue">160</s:Int64>
	<s:Boolean x:Key="/Default/CodeStyle/EditorConfig/SyncToVisualStudio/@EntryValue">False</s:Boolean>
	<s:String x:Key="/Default/CodeStyle/FileHeader/FileHeaderText/@EntryValue">Copyright $CREATED_YEAR$-$CURRENT_YEAR$ Epic Games, Inc. All Rights Reserved.</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=AKV/@EntryIndexedValue">AKV</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=AWS/@EntryIndexedValue">AWS</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=BLAKE/@EntryIndexedValue">BLAKE</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=CB/@EntryIndexedValue">CB</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=DB/@EntryIndexedValue">DB</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=DDC/@EntryIndexedValue">DDC</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=GC/@EntryIndexedValue">GC</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=JWT/@EntryIndexedValue">JWT</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=OK/@EntryIndexedValue">OK</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=REAPI/@EntryIndexedValue">REAPI</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/Abbreviations/=SHA/@EntryIndexedValue">SHA</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/PredefinedNamingRules/=PrivateStaticFields/@EntryIndexedValue">&lt;Policy Inspect="True" Prefix="s_" Suffix="" Style="aaBb" /&gt;</s:String>
	<s:String x:Key="/Default/CodeStyle/Naming/CSharpNaming/UserRules/=f9fce829_002De6f4_002D4cb2_002D80f1_002D5497c44f51df/@EntryIndexedValue">&lt;Policy&gt;&lt;Descriptor Staticness="Static" AccessRightKinds="Private" Description="Static fields (private)"&gt;&lt;ElementKinds&gt;&lt;Kind Name="FIELD" /&gt;&lt;/ElementKinds&gt;&lt;/Descriptor&gt;&lt;Policy Inspect="True" Prefix="s_" Suffix="" Style="aaBb" /&gt;&lt;/Policy&gt;</s:String>
	<s:String x:Key="/Default/Environment/PerformanceGuide/SwitchBehaviour/=Antivirus/@EntryIndexedValue">LIVE_MONITOR</s:String>
	<s:Boolean x:Key="/Default/Environment/SettingsMigration/IsMigratorApplied/=JetBrains_002EReSharper_002EFeature_002EServices_002ECodeCleanup_002EFileHeader_002EFileHeaderSettingsMigrate/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/Environment/SettingsMigration/IsMigratorApplied/=JetBrains_002EReSharper_002EPsi_002ECSharp_002ECodeStyle_002ECSharpKeepExistingMigration/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/Environment/SettingsMigration/IsMigratorApplied/=JetBrains_002EReSharper_002EPsi_002ECSharp_002ECodeStyle_002ECSharpPlaceEmbeddedOnSameLineMigration/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/Environment/SettingsMigration/IsMigratorApplied/=JetBrains_002EReSharper_002EPsi_002ECSharp_002ECodeStyle_002ECSharpUseContinuousIndentInsideBracesMigration/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/Environment/SettingsMigration/IsMigratorApplied/=JetBrains_002EReSharper_002EPsi_002ECSharp_002ECodeStyle_002ESettingsUpgrade_002EMigrateBlankLinesAroundFieldToBlankLinesAroundProperty/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/Environment/SettingsMigration/IsMigratorApplied/=JetBrains_002EReSharper_002EPsi_002ECSharp_002ECodeStyle_002ESettingsUpgrade_002EPredefinedNamingRulesToUserRulesUpgrade/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/Housekeeping/ExcludedProjects/ProjectMasksToIgnore/=Himalia/@EntryIndexedValue">False</s:Boolean>
	<s:Boolean x:Key="/Default/Housekeeping/ExcludedProjects/ProjectMasksToIgnore/=_002AHimalia_002A/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/Housekeeping/ExcludedProjects/ProjectMasksToIgnore/=Himalia/@EntryIndexRemoved">True</s:Boolean>
	<s:Boolean x:Key="/Default/PatternsAndTemplates/Todo/TodoPatterns/=0A4664ADE151C349B62FB87D8470A5D9/@KeyIndexDefined">False</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=appsettings/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=autoscaling/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=datadog/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=Mongo/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=Oodle/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=REAPI/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=restsharp/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=Serilog/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=Thebe/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=timeuuid/@EntryIndexedValue">True</s:Boolean>
	<s:Boolean x:Key="/Default/UserDictionary/Words/=versioning/@EntryIndexedValue">True</s:Boolean>	
	<s:String x:Key="/Default/CodeStyle/FileHeader/FileHeaderText/@EntryValue">Copyright Epic Games, Inc. All Rights Reserved.&#xD;
</s:String>
	</wpf:ResourceDictionary>