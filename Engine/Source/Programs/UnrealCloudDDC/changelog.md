# Unreleased

* Added `ReadBuildPart` and `WriteBuildPart` actions - these allows access control per part based on its name for builds. This allows for more sensitive parts of builds to still be distributed as the same thing but those files can be excluded from some users. This is a very advanced use case and we generally recommend to just assign Read and WriteBuildPart to everyone like you would `ReadObject` and `WriteObject` .

# 1.4.0
Noteworthy and potentially breaking changes:
* All endpoints have been updated to use the new per-bucket authorization by default. Set the `Auth.UseLegacyConfiguration` setting to `true` to enable the legacy authentication behavior.
* Namespace-based authorization is deprecated, along with the `Auth.UseLegacyConfiguration` setting, and will be removed in a future release.
* The `ConnectionString` setting for replicators is no longer strictly required, allowing a `SourceSite` to be defined instead, which will look up the connection string of the relevant peer in the `cluster.peers` configuration. `SourceSite` will be respected over a `ConnectionString`, and an error will be thrown if neither are provided.
* The `ReferenceResolver.UseV2ReferenceResolving` setting is now enabled by default.
* The `UnrealCloudDDC.EnableForceSubmitRefBlobToBlobStore` setting is now disabled by default. When disabled, inlined blobs are only written to Scylla, significantly reducing the number of small blobs in S3. When `UnrealCloudDDC.EnableInlineSmallBlobs` is disabled (not recommended), then blobs are still written to S3.
* The `UnrealCloudDDC.EnableBucketStatsTracking` setting is now disabled by default, as it can lead to performance issues when querying sizes for many attachments. The strategy for collecting these stats will be reworked in a future release.
* Removed bucket enumeration endpoint (`GET /api/v1/refs/{ns}/{bucket}`) as its performance was inefficient and discouraged, and did not function correctly when `EnableBucketStatsTracking` was set to `false`.
* Removed the `UnrealCloudDDC.RequirePrivatePortForEnumeration` setting since its relevant endpoint has been removed.
* Regular expressions passed to the build search endpoint (`POST /api/v2/builds/{ns}/search`) now ignore the case of the provided `bucketRegex`.
* The `S3.MultiPartMaxMemoryBufferSize` setting has been reduced to `67108864` (was `int.MaxValue`).
* Added a service to periodically clean up temp files, controlled using the `GC.CleanTempFiles` setting (defaults to `true`).

New features:
* Added the `Scylla.UseConsistentContentIdResolve` setting, which uses local_quorum consistency when resolving content IDs.
* Added the `GET /api/v2/builds/{ns}/{bucket}/{buildId}/parts/{partId}/fileList` endpoint, useful for inspecting which files belong to a build.
* Added the `includePartial` option to `POST /api/v2/builds/{ns}/{bucket}/search` endpoint, used to include non-finalized builds to search results.
* Added the `S3.SiteBucketsPerStoragePool` setting, useful for defining which site-specific S3 bucket to use for each storage pool. 
* Added the `SourceSite` setting to each replicator, used to look up the connection URL using a peer by its `Name` in the `cluster.peers` configuration, rather than defining its `ConnectionString` explicitly.
* Added the `S3.DisablePayloadChecksums` setting, which reduces CPU usage at the cost of payload verification.
* Added the `GC.AllowBlockMetadataCleanup` setting (defaults to `true`).
* Added the `UnrealCloudDDC.RemoveReferencesOnBlobDeletion` setting (defaults to `true`).
* Added the `UnrealCloudDDCOrphanBlobCutoffDurationMinutes` setting (defaults to `60`).

Minor improvements:
* Added retries for Amazon S3 multipart uploads.
* Return 408 status codes when blob upload tasks are cancelled.
* Improved Amazon S3 rate limit handling.
* Improved blob replication stability.
* Return 503 status codes when the underlying database is unavailable.
* Added retries to build existence checks.
* Added multipart support for blob replication.
* Improved audit logging parallelism in `BuildsController`.
* Added string-to-number conversions for build search API when using compact binary objects.
* Increased the default maximum number of builds scanned during a search from `2000` to `20000`.
* Validate that all parts of a build are finalized before the build itself can be finalized.
* Use check-and-set pattern for blob replication state to allow state to be mutated externally.
* Added optimization to move files that are already buffered on the file system to their final destination, rather than copying the file by reading into an in-memory buffer.

Bug fixes:
* Fallback to file system buffering when unable to allocate enough memory, even for small payloads.
* Fixed ObjectId JSON serialization.
* Fixed retry logic for blob replication.
* Fixed an issue where last access background service was running more frequently than intended.
* Fixed NPE in `BuildsController` audit logging when no part ID or build ID was specified, but a namespace was.
* Removed redundant telemetry tracing scope in blob replication.
* Fixed an issue where setting `EnableInlineSmallBlobs` to `false` would incorrectly attempt to inline all blobs.
* Fixed incorrect dependency injection type in `ReplicationController`.

Helm chart changes:
* Dependency on `bitnami/common` is now vendored into the source tree.
* Changed the `HorizontalPodAutoscaler` to use container-specific metrics for CPU and memory rather than Pod-wide metrics.
* Fixed ConfigMap generation when omitting any `Jupiter` configuration.
* Added missing replicator version configuration to ConfigMap.

# 1.3.0
Noteworthy and potentially breaking changes:
* Failing upload of compressed buffers if more data then intended is sent.
If a valid compressed buffer was sent and then extra data appended to it at the end, Cloud DDC would accept the full data and not just the initial buffer sent and store that. This is not a frequently occurring issue but rather something we ran into during development of the builds api.
These kinds of uploads will now only store and serve the actual compressed buffer and discards the extra data.

* Extra services in helm chart now requires a `enabled` flag
This breaking change was introduced to make it easier to manage multiple helm charts with different extra services per file so that we could move most of their definitions to a common base file while then enabling them per region we deploy them in.

New features:
* [Experimental] This releases contains a initial version of the build api which we use for build distribution of staged and packaged builds as well as cooked output. This is not yet supported or documented and may still change.
* [Experimental] As part of build out the builds api we have a experimental dashboard for Cloud DDC with some limited functionality for admins. The source for this is not yet provided as we have yet to decide if we want to keep this around but you will see some references to it especially in the helm charts.
* S3 operations now attempt to use multipart for both GET and PUT, GET only works for blobs that were uploaded with multipart PUT. Mostly useful for large files in the builds api.
* Cloud DDC can now serve a yaml file that describes which endpoints are deployed and controls based on your public ip which instance to talk to. We use this to control which local cache servers and regional replicas to use. Only used by tools in the builds api.
* Authentication configuration can now be served up in a public unauthenticated endpoint, if you use this make sure that it doesn't include any sensitive information. We encrypt this data during transmission to obfuscate it but it should not be considered safe. This is useful to allow tools to authenticate gainst Cloud DDC without requiring the `oidc-configuration.json` file which is typically found in the workspace. Mostly used by the builds api and its related tools.
* Added option to prevent overwrites of refs per namespace, is opt in per namespace by setting AllowOverwritesOfRefs (defaults to true, old behavior). Can be overridden per request by specifying the X-Jupiter-Allow-Overwrite header (with value true). We are using this to help find and fix inconsistencies within DDC types. Will likely be made the default behavior in the future when we have addressed all the issues encountered.
* Introduced a new way to configure auth which is currently only used by the builds api (but will replace the method used for DDC as well in the future). This allows us to control access not just per namespace but also by applying regular expressions on the bucket names.

Minor improvements:
* Optimized reference resolving especially for very large refs (shader databases and oplogs)
* Avoid buffering payloads during a cache miss if the stream is seekable (so we can upload it to all layers that are missing it)
* Disabled scylla tracing by default as its very large volume , can be re-enabled using the OTEL_TRACE_SCYLLA_REQUESTS environment variable.
* Added UE-isBuildMachine header to list of headers we export to tracing
* Trim away extra whitespace in claims when checking if what we get from IdP matches what we expect. Makes it less annoying to configure the IdP as it can be quite hard to spot a extra whitespace in your configuration for some of them.
* Reconfigured forwarding headers to allow for more then one forwarding hop and removed assumption on proxy being on localhost. Makes the source ip of the connection correct in our audit logs.
* Added ability to GC the filesystem cache per namespace and to set a max size per namespace. This allows us to make sure that one busy namespace doesn't out compete another important but less frequently accessed namespace.
* Added namespace option to bypass the filesystem cache on write, useful for oplogs that are write heavy operations and were they are not always read. Gives better utilization of the filesystem cache.
* Returning a X-Jupiter-IoHash header when retrieving a compressed blob. This is the hash of the compressed content so it can be used by clients to verify that the content received matches with what is expected without decompressing it first.
* Changed the auth endpoint to use the new method for controlling auth to run its checks which also includes debug logging that explains the results.
* Deleting namespaces and buckets that are found to exist but not be used.
* Allow all origins to call into Jupiter in CORS - enabling people to use their own web apps to call the api. They will need to send auth tokens anyway so no need to prevent CORS.

Bug fixes:

* Various improvements to tracing and logging
* Improvements to temp file management including bug fixes and better debugging information.
* Improved the reliability of the replication tasks - especially during periods of very high load
* Fixed issue were status controllers didn't support requesting compact binary responses on its endpoints.
* Removed internal filter from admin controller, meaning that these apis can be accessed on the public endpoints (but requires a admin token). They are not super useful if they can not be easily accessed as they are usually used by humans and not by the system itself (which is were the internal filter is mostly useful).
* Fixed temp file leaks when using replication snapshots
* Include information on when the ref was created in the replication consistency check message so that we can determine if the issues we are seeing are due to new refs or old ones. (new ones would be expected to take a bit of time to replicate)
* Upgraded Serilog.Enrichers.ClientInfo to fix vulnerability warning
* Suppressing exceptions raised during ref store consistency check - we do not particularly care if a ref is in a partial state for this operation.
* Handle bucket already exists errors from S3 while creating the bucket (this indicates that we had two attempts at creating the bucket at the same time, so the second one fails, as we just care about the bucket existing we can safely ignore this error)
* Fixed issue were mongo content id comparisons were failing because .Equals is not supported in mongo queries for strings.
* Upgraded Blake3 version to fix inconsistency with EpicGames.Iohash
* Upgraded Microsoft.Extensions.Caching.Memory to 8.0.1 to fix build issues
* Fixed issue where the peer store would call into itself to see if the blob was present which makes no sense.
* Fix issue were the blob cleanup would fail if no blobs had been written to the filesystem yet.
* Always require requests to opt in to getting a redirect uri response even if its allowed to be returned for the namespace policy. This makes sure that only clients that actually supports this gets a redirect.
* Fixed issue were large blobs that could not be inlined was not added to the blob replication log.
* Fixed issue were settings applied to the Jupiter section were not being passed if you were also using nginx domain sockets. This especially impacted the 'PendingConnectionMax' setting which you bump when you also bump nginx values.
* Fixed issue were the speculative blob replication wouldn't work when combined with on demand replication (as it noticed that blobs were always available in other regions and thus already existed)
* Enabled minio browser on port 9001 when using the tests compose, useful to be able to verify the contents after tests have run.


Helm chart changes:
* Fixed issue were securityContexts for the pods was unable to be specified.
* Fixed issue with helm state always diffing causing ArgoCD to apply helm over and over again.
* Removed pod ip setting as its not always supported in k8s and we do not need it for anything right now.
* Upgraded dotnet-monitor to 8.0.5


# 1.2.0
* `InlineMaxBlobSize` option has moved from `Scylla` to `UnrealCloudDDC` as this can now be used to generally control if blob are inlined into the ref store (minimal practical difference as only Scylla supports inlining blobs). Resolves issue with `EnablePutRefBodyIntoBlobStore` option which will prevent these inlined blobs from also being added to the blob store.
* Added experimental option that is disabled by default to allow enumeration of buckets (enabled per namespace). Could potentially be used by oplogs.
* Optimizations for deleting blobs from multiple namespaces at once.
* Blob replication method added, can be enabled by setting the `Version` option under each replicator to `Blobs` - this will likely be made the default in future releases. The blob based replication is faster and more reliable then our old speculative replication as this has a more explicit list of which blobs to replicate rather then the previous method that was more indirect via the submitted ref (which can be mutated). This is also more useful for oplogs where a ref has a lot of references but few changed blobs each upload.
* Added support for new blob endpoint under refs as used in UE 5.6
* Added configuration for HPA (Horizontal pod autoscaler) in Helm chart. We use this to autoscale Cloud DDC instance a bit during bursty periods (in combination with a node autoscaler).
* Added option `AllowedNamespaces` on a authentication scheme to limit which namespaces a scheme is allowed to grant access to. This can be useful if you have a 3rd party authentication server you want to use but do not control and only want to grant access to some data for that party. For most use cases claims should be sufficent to control the access you need to grant.

# 1.1.1
* Fixes to helm chart when using ServiceAccounts for authentication and configuring replication

# 1.1.0
* Added ability to specify a access token if using ServiceAccounts to the ServiceCredentials (allowing for replication to function when no OIDC configuration is present)
* Fixed issue with blob stats using statements which did not work for CosmosDB by disabling them which means blob stats will not function for CosmosDB (this feature is not enabled by default anyway)
* Added ability to define a pod annotation that contains a checksum of the configmap, causing pods to get restarted when the config changes.
* Fixed issues with batch endpoints for compressed blob endpoints, not used by any production workloads yet.
* Speed up S3 blob listing for GC by listing them per S3 prefix. Slower for very small datasets were speed doesn't matter. Can be disabled with `S3.PerPrefixListing`
* Changed filesystem cleanup to just delete objects that are quite old rather then finding the oldest, makes it much more responsive and needs less state in memory at the cost of the cleanup being a bit more random but this is just the filecache anyway.
* Fixes to ondemand replication to avoid infinite recursion.

# 1.0.0
* .NET 8 Upgrade.
* Fixes for very large payloads (2GB+)
* Bug fixes for replication, mostly fixes to improve behavior when replication has fallen behind.
* Reduced GC pressure when processing a lot of new blobs that could cause very long stalls randomly (up to 1s response times).
* Speed up GC of refs when using Scylla.
* Tweaks to nginx configuration when using it as a reverse proxy.
* Reference store consistency check added, enabled via `ConsistencyCheck:EnableRefStoreChecks`. Can help repair inconsitencies between tables in Scylla which could happen due to bugs in earlier versions (resulting in data that is never garbage collected).
* Endpoints for serving symbol data that is compatible with MS Symbol server http api (but still requires auth which is not supported by Visual Studio out of the box). Experimental feature.

# 0.6.0
* Added option to track per bucket stats `EnableBucketStatsTracking`, this is still WIP.
* Added option to tweak nginx keep alive connections and increased it, only applies if using the nginx proxy.
* Added option to control TCP backlog in kestrel.
* Improved error message when no keyspace is set to Scylla
* Increased proxy timeout when using nginx, allowing for long operations (upload of oplogs) to not timeout.
* Added option to disable chunk encoding `S3.UseChunkEncoding` which needs to be set when using GCS.

# 0.5.2
* Fixed issue in helm charts defaulting to a incorrect docker registry path for the worker deployment.
* Added ability to configure and bumped number of keepalive connections in nginx, resolves issues during large spikes of traffic.

# 0.5.1
* Fixed issue in helm charts defaulting to a incorrect docker registry path.

# 0.5.0
* Fixed issue introduced in `0.4.0` that would cause last access tracking to not work correctly.
* Ability to run with auth enabled but using only the service account scheme - useful for a simpler setup as that does not require setup specific information.
* Improvments to scylla requests to avoid churning the scylla cache when doing GC.
* Fixed bug in the ref memory cache were overwrites would not correctly updated the local cache.
* Tweaks to scylla node connection limits to allow for more requests per connection.

# 0.4.0
* *Breaking* The scylla connection string now needs to include the default keyspace, a example connection string is  `Contact Points=your-scylla-dns.your-domain.com;;Default Keyspace=jupiter;`. The keyspace is `jupiter` is you are migrating from older releases. This allows you to also set the keyspace to something different if you want to run multiple instances of Unreal Cloud DDC against the same scylla cluster.)
* Migration options from `0.3.0` have been updated to assume you have migrated by default.
* Added `prepareNvmeFilesystem` section in Helm chart that creates a initContainer which will format a attached nvme drive.
* Fixed issue when content id remapped due to a new version, its supposed to use the smaller version but was in fact using the larger.
* Bug fixes to make the speculative replication more resilient.
* Ability to use nginx to sanatize http traffic and only use a level 4 load balancer in front as a performance improvment.
* Fixes for handling 2GB+ files.
* Fixes for on-demand replication.
* Support for S3 multipart uploads of large files.
* `Scylla.AvoidSchemaChanges` can now be set to avoid triggering schema modifications - this forces you to manually apply any schema changes required when upgrading but also means you can avoid triggering schema changes while maintinance is running.
* Added `MetricsService` which is disabled by default, will scan all data to determine things like number of objects in each bucket and sizes. Puts a fairly heavy load on your DB so is disabled by default.
* Added GC of non-finalized refs at a more aggresive cadence then normal refs (removed when they are 2 hours old)
  
# 0.3.0
* Azure blob storage now supports storage pools
* Last access table refactoring - Moved the last accessing tracking out of the objects table and into a seperate table. Saves on compation work for Scylla. Set `Scylla.ListObjectsFromLastAccessTable` to migrate GC refs to use this new table (will be default in the next release).
* Blob index table refactoring - Multiple changes to the blob index table, avoiding collections in the scylla rows and instead prefering clustering keys. This reduces amount of work that needs to happen when updating regions or references to blobs and improves performance (and caching) on the DB. We automatically migrate old blob index entiries when `Scylla.MigrateFromOldBlobIndex` is set (it is by default). We will start requring data in these tables by the next release. 
* Bucket table refactoring - Reads data from old and new table for this release. Set `Scylla.ListObjectsFromOldNamespaceTable` to disable reading the old table.
* Generally reduced some of the excessive logging from the worker in favor of open telemetry metrics instead.
* GC Rules - Added ability to disable GC or configure it to use Time-To-Live instead of last access tracking.
* FallbackNamespaces added - allows you to configure a second namespace used for blob operations if blobs are missing from the first. This can be used to store Virtual Assets in a seperate store pool that have different GC rules.
* Added ability to use presigned urls when uploading / downloading blobs
* If using a nginx reverse proxy on the same host we can support redirecting filesystem reads to it (X-Accel).
* Fixed issue with compact binary packages not being serialized correctly

# 0.2.0
* Improved handling of cassandra timeouts during ref GC.
* Added deletion of invalid references during blob GC - this reduces the size for entries in the blob_index table.
* Using OpenTelemetry for tracing - still supports datadog traces but this can now be forwarded to any OpenTelemetry compatible service.
* Fixed issue in Azure Blob storage with namespace containing _
* Optimized content id resolving
* Fixed issue with crash on startup from background services for certain large configuration objects
* Fixed issues with content id when using CosmosDB

# 0.1.0
* First public release of UnrealCloudDDC

# Older releases
No detailed changelog provided for older releases.