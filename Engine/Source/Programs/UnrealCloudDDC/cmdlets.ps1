function Start-Minio {
    $env:MINIO_ACCESS_KEY="FSYPTVCPKC2IV5D1U8RE"
    $env:MINIO_SECRET_KEY="WaUf4Zq+J6YMlvbCVvTRkQJIPc7DvZwTm5UP0tPJ"
    minio server .minio
}

function Start-Scylla {
    docker run --name some-scylla -it --rm --hostname some-scylla -p 9042:9042 scylladb/scylla --smp 1
}

function Start-DotnetMonitor {
    param (
        [Parameter(Mandatory)] [string] $podName,
        [int] $localPort = 52323,
        [int] $remotePort = 52323
    )
    
    if (-Not ($null -eq $global:dotnetMonitorPortForwardJob))
    {
        Write-Host "Dotnet Monitor Port Forwarding is already running"  -ForegroundColor Yellow
        return
    }
    $global:dotnetMonitorPortForwardJob = Start-Job -ScriptBlock { 
        param($podName, $localPort, $remotePort)
        Write-Host "kubectl port-forward `"pods/$podName`" `"$localPort`:$remotePort`" "
        kubectl port-forward "pods/$podName" "$localPort`:$remotePort" 
    } -ArgumentList $podName, $localPort, $remotePort

    Write-Host "Dotnet Monitor Port Forwarding started on port `"$localPort`" to pod `"$podName`". Run Stop-DotnetMonitor to stop it."  -ForegroundColor Yellow        
    Write-Host "Run curl http://localhost:52323/trace?profile=cpu,http,metrics to generate a nettrace ."        

    # Use Receive-Job $global:dotnetMonitorPortForwardJob to fetch the stdout of the background task
}

function Stop-DotnetMonitor {
    $job = $global:dotnetMonitorPortForwardJob

    if ($null -eq $job)
    {
        Write-Host "Dotnet Monitor Port Forwarding does not seem to be running"  -ForegroundColor Red
        return
    }
    
    Stop-Job $job
    Remove-Job $job

    $global:dotnetMonitorPortForwardJob = $null

    Write-Host "Dotnet Monitor Port Forwarding was stopped"       
}

# Uses Get-FileHash to calculate the hash of a string
function Get-StringHash {
    [CmdletBinding()]
    param (
        [Parameter(Mandatory, position=0)] [string] $stringToHash,
        [Parameter(Mandatory, position=1)] [string] $algorithm,
        [parameter(mandatory=$false, position=2, ValueFromRemainingArguments=$true)] $unboundArgs
    )

    if ($null -eq $unboundArgs) {
        $unboundArgs = @{}
    }

    $stringAsStream = [System.IO.MemoryStream]::new()
    $writer = [System.IO.StreamWriter]::new($stringToHash)
    $writer.write($stringToHash)
    $writer.Flush()
    $stringAsStream.Position = 0
    Get-FileHash -InputStream $stringAsStream -Algorithm $algorithm @unboundArgs | Select-Object Hash
}
