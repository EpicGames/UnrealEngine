# syntax=docker/dockerfile:1.2

FROM mcr.microsoft.com/dotnet/sdk:10.0 AS build

WORKDIR /app

ARG UE_P4_CHANGELIST=0

# Copy csprojs for nuget restore
COPY Programs/Shared/* Programs/Shared/
COPY Programs/Shared/EpicGames.Core/*.csproj ./Programs/Shared/EpicGames.Core/
COPY Programs/Shared/EpicGames.AspNet/*.csproj ./Programs/Shared/EpicGames.AspNet/
COPY Programs/Shared/EpicGames.IoHash/*.csproj ./Programs/Shared/EpicGames.IoHash/
COPY Programs/Shared/EpicGames.Oodle/*.csproj ./Programs/Shared/EpicGames.Oodle/
COPY Programs/Shared/EpicGames.Serialization/*.csproj ./Programs/Shared/EpicGames.Serialization/
COPY Programs/UnrealCloudDDC/Jupiter.Common/*.csproj ./Programs/UnrealCloudDDC/Jupiter.Common/
COPY Programs/UnrealCloudDDC/Jupiter/*.csproj ./Programs/UnrealCloudDDC/Jupiter/
RUN dotnet restore Programs/UnrealCloudDDC/Jupiter/Jupiter.csproj
# all nuget prereqs are now done

# copy the source we need
COPY Programs/Shared/ Programs/Shared/
COPY Programs/UnrealCloudDDC/ Programs/UnrealCloudDDC/

# build and create the dll
RUN dotnet publish -c Release -o out Programs/UnrealCloudDDC/Jupiter/Jupiter.csproj

# setup the base version of UnrealCloudDDC
FROM mcr.microsoft.com/dotnet/aspnet:10.0 AS unreal-cloud-ddc-base
WORKDIR /app
COPY --from=build /app/out ./
COPY Programs/UnrealCloudDDC/ThirdParty/license.txt license.txt
LABEL org.opencontainers.image.documentation https://github.com/EpicGames/UnrealEngine/blob/ue5-main/Engine/Source/Programs/UnrealCloudDDC/README.md
LABEL org.opencontainers.image.licenses "MIT"

# create the actual deployable slim image
FROM unreal-cloud-ddc-base AS unreal-cloud-ddc-slim
ENTRYPOINT dotnet Jupiter.dll

# add some useful debug tooling
FROM unreal-cloud-ddc-base AS unreal-cloud-ddc
RUN apt-get update &&\ 
    apt-get install curl unzip procps -y &&\
    rm -rf /var/lib/apt/lists/* &&\
    curl -sSL https://aka.ms/getvsdbgsh | /bin/sh /dev/stdin -v latest -l /vsdbg
ENTRYPOINT dotnet Jupiter.dll

FROM unreal-cloud-ddc-base AS unreal-cloud-ddc-datadog
# Add the debugger from the normal UnrealCloudDDC build
RUN apt-get update &&\ 
    apt-get install curl unzip procps -y &&\
    rm -rf /var/lib/apt/lists/* &&\
    curl -sSL https://aka.ms/getvsdbgsh | /bin/sh /dev/stdin -v latest -l /vsdbg
# Add the datadog tracer
RUN curl -LO https://github.com/DataDog/dd-trace-dotnet/releases/download/v3.34.0/datadog-dotnet-apm_3.34.0_amd64.deb
RUN dpkg -i ./datadog-dotnet-apm_3.34.0_amd64.deb
RUN /opt/datadog/createLogPath.sh

# datadog APM requirments
ENV CORECLR_ENABLE_PROFILING=1
ENV CORECLR_PROFILER={846F5F1C-F9AE-4B07-969E-05C26BC060D8}
ENV CORECLR_PROFILER_PATH=/opt/datadog/Datadog.Trace.ClrProfiler.Native.so
ENV DD_INTEGRATIONS=/opt/datadog/integrations.json
ENV DD_DOTNET_TRACER_HOME=/opt/datadog
# enable datadog continus profiler
ENV LD_PRELOAD=/opt/datadog/continuousprofiler/Datadog.Linux.ApiWrapper.x64.so

# Disable automatic instrumentation of APM and log injection
# We use open telemetry to send the APM stats, we have this tracer for other features like runtime metrics
ENV DD_TRACE_ENABLED=false
ENV DD_LOGS_INJECTION=false
# do not consider http 400-499 statuses errors
ENV DD_HTTP_CLIENT_ERROR_STATUSES=500-599

ENV DD_RUNTIME_METRICS_ENABLED=true
ENTRYPOINT dotnet Jupiter.dll