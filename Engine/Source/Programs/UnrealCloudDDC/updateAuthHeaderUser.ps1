$TempFile = New-TemporaryFile

& "dotnet" "$PSScriptRoot\..\..\..\..\Engine\Binaries\DotNET\OidcToken\portable\OidcToken.dll" "--Service=EpicGames-Okta" "--OutFile" $TempFile.FullName 
$env:curl_auth_header = Get-Content -Raw -Path $TempFile.FullName | ConvertFrom-Json | ForEach-Object { "Authorization: Bearer $($_.Token)" }
$env:auth_token = Get-Content -Raw -Path $TempFile.FullName | ConvertFrom-Json | ForEach-Object { "$($_.Token)" }