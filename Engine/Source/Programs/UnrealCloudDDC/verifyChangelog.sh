#!/bin/bash
dir=$(dirname "${BASH_SOURCE[0]}")
echo "Looking for files in $dir"

current_version=$(yq e '.version' "$dir/version.yaml")
echo "Current Version on disk is: '$current_version'"

if grep -qF "$current_version" "$dir/changelog.md";  then
    echo "Version found '$current_version'"
else
    echo "No changelog section found for release $current_version" >&2
    exit 1
fi

release_version=$1
if [[ -n "$release_version" ]]; then    
    
    if [[ "$current_version" != "$release_version" ]]; then    
        echo "Mismatching version numbers found. Version specified: $release_version while version on disk is $current_version" >&2
        exit 2
    fi

    if grep -qF "$release_version" "$dir/changelog.md"; then
        echo "Release Version found '$release_version'"
    else
        echo "No changelog section found for release $release_version" >&2
        exit 1
    fi
fi

